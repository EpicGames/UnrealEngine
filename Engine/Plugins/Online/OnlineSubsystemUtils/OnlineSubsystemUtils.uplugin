{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem Utils",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Shared code for interacting online service and online subsystem implementations.",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : true,

    "Modules": [
        {
            "Name": "OnlineSubsystemUtils",
            "Type": "Runtime",
            "LoadingPhase": "PostConfigInit"
        },
        {
            "Name": "OnlineBlueprintSupport",
            "Type": "UncookedOnly",
            "LoadingPhase": "PostConfigInit"
        }
    ],
	"LocalizationTargets":
	[
		{
			"Name": "OnlineSubsystemUtils",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins":
	[
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineServices",
			"Enabled": true
		}
	]
}