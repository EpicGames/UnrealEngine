// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"
#include "K2Node_InAppPurchaseRestore2.generated.h"

UCLASS(MinimalAPI)
class UK2Node_InAppPurchaseRestore2 : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};
