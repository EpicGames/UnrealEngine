// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"
#include "K2Node_LeaderboardFlush.generated.h"

#define UE_API ONLINEBLUEPRINTSUPPORT_API

UCLASS(MinimalAPI)
class UK2Node_LeaderboardFlush : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	UE_API virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface
};

#undef UE_API
