// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"
#include "K2Node_LeaderboardQuery.generated.h"

UCLASS(MinimalAPI)
class UK2Node_LeaderboardQuery : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};
