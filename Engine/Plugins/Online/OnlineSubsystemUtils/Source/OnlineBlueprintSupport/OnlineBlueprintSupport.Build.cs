// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OnlineBlueprintSupport : ModuleRules
{
	public OnlineBlueprintSupport(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"BlueprintGraph",
				"Core",
				"CoreUObject",
				"Engine",
				"OnlineSubsystemUtils",
			}
		);
	}
}
