// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseCheckout.h"
#include "InAppPurchaseCheckoutCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseCheckout)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseCheckout::UK2Node_InAppPurchaseCheckout(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseCheckoutCallbackProxy, CreateProxyObjectForInAppPurchaseCheckout);
	ProxyFactoryClass = UInAppPurchaseCheckoutCallbackProxy::StaticClass();

	ProxyClass = UInAppPurchaseCheckoutCallbackProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE

