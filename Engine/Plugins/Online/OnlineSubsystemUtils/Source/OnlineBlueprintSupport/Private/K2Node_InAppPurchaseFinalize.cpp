// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseFinalize.h"
#include "InAppPurchaseFinalizeProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseFinalize)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseFinalize::UK2Node_InAppPurchaseFinalize(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseFinalizeProxy, CreateProxyObjectForInAppPurchaseFinalize);
	ProxyFactoryClass = UInAppPurchaseFinalizeProxy::StaticClass();

	ProxyClass = UInAppPurchaseFinalizeProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE

