// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseGetKnownReceipts.h"
#include "InAppPurchaseReceiptsCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseGetKnownReceipts)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseGetKnownReceipts::UK2Node_InAppPurchaseGetKnownReceipts(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseReceiptsCallbackProxy, CreateProxyObjectForInAppPurchaseGetKnownReceipts);
	ProxyFactoryClass = UInAppPurchaseReceiptsCallbackProxy::StaticClass();

	ProxyClass = UInAppPurchaseReceiptsCallbackProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE

