// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseQuery2.h"
#include "InAppPurchaseQueryCallbackProxy2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseQuery2)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseQuery2::UK2Node_InAppPurchaseQuery2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseQueryCallbackProxy2, CreateProxyObjectForInAppPurchaseQuery);
	ProxyFactoryClass = UInAppPurchaseQueryCallbackProxy2::StaticClass();

	ProxyClass = UInAppPurchaseQueryCallbackProxy2::StaticClass();
}

#undef LOCTEXT_NAMESPACE

