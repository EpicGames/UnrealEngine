// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseQueryOwnedProducts.h"
#include "InAppPurchaseReceiptsCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseQueryOwnedProducts)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseQueryOwnedProducts::UK2Node_InAppPurchaseQueryOwnedProducts(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseReceiptsCallbackProxy, CreateProxyObjectForInAppPurchaseQueryOwnedProducts);
	ProxyFactoryClass = UInAppPurchaseReceiptsCallbackProxy::StaticClass();

	ProxyClass = UInAppPurchaseReceiptsCallbackProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE

