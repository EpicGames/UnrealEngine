// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseRestore2.h"
#include "InAppPurchaseRestoreCallbackProxy2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseRestore2)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseRestore2::UK2Node_InAppPurchaseRestore2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseRestoreCallbackProxy2, CreateProxyObjectForInAppPurchaseRestore);
	ProxyFactoryClass = UInAppPurchaseRestoreCallbackProxy2::StaticClass();

	ProxyClass = UInAppPurchaseRestoreCallbackProxy2::StaticClass();
}

#undef LOCTEXT_NAMESPACE

