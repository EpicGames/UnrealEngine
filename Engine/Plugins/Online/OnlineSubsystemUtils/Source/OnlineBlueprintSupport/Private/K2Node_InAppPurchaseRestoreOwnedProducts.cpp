// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseRestoreOwnedProducts.h"
#include "InAppPurchaseReceiptsCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseRestoreOwnedProducts)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseRestoreOwnedProducts::UK2Node_InAppPurchaseRestoreOwnedProducts(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseReceiptsCallbackProxy, CreateProxyObjectForInAppPurchaseRestoreOwnedProducts);
	ProxyFactoryClass = UInAppPurchaseReceiptsCallbackProxy::StaticClass();

	ProxyClass = UInAppPurchaseReceiptsCallbackProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE

