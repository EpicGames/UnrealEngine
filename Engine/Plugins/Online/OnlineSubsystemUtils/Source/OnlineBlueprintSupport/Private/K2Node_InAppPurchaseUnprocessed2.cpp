// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InAppPurchaseUnprocessed2.h"
#include "InAppPurchaseCallbackProxy2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InAppPurchaseUnprocessed2)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_InAppPurchaseUnprocessed2::UK2Node_InAppPurchaseUnprocessed2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UInAppPurchaseCallbackProxy2, CreateProxyObjectForInAppPurchaseUnprocessedPurchases);
	ProxyFactoryClass = UInAppPurchaseCallbackProxy2::StaticClass();

	ProxyClass = UInAppPurchaseCallbackProxy2::StaticClass();
}

#undef LOCTEXT_NAMESPACE

