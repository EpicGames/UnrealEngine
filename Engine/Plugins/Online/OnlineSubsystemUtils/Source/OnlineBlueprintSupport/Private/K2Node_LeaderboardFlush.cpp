// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_LeaderboardFlush.h"
#include "LeaderboardFlushCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_LeaderboardFlush)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_LeaderboardFlush::UK2Node_LeaderboardFlush(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(ULeaderboardFlushCallbackProxy, CreateProxyObjectForFlush);
	ProxyFactoryClass = ULeaderboardFlushCallbackProxy::StaticClass();

	ProxyClass = ULeaderboardFlushCallbackProxy::StaticClass();
}

FText UK2Node_LeaderboardFlush::GetTooltipText() const
{
	return LOCTEXT("K2Node_LeaderboardFlush_Tooltip", "Flushes leaderboards for a session");
}

FText UK2Node_LeaderboardFlush::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("FlushLeaderboards", "Flush Leaderboards");
}

FText UK2Node_LeaderboardFlush::GetMenuCategory() const
{
	return LOCTEXT("LeaderboardFlushCategory", "Online|Leaderboard");
}

#undef LOCTEXT_NAMESPACE

