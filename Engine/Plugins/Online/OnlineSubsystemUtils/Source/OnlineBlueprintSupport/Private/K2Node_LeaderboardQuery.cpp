// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_LeaderboardQuery.h"
#include "LeaderboardQueryCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_LeaderboardQuery)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_LeaderboardQuery::UK2Node_LeaderboardQuery(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(ULeaderboardQueryCallbackProxy, CreateProxyObjectForIntQuery);
	ProxyFactoryClass = ULeaderboardQueryCallbackProxy::StaticClass();

	ProxyClass = ULeaderboardQueryCallbackProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE

