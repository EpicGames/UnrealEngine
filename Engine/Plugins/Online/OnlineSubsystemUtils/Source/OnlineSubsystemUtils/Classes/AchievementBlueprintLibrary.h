// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AchievementBlueprintLibrary.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class APlayerController;

// Library of synchronous achievement calls
UCLASS(MinimalAPI, meta=(ScriptName="AchievementLibrary"))
class UAchievementBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	 * Gets the status of an achievement ID (you must call CacheAchievements first to cache them)
	 *
	 * @param AchievementID - The id of the achievement we are looking up
	 * @param bFoundID - If the ID was found in the cache (if not, none of the other values are meaningful)
	 * @param Progress - The progress amount of the achievement
	 */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject"), Category = "Online|Achievements")
	static UE_API void GetCachedAchievementProgress(UObject* WorldContextObject, APlayerController* PlayerController, FName AchievementID, /*out*/ bool& bFoundID, /*out*/ float& Progress);

	/**
	 * Get the description for an achievement ID (you must call CacheAchievementDescriptions first to cache them)
	 *
	 * @param AchievementID - The id of the achievement we are searching for data of
	 * @param bFoundID - If the ID was found in the cache (if not, none of the other values are meaningful)
	 * @param Title - The localized title of the achievement
	 * @param LockedDescription - The localized locked description of the achievement
	 * @param UnlockedDescription - The localized unlocked description of the achievement
	 * @param bHidden - Whether the achievement is hidden
	 */
	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject"), Category="Online|Achievements")
	static UE_API void GetCachedAchievementDescription(UObject* WorldContextObject, APlayerController* PlayerController, FName AchievementID, /*out*/ bool& bFoundID, /*out*/ FText& Title, /*out*/ FText& LockedDescription, /*out*/ FText& UnlockedDescription, /*out*/ bool& bHidden);
};

#undef UE_API
