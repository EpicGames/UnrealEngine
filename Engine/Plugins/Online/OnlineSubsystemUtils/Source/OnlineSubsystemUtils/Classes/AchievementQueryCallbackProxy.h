// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Net/OnlineBlueprintCallProxyBase.h"
#include "AchievementQueryCallbackProxy.generated.h"

class APlayerController;
class FUniqueNetId;

UCLASS(MinimalAPI)
class UAchievementQueryCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful query
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when there is an unsuccessful query
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// Fetches and caches achievement progress from the default online subsystem
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Achievements")
	static UAchievementQueryCallbackProxy* CacheAchievements(UObject* WorldContextObject, class APlayerController* PlayerController);

	// Fetches and caches achievement descriptions from the default online subsystem
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Achievements")
	static UAchievementQueryCallbackProxy* CacheAchievementDescriptions(UObject* WorldContextObject, class APlayerController* PlayerController);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

private:
	// Internal callback when the achievement query completes, calls out to the public success/failure callbacks
	void OnQueryCompleted(const FUniqueNetId& UserID, bool bSuccess);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// Are we querying achievement progress or achievement descriptions?
	bool bFetchDescriptions;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
