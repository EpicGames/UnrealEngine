// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OnlineStats.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "AchievementWriteCallbackProxy.generated.h"

class APlayerController;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FAchievementWriteDelegate, FName, WrittenAchievementName, float, WrittenProgress, int32, WrittenUserTag);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FAchievementWriteCompleteDelegate, FString, WrittenAchievementName, float, WrittenProgress, int32, WrittenUserTag);

UCLASS(MinimalAPI)
class UAchievementWriteCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful achievement write
	UPROPERTY(BlueprintAssignable)
	FAchievementWriteCompleteDelegate OnWriteSuccess;

	// Called when there is an unsuccessful achievement write
	UPROPERTY(BlueprintAssignable)
	FAchievementWriteCompleteDelegate OnWriteFailure;

	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category="Online|Achievements")
	static UAchievementWriteCallbackProxy* WriteProgress(UObject* WorldContextObject, APlayerController* PlayerController, FString AchievementName, float Progress = 100.0f, int32 UserTag = 0);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

	// UObject interface
	virtual void BeginDestroy() override;
	// End of UObject interface

private:
	// Internal callback when the achievement is written, calls out to the public success/failure callbacks
	void OnAchievementWritten(const FUniqueNetId& UserID, bool bSuccess);

private:
	/** The player controller triggering things */
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	/** The achievements write object */
	FOnlineAchievementsWritePtr WriteObject;

	/** The achievement name */
	FString AchievementName;

	/** The amount of progress made towards the achievement */
	float AchievementProgress;

	/** The specified user tag */
	int32 UserTag;

	/** The world context object in which this call is taking place */
	UObject* WorldContextObject;

};
