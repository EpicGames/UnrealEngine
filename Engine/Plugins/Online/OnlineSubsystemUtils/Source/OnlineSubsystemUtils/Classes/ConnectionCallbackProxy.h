// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineIdentityInterface.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "ConnectionCallbackProxy.generated.h"

class APlayerController;
class Error;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnlineConnectionResult, int32, ErrorCode);


UCLASS(MinimalAPI)
class UConnectionCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful query
	UPROPERTY(BlueprintAssignable)
	FOnlineConnectionResult OnSuccess;

	// Called when there is an unsuccessful query
	UPROPERTY(BlueprintAssignable)
	FOnlineConnectionResult OnFailure;

	// Connects to an online service such as Google Play
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage="Please use Show External Login UI instead", BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Achievements")
	static UConnectionCallbackProxy* ConnectToService(UObject* WorldContextObject, class APlayerController* PlayerController);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

private:
	// Internal callback when the achievement query completes, calls out to the public success/failure callbacks
	void OnLoginCompleted(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// Connection callback delegate
	FOnLoginCompleteDelegate OnLoginCompleteDelegate;

	// OnLoginComplete delegate handle
	FDelegateHandle OnLoginCompleteDelegateHandle;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
