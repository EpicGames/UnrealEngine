// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlineSessionDelegates.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "CreateSessionCallbackProxy.generated.h"

class APlayerController;

UCLASS(MinimalAPI)
class UCreateSessionCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when the session was created successfully
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when there was an error creating the session
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// Creates a session with the default online subsystem
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Session")
	static UCreateSessionCallbackProxy* CreateSession(UObject* WorldContextObject, class APlayerController* PlayerController, int32 PublicConnections, bool bUseLAN, bool bUseLobbiesIfAvailable = true);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

private:
	// Internal callback when session creation completes, calls StartSession
	void OnCreateCompleted(FName SessionName, bool bWasSuccessful);

	// Internal callback when session creation completes, calls StartSession
	void OnStartCompleted(FName SessionName, bool bWasSuccessful);

	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The delegate executed by the online subsystem
	FOnCreateSessionCompleteDelegate CreateCompleteDelegate;

	// The delegate executed by the online subsystem
	FOnStartSessionCompleteDelegate StartCompleteDelegate;

	// Handles to the registered delegates above
	FDelegateHandle CreateCompleteDelegateHandle;
	FDelegateHandle StartCompleteDelegateHandle;

	// Number of public connections
	int NumPublicConnections;

	// Whether or not to search LAN
	bool bUseLAN;

	// Whether or not to use lobbies (as opposed to using a raw server list)
	bool bUseLobbiesIfAvailable;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
