// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlineSessionDelegates.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "DestroySessionCallbackProxy.generated.h"

class APlayerController;

UCLASS(MinimalAPI)
class UDestroySessionCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful destroy
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when there is an unsuccessful destroy
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// Destroys a session previously created by the default online subsystem
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Session")
	static UDestroySessionCallbackProxy* DestroySession(UObject* WorldContextObject, class APlayerController* PlayerController);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

private:
	// Internal callback when the operation completes, calls out to the public success/failure callbacks
	void OnCompleted(FName SessionName, bool bWasSuccessful);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The delegate executed by the online subsystem
	FOnDestroySessionCompleteDelegate Delegate;

	// Handle to the registered OnDestroySessionComplete delegate
	FDelegateHandle DelegateHandle;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
