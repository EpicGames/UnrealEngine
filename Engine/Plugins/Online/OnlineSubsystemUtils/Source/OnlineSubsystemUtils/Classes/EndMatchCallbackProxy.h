// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/OnlineBlueprintCallProxyBase.h"
#include "EndMatchCallbackProxy.generated.h"

namespace EMPMatchOutcome { enum Outcome : int; }
template <typename InterfaceType> class TScriptInterface;

class APlayerController;
class ITurnBasedMatchInterface;
class UTurnBasedMatchInterface;

UCLASS(MinimalAPI)
class UEndMatchCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	virtual ~UEndMatchCallbackProxy();

	// Called when the match ends successfully
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when ending the match fails
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// End a match that is in progress while it is the current player's turn
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UEndMatchCallbackProxy* EndMatch(UObject* WorldContextObject, class APlayerController* PlayerController, TScriptInterface<ITurnBasedMatchInterface> MatchActor, FString MatchID, EMPMatchOutcome::Outcome LocalPlayerOutcome, EMPMatchOutcome::Outcome OtherPlayersOutcome);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

	UTurnBasedMatchInterface* GetTurnBasedMatchInterfaceObject() { return TurnBasedMatchInterface; }

	void EndMatchDelegate(FString MatchID, bool Succeeded);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;

	// TurnBasedMatchInterface object, used to set the match data after a match is found
	UTurnBasedMatchInterface* TurnBasedMatchInterface;

	// ID of the match to end
	FString MatchID;
	
	// Match outcome for the current player (win/loss/tie)
	EMPMatchOutcome::Outcome LocalPlayerOutcome;
	
	// Match outcome for all other players (win/loss/tie)
	EMPMatchOutcome::Outcome OtherPlayersOutcome;
};
