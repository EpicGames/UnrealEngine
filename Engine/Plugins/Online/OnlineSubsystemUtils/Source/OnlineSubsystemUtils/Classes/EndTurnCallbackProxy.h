// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/OnlineBlueprintCallProxyBase.h"
#include "EndTurnCallbackProxy.generated.h"

class ITurnBasedMatchInterface;
class UTurnBasedMatchInterface;
template <typename InterfaceType> class TScriptInterface;

class APlayerController;

UCLASS(MinimalAPI)
class UEndTurnCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	virtual ~UEndTurnCallbackProxy();

	// Called when there is a successful query
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when there is an unsuccessful query
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// Ends the turn for the current player
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UEndTurnCallbackProxy* EndTurn(UObject* WorldContextObject, class APlayerController* PlayerController, FString MatchID, TScriptInterface<ITurnBasedMatchInterface> TurnBasedMatchInterface);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

    void UploadMatchDataDelegate(FString MatchID, bool Succeeded);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;

	// The MatchID of the active match
	FString MatchID;

	// Object that is inheriting from UTurnBasedMatchInterface. Replicated properties will be serialize into the platform specific match data object.
	UTurnBasedMatchInterface* TurnBasedMatchInterface;
};
