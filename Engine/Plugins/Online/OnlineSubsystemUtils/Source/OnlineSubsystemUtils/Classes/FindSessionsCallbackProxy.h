// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlineSessionDelegates.h"
#include "OnlineSessionSettings.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "FindSessionsCallbackProxy.generated.h"

class APlayerController;

USTRUCT(BlueprintType)
struct FBlueprintSessionResult
{
	GENERATED_USTRUCT_BODY()

	FOnlineSessionSearchResult OnlineResult;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FBlueprintFindSessionsResultDelegate, const TArray<FBlueprintSessionResult>&, Results);

UCLASS(MinimalAPI)
class UFindSessionsCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful query
	UPROPERTY(BlueprintAssignable)
	FBlueprintFindSessionsResultDelegate OnSuccess;

	// Called when there is an unsuccessful query
	UPROPERTY(BlueprintAssignable)
	FBlueprintFindSessionsResultDelegate OnFailure;

	// Searches for advertised sessions with the default online subsystem
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Session")
	static UFindSessionsCallbackProxy* FindSessions(UObject* WorldContextObject, class APlayerController* PlayerController, int32 MaxResults, bool bUseLAN, bool bUseLobbies);

	UFUNCTION(BlueprintPure, Category = "Online|Session")
	static int32 GetPingInMs(const FBlueprintSessionResult& Result);

	UFUNCTION(BlueprintPure, Category = "Online|Session")
	static FString GetServerName(const FBlueprintSessionResult& Result);

	UFUNCTION(BlueprintPure, Category = "Online|Session")
	static int32 GetCurrentPlayers(const FBlueprintSessionResult& Result);

	UFUNCTION(BlueprintPure, Category = "Online|Session")
	static int32 GetMaxPlayers(const FBlueprintSessionResult& Result);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

private:
	// Internal callback when the session search completes, calls out to the public success/failure callbacks
	void OnCompleted(bool bSuccess);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The delegate executed by the online subsystem
	FOnFindSessionsCompleteDelegate Delegate;

	// Handle to the registered OnFindSessionsComplete delegate
	FDelegateHandle DelegateHandle;

	// Object to track search results
	TSharedPtr<FOnlineSessionSearch> SearchObject;

	// Whether or not to search LAN
	bool bUseLAN;

	// Whether or not to search for Lobbies instead of Sessions
	bool bUseLobbies;

	// Maximum number of results to return
	int MaxResults;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
