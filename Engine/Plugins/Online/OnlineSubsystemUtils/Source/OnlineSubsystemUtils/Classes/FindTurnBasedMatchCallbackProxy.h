// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineTurnBasedInterface.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "FindTurnBasedMatchCallbackProxy.generated.h"

template <typename InterfaceType> class TScriptInterface;

class APlayerController;
class ITurnBasedMatchInterface;
class UFindTurnBasedMatchCallbackProxy;
class UTurnBasedMatchInterface;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnlineTurnBasedMatchResult, FString, MatchID);

class FFindTurnBasedMatchCallbackProxyMatchmakerDelegate : public FTurnBasedMatchmakerDelegate, public TSharedFromThis<FTurnBasedMatchmakerDelegate>
{
public:

	FFindTurnBasedMatchCallbackProxyMatchmakerDelegate();
	virtual ~FFindTurnBasedMatchCallbackProxyMatchmakerDelegate();

	virtual void OnMatchmakerCancelled() override;
	virtual void OnMatchmakerFailed() override;
	virtual void OnMatchFound(FTurnBasedMatchRef Match) override;

	void SetFindTurnBasedMatchCallbackProxy(UFindTurnBasedMatchCallbackProxy* InFindTurnBasedMatchCallbackProxy) { FindTurnBasedMatchCallbackProxy = InFindTurnBasedMatchCallbackProxy; }
	void SetTurnBasedInterface(IOnlineTurnBasedPtr InTurnBasedInterface) { TurnBasedInterface = InTurnBasedInterface; }
private:
	UFindTurnBasedMatchCallbackProxy* FindTurnBasedMatchCallbackProxy;
	IOnlineTurnBasedPtr TurnBasedInterface;
};

UCLASS(MinimalAPI)
class UFindTurnBasedMatchCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

    virtual ~UFindTurnBasedMatchCallbackProxy();
    
	// Called when matchmaking succeeded.
	UPROPERTY(BlueprintAssignable)
	FOnlineTurnBasedMatchResult OnSuccess;

	// Called when matchmaking failed
	UPROPERTY(BlueprintAssignable)
	FOnlineTurnBasedMatchResult OnFailure;

	// Use the platform matchmaking service (like Game Center) to find a match.
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UFindTurnBasedMatchCallbackProxy* FindTurnBasedMatch(UObject* WorldContextObject, class APlayerController* PlayerController, TScriptInterface<ITurnBasedMatchInterface> MatchActor, int32 MinPlayers, int32 MaxPlayers, int32 PlayerGroup, bool ShowExistingMatches);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

	UTurnBasedMatchInterface* GetTurnBasedMatchInterfaceObject() { return TurnBasedMatchInterface; }

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;

	// TurnBasedMatchInterface object, used to set the match data after a match is found
	UTurnBasedMatchInterface* TurnBasedMatchInterface;

	// Minimum number of players needed for the match if a match is created
	uint32 MinPlayers;

	// Maximum number of players needed for the match if a match is created
	uint32 MaxPlayers;

	// Another matchmaking parameter that must be the same for players to matchmake together - for example, this could be the game mode (1 for deathmatch, 2 for capture the flag, etc.)
	uint32 PlayerGroup;

	// Show matches that the player is already a part of in the matchmaking interface
	bool ShowExistingMatches;

	TSharedPtr<FFindTurnBasedMatchCallbackProxyMatchmakerDelegate, ESPMode::ThreadSafe> Delegate;
};
