// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystem.h"
#include "InAppPurchaseDataTypes.h"
#include "InAppPurchaseCallbackProxy2.generated.h"

/**
 * Micro-transaction purchase information
 */
USTRUCT(BlueprintType)
struct FInAppPurchaseProductInfo2
{
	GENERATED_USTRUCT_BODY()

		// The unique product identifier
		UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString Identifier;

	// the unique transaction identifier
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString TransactionIdentifier;

	// The localized display name
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString DisplayName;

	// The localized display description name
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString DisplayDescription;

	// The localized display price name
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString DisplayPrice;

	// Raw price without currency code and symbol
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		float RawPrice = 0.0f;

	// The localized currency code of the price
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString CurrencyCode;

	// The localized currency symbol of the price
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString CurrencySymbol;

	// The localized decimal separator used in the price
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString DecimalSeparator;

	// The localized grouping separator of the price
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString GroupingSeparator;

	// Opaque receipt data for the transaction
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString ReceiptData;

	// Dynamic fields from raw Json data.
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		TMap<FString, FString> DynamicFields;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FInAppPurchaseResult2, EInAppPurchaseStatus, PurchaseStatus, const TArray<FInAppPurchaseReceiptInfo2>&, InAppPurchaseReceipts);

UCLASS(MinimalAPI)
class UInAppPurchaseCallbackProxy2 : public UObject
{
	GENERATED_UCLASS_BODY()

		// Called when there is a successful In-App Purchase transaction
		UPROPERTY(BlueprintAssignable)
		FInAppPurchaseResult2 OnSuccess;

		// Called when there is an unsuccessful In-App Purchase transaction
		UPROPERTY(BlueprintAssignable)
		FInAppPurchaseResult2 OnFailure;

	// Kicks off a transaction for the provided product identifier
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Make an In-App Purchase v2", DeprecatedFunction, DeprecationMessage = "Please use 'Start an In-App Purchase' and remember to pass the output receipts to 'Finalize In-App Purchase Transaction' after being validated and processed"), Category = "Online|InAppPurchase")
	static UInAppPurchaseCallbackProxy2* CreateProxyObjectForInAppPurchase(class APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest);
	
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Process any New Unprocessed Purchases v2", DeprecatedFunction, DeprecationMessage = "Please use 'Get known In-App Receipts' and remember to pass the output receipts to 'Finalize In-App Purchase Transaction' after being validated and processed"), Category = "Online|InAppPurchase")
	static UInAppPurchaseCallbackProxy2* CreateProxyObjectForInAppPurchaseUnprocessedPurchases(class APlayerController* PlayerController);
	
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Query for Owned Purchases", DeprecatedFunction, DeprecationMessage = "Please use 'Query for Owned In-App Products' and remember to pass the output receipts to 'Finalize In-App Purchase Transaction' after being validated and processed"), Category = "Online|InAppPurchase")
	static UInAppPurchaseCallbackProxy2* CreateProxyObjectForInAppPurchaseQueryOwned(class APlayerController* PlayerController);

public:

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface

private:

	/** Called by the InAppPurchase system when the transaction has finished */
	void OnInAppPurchaseComplete_Delayed();
	void OnCheckoutComplete(const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt);
	void OnPurchaseComplete();
	void OnQueryReceiptsComplete(const FOnlineError& Result);

	/** Triggers the In-App Purchase Transaction for the specifed user; the Purchase Request object must already be set up */
	void Trigger(class APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest);
	/** Triggers the In-App Purchase processing for unprocessed receipts */
	void TriggerGetUnprocessedPurchases(APlayerController* PlayerController);
	/** Triggers the In-App Purchase processing for unprocessed receipts */
	void TriggerGetOwnedPurchases(APlayerController* PlayerController);

private:

	/** Delegate called when a InAppPurchase has been successfully read */
	FOnPurchaseCheckoutComplete InAppPurchaseCompleteDelegate;

	/** Handle to the registered InAppPurchaseCompleteDelegate */
	FDelegateHandle InAppPurchaseCompleteDelegateHandle;

	/** Did we fail immediately? */
	bool bFailedToEvenSubmit;

	/** Pointer to the world, needed to delay the results slightly */
	TWeakObjectPtr<UWorld> WorldPtr;

	/** Did the purchase succeed? */
	bool bWasSuccessful = false;

	TArray<FInAppPurchaseReceiptInfo2> SavedReceipts;

	FUniqueNetIdPtr PurchasingPlayer;

	IOnlinePurchasePtr PurchaseInterface;

	EInAppPurchaseStatus SavedPurchaseStatus;
};
