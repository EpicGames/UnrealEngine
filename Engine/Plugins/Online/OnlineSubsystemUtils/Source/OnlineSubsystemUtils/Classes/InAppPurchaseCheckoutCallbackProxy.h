// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"
#include "InAppPurchaseDataTypes.h"
#include "Misc/Optional.h"
#include "OnlineSubsystem.h"
#include "InAppPurchaseCheckoutCallbackProxy.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnlineProxyInAppCheckoutResult, EInAppPurchaseStatus, PurchaseStatus, const FInAppPurchaseReceiptInfo2&, InAppPurchaseReceipt);

UCLASS(MinimalAPI)
class UInAppPurchaseCheckoutCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful In-App Purchase transaction
	UPROPERTY(BlueprintAssignable)
	FOnlineProxyInAppCheckoutResult OnSuccess;

	// Called when there is an unsuccessful In-App Purchase transaction
	UPROPERTY(BlueprintAssignable)
	FOnlineProxyInAppCheckoutResult OnFailure;

	// Kicks off a transaction for the provided product identifier
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Start an In-App Purchase"), Category = "Online|InAppPurchase")
	static UInAppPurchaseCheckoutCallbackProxy* CreateProxyObjectForInAppPurchaseCheckout(class APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest);

private:

	/** Called by the InAppPurchase system when the transaction has finished */
	void OnCheckoutComplete(const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt);
	void OnPurchaseComplete();

	/** Triggers the In-App Purchase Transaction for the specifed user; the Purchase Request object must already be set up */
	void TriggerCheckout(class APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest);

	void ClearPreviousResult();
private:
	/** Pointer to the world, needed to delay the results slightly */
	TWeakObjectPtr<UWorld> WorldPtr;

	/** Receipt in case of a succsessful purchase */
	TOptional<FInAppPurchaseReceiptInfo2> SavedReceipt;

	/** Received purchase status */
	EInAppPurchaseStatus SavedPurchaseStatus = EInAppPurchaseStatus::Failed;
};
