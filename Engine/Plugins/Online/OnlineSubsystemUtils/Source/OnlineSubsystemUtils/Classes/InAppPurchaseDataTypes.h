// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InAppPurchaseDataTypes.generated.h"

struct FOnlineError;

/**
 * Micro-transaction request information
 */
USTRUCT(BlueprintType)
struct FInAppPurchaseProductRequest2
{
	GENERATED_USTRUCT_BODY()

	// The unique product identifier that matches the one from your targeted store.
	UPROPERTY(BlueprintReadWrite, Category = ProductInfo)
	FString ProductIdentifier;

	// Flag to determine whether this is a consumable purchase, or not.
	UPROPERTY(BlueprintReadWrite, Category = ProductInfo)
	bool bIsConsumable = false;
};

/**
 * State of a purchase transaction
 */
UENUM(BlueprintType)
enum class EInAppPurchaseStatus : uint8
{
	Invalid = 0 UMETA(DisplayName = "Invalid"),
	/** purchase completed but failed */
	Failed UMETA(DisplayName = "Failed"),
	/** purchase has been deferred (neither failed nor completed) */
	Deferred UMETA(DisplayName = "Deferred"),
	/** purchase canceled by user */
	Canceled UMETA(DisplayName = "Canceled"),
	/** purchase succeeded */
	Purchased UMETA(DisplayName = "Purchased"),
	/** restore succeeded */
	Restored UMETA(DisplayName = "Restored"),
};

EInAppPurchaseStatus PurchaseStatusFromOnlineError(const FOnlineError& OnlineError);

/**
 * Micro-transaction purchase information
 */
USTRUCT(BlueprintType)
struct FInAppPurchaseReceiptInfo2
{
	GENERATED_USTRUCT_BODY()

	// The item name
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
	FString ItemName;

	// The unique product identifier
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
	FString ItemId;

	// the purchase validation information
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
	FString ValidationInfo;

	// the unique transaction identifier
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
	FString TransactionIdentifier;
};
