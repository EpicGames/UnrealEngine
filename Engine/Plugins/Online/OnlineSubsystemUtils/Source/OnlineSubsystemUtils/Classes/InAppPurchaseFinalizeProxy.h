// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OnlineSubsystem.h"
#include "InAppPurchaseFinalizeProxy.generated.h"

struct FInAppPurchaseReceiptInfo2;

UCLASS(MinimalAPI)
class UInAppPurchaseFinalizeProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Finalizes a transaction for the provided transaction identifier
	UFUNCTION(BlueprintCallable, meta = (DisplayName="Finalize In-App Purchase Transaction"), Category="Online|InAppPurchase")
	static UInAppPurchaseFinalizeProxy* CreateProxyObjectForInAppPurchaseFinalize(const FInAppPurchaseReceiptInfo2& InAppPurchaseReceipt, class APlayerController* PlayerController);

private:

	/** Triggers the In-App Purchase Finalize Transaction for the specifed user */
	void Trigger(const FInAppPurchaseReceiptInfo2& InAppPurchaseReceipt, class APlayerController* PlayerController);
};
