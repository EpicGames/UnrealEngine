// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "InAppPurchaseQueryCallbackProxy2.generated.h"

UENUM(BlueprintType)
enum class EOnlineProxyStoreOfferDiscountType : uint8
{
	/** Offer isn't on sale*/
	NotOnSale = 0 UMETA(DisplayName = "NotOnSale"),
	/** Offer price should be displayed as a percentage of regular price */
	Percentage UMETA(DisplayName = "Percentage"),
	/** Offer price should be displayed as an amount off regular price */
	DiscountAmount UMETA(DisplayName = "DiscountAmount"),
	/** Offer price should be displayed as a new price */
	PayAmount UMETA(DisplayName = "PayAmount"),
};

/**
 * Offer entry for display from online store
 */
USTRUCT(BlueprintType)
struct FOnlineProxyStoreOffer
{
	GENERATED_USTRUCT_BODY()

	/** Unique offer identifier */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString OfferId;

	/** Title for display */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FText Title;
	/** Short description for display */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FText Description;
	/** Full description for display */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FText LongDescription;

	/** Regular non-sale price as text for display */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FText RegularPriceText;
	/** Regular non-sale price in numeric form for comparison/sorting */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		int32 RegularPrice = 0;

	/** Final-Pricing (Post-Sales/Discounts) as text for display */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FText PriceText;
	/** Final-Price (Post-Sales/Discounts) in numeric form for comparison/sorting */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		int32 NumericPrice = 0;

	/** Price currency code */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString CurrencyCode;

	/** Date the offer was released */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FDateTime ReleaseDate = FDateTime::MinValue();
	/** Date this information is no longer valid (maybe due to sale ending, etc) */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FDateTime ExpirationDate = FDateTime::MinValue();
	/** Type of discount currently running on this offer (if any) */
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		EOnlineProxyStoreOfferDiscountType DiscountType = EOnlineProxyStoreOfferDiscountType::NotOnSale;
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		TMap<FString, FString> DynamicFields;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInAppPurchaseQuery2Result, const TArray<FOnlineProxyStoreOffer>&, InAppOfferInformation);

UCLASS(MinimalAPI)
class UInAppPurchaseQueryCallbackProxy2 : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful InAppPurchase query
	UPROPERTY(BlueprintAssignable)
	FInAppPurchaseQuery2Result OnSuccess;

	// Called when there is an unsuccessful InAppPurchase query
	UPROPERTY(BlueprintAssignable)
	FInAppPurchaseQuery2Result OnFailure;

	// Queries a InAppPurchase for an integer value
	UFUNCTION(BlueprintCallable, meta = (DisplayName="Read In App Purchase Information2"), Category="Online|InAppPurchase")
	static UInAppPurchaseQueryCallbackProxy2* CreateProxyObjectForInAppPurchaseQuery(class APlayerController* PlayerController, const TArray<FString>& ProductIdentifiers);

public:
	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface

private:
	/** Called by the InAppPurchase system when the read is finished */
	void OnInAppPurchaseRead_Delayed();
	void OnInAppPurchaseRead(bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& Error);

	void CreateProxyProductInformation(TArray<FOnlineStoreOfferRef>& SourceArray, TArray<FOnlineProxyStoreOffer>& TargetArray);

	/** Triggers the query for a specifed user*/
	void TriggerQuery(class APlayerController* PlayerController, const TArray<FString>& ProductIdentifiers);

private:
	// Pointer to the world, needed to delay the results slightly
	TWeakObjectPtr<UWorld> WorldPtr;

	// Did the read succeed?
	bool bSavedWasSuccessful;
	TArray<FOnlineProxyStoreOffer> SavedProductInformation;
};
