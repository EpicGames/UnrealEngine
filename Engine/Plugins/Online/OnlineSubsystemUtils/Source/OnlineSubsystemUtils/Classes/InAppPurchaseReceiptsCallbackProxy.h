// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystem.h"
#include "InAppPurchaseDataTypes.h"
#include "InAppPurchaseReceiptsCallbackProxy.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnlineProxyInAppReceiptsResult, EInAppPurchaseStatus, PurchaseStatus, const TArray<FInAppPurchaseReceiptInfo2>&, InAppPurchaseReceipts);

UCLASS(MinimalAPI)
class UInAppPurchaseReceiptsCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful In-App Purchase transaction
	UPROPERTY(BlueprintAssignable)
	FOnlineProxyInAppReceiptsResult OnSuccess;

	// Called when there is an unsuccessful In-App Purchase transaction
	UPROPERTY(BlueprintAssignable)
	FOnlineProxyInAppReceiptsResult OnFailure;

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get known In-App Receipts"), Category = "Online|InAppPurchase")
	static UInAppPurchaseReceiptsCallbackProxy* CreateProxyObjectForInAppPurchaseGetKnownReceipts(class APlayerController* PlayerController);
	
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Query for Owned In-App Products"), Category = "Online|InAppPurchase")
	static UInAppPurchaseReceiptsCallbackProxy* CreateProxyObjectForInAppPurchaseQueryOwnedProducts(class APlayerController* PlayerController);

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Restore Owned In-App Products"), Category = "Online|InAppPurchase")
	static UInAppPurchaseReceiptsCallbackProxy* CreateProxyObjectForInAppPurchaseRestoreOwnedProducts(class APlayerController* PlayerController);
private:
	void ClearPreviousResult();

	/** Called by the InAppPurchase system when the transaction has finished */
	void OnQueryComplete();
	void OnQueryReceiptsComplete(const FOnlineError& Resul, FUniqueNetIdWeakPtr PurchasingPlayerWeak);

	/** Triggers the In-App Purchase processing for unprocessed receipts */
	void TriggerGetKnownReceipts(APlayerController* PlayerController);
	/** Triggers the In-App Purchase processing for unprocessed receipts */
	void TriggerGetOwnedProducts(APlayerController* PlayerController, bool bRestorePurchases);

private:

	/** Delegate called when a InAppPurchase has been successfully read */
	FOnlineProxyInAppReceiptsResult InAppPurchaseCompleteDelegate;

	/** Handle to the registered InAppPurchaseCompleteDelegate */
	FDelegateHandle InAppPurchaseCompleteDelegateHandle;

	/** Pointer to the world, needed to delay the results slightly */
	TWeakObjectPtr<UWorld> WorldPtr;

	/** Did the purchase succeed? */
	bool bWasSuccessful = false;

	TArray<FInAppPurchaseReceiptInfo2> SavedReceipts;

	EInAppPurchaseStatus SavedPurchaseStatus = EInAppPurchaseStatus::Failed;
};
