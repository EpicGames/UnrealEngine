// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OnlineSubsystem.h"
#include "GameFramework/OnlineReplStructs.h"
#include "InAppPurchaseRestoreCallbackProxy2.generated.h"

enum class EInAppPurchaseStatus : uint8;
struct FInAppPurchaseProductRequest2;
struct FOnlineError;

/**
 * Micro-transaction purchase information
 */
USTRUCT(BlueprintType)
struct FInAppPurchaseRestoreInfo2
{
	GENERATED_USTRUCT_BODY()

		// The item name
		UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString ItemName;

	// The unique product identifier
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString ItemId;

	// the unique transaction identifier
	UPROPERTY(BlueprintReadOnly, Category = ProductInfo)
		FString ValidationInfo;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FInAppPurchaseRestoreResult2, EInAppPurchaseStatus, PurchaseStatus, const TArray<FInAppPurchaseRestoreInfo2>&, InAppPurchaseRestoreInfo);

UCLASS(MinimalAPI)
class UInAppPurchaseRestoreCallbackProxy2 : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful In-App Purchase transaction
	UPROPERTY(BlueprintAssignable)
	FInAppPurchaseRestoreResult2 OnSuccess;

	// Called when there is an unsuccessful In-App Purchase transaction
	UPROPERTY(BlueprintAssignable)
	FInAppPurchaseRestoreResult2 OnFailure;

	// Kicks off a transaction for the provided product identifier
	UFUNCTION(BlueprintCallable, meta = (DisplayName="Restore In-App Purchases2", DeprecatedFunction, DeprecationMessage = "Please use 'Restore Owned In-App Products' and remember to pass the output receipts to 'Finalize In-App Purchase Transaction' after being validated and processed"), Category="Online|InAppPurchase")
	static UInAppPurchaseRestoreCallbackProxy2* CreateProxyObjectForInAppPurchaseRestore(const TArray<FInAppPurchaseProductRequest2>& ConsumableProductFlags, class APlayerController* PlayerController);

public:

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface

private:

	/** Called by the InAppPurchase system when the transaction has finished */
	void OnInAppPurchaseRestoreComplete_Delayed();
	void OnInAppPurchaseRestoreComplete();

	void OnQueryReceiptsComplete(const FOnlineError& Result);

	/** Triggers the In-App Purchase Restore Transaction for the specifed user */
	void Trigger(const TArray<FInAppPurchaseProductRequest2>& ConsumableProductFlags, class APlayerController* PlayerController);

private:

	bool bWasSuccessful;
	/** Pointer to the world, needed to delay the results slightly */
	TWeakObjectPtr<UWorld> WorldPtr;

	TArray<FInAppPurchaseRestoreInfo2> SavedReceipts;

	IOnlinePurchasePtr PurchaseInterface;

	FUniqueNetIdRepl PurchasingPlayer;
	
	EInAppPurchaseStatus SavedPurchaseStatus;
};
