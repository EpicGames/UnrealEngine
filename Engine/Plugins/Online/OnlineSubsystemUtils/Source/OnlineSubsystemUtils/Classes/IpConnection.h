// Copyright Epic Games, Inc. All Rights Reserved.

//
// Ip based implementation of a network connection used by the net driver class
//

#pragma once

#include "Engine/NetConnection.h"
#include "IpConnection.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class FInternetAddr;
class ISocketSubsystem;
class FSocket;
class FIpConnectionHelper;

namespace UE::Net::Private
{
	class FNetDriverAddressResolution;
	class FNetConnectionAddressResolution;
}

UCLASS(MinimalAPI, transient, config=Engine)
class UIpConnection : public UNetConnection
{
    GENERATED_UCLASS_BODY()

	friend FIpConnectionHelper;
	friend UE::Net::Private::FNetDriverAddressResolution;

public:
	//~ Begin NetConnection Interface
	UE_API virtual void InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	UE_API FString LowLevelGetRemoteAddress(bool bAppendPort=false) override;
	UE_API FString LowLevelDescribe() override;
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual void CleanUp() override;
	UE_API virtual void ReceivedRawPacket(void* Data, int32 Count) override;
	UE_API virtual float GetTimeoutValue() override;
	//~ End NetConnection Interface

	/**
	 * If CVarNetIpConnectionUseSendTasks is true, blocks until there are no outstanding send tasks.
	 * Since these tasks need to access the socket, this is called before the net driver closes the socket.
	 */
	UE_API void WaitForSendTasks();

	/**
	 * Gets the cached socket for this connection.
	 *
	 * @return	The cached socket for this connection
	 */
	FSocket* GetSocket() const
	{
		return SocketPrivate.Get();
	}

private:
	/**
	 * Struct to hold the result of a socket SendTo call. If net.IpConnectionUseSendTasks is true,
	 * these are communicated back to the game thread via SocketSendResults.
	 */
	struct FSocketSendResult
	{
		int32 BytesSent = 0;
		ESocketErrors Error = SE_NO_ERROR;
	};

	/** Critical section to protect SocketSendResults */
	FCriticalSection SocketSendResultsCriticalSection;

	/** Socket SendTo results from send tasks if net.IpConnectionUseSendTasks is true */
	TArray<FSocketSendResult> SocketSendResults;

	/**
	 * If net.IpConnectionUseSendTasks is true, reference to the last send task used as a prerequisite
	 * for the next send task. Also, CleanUp() blocks until this task is complete.
	 */
	FGraphEventRef LastSendTask;

	/** The socket used for communication (typically shared between NetConnection's, Net Address Resolution, and the NetDriver) */
	TSharedPtr<FSocket> SocketPrivate;

	/** List of previously active sockets for this connection, whose cleanup is deferred for multithreaded safety and to prevent remote ICMP errors */
	TArray<TSharedPtr<FSocket>> DeferredCleanupSockets;

	/** The time at which a socket was last queued for deferred cleanup */
	double DeferredCleanupTimeCheck = 0.0;

	/** The number of 'DeferredCleanupSockets' entries ready for cleanup (may be set by async send tasks) */
	std::atomic<int32> DeferredCleanupReadyCount = 0;

	/** Instead of disconnecting immediately on a socket error, wait for some time to see if we can recover. Specified in Seconds. */
	UPROPERTY(Config)
	float SocketErrorDisconnectDelay;

	/** Cached time of the first send socket error that will be used to compute disconnect delay. */
	double SocketError_SendDelayStartTime;

	/** Cached time of the first recv socket error that will be used to compute disconnect delay. */
	double SocketError_RecvDelayStartTime;

#if !UE_BUILD_SHIPPING
	/** The number of socket-level sends that have occurred during initial connect */
	int32 InitialConnectSocketSendCount = 0;

	/** The last time initial connect diagnostics put out a log */
	double InitialConnectLastLogTime = 0.0;

	/** The value of 'InitialConnectSocketSendCount' the last time there was an initial connect diagnostic log */
	int32 InitialConnectLastLogSocketSendCount = 0;
#endif

private:
	/** Performs the work of LowLevelSend once we have confirmed that we want to send data to remote. */
	UE_API void SendToRemote(uint8* Data, int32 CountBits, FOutPacketTraits& Traits);

	/** Sets the local socket pointer, and safely cleans up any references to old sockets */
	UE_API void SetSocket_Local(const TSharedPtr<FSocket>& InSocket);

	/** Safe/non-blocking cleanup of a shared socket, which may be in use by async sends, or may be at risk of triggering ICMP unreachable errors */
	UE_API void SafeDeferredSocketCleanup(const TSharedPtr<FSocket>& InSocket);

	/** Handles any SendTo errors on the game thread. */
	UE_API void HandleSocketSendResult(const FSocketSendResult& Result, ISocketSubsystem* SocketSubsystem);

	/** Notifies us that we've encountered an error while receiving a packet. */
	UE_API void HandleSocketRecvError(class UNetDriver* NetDriver, const FString& ErrorString);

	/** NetDriver level early address resolution (may pass work on to NetConnection level address resolution) */
	TPimplPtr<UE::Net::Private::FNetConnectionAddressResolution> Resolver;

	/** If address resolution has results ready, assigns the resolved socket and RemoteAddr. Returns true if socket was assigned. */
	UE_API bool TryConsumeAddressResolution(const TCHAR* LogSuffix = TEXT(""));

protected:
	/**
	 * Disables address resolution by pushing the disabled flag into the status field.
	 */
	UE_API void DisableAddressResolution();

	/**
	 * Handles a NetConnection timeout. Overridden in order to handle parsing multiple GAI results during resolution.
	 * 
	 * @param ErrorStr A string containing the current error message for either usage or writing into.
	 */
	UE_API virtual void HandleConnectionTimeout(const FString& ErrorStr) override;
};

#undef UE_API
