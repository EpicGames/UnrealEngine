// Copyright Epic Games, Inc. All Rights Reserved.

//
// Ip endpoint based implementation of the net driver
//

#pragma once

#include "Engine/NetDriver.h"
#include "HAL/Runnable.h"
#include "Containers/CircularQueue.h"
#include "SocketSubsystem.h"
#include "Containers/SpscQueue.h"
#include "IpNetDriver.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FPacketBufferView;
struct FReceivedPacketView;
struct FURL;


// Forward declarations
class Error;
class FInternetAddr;
class FNetworkNotify;
class FSocket;
struct FRecvMulti;

namespace UE::Net::Private
{
	class FNetDriverAddressResolution;
}


// CVars
#if !UE_BUILD_SHIPPING
extern TSharedPtr<FInternetAddr> GCurrentDuplicateIP;
#endif


namespace UE::Net
{
	/** Results for UIpNetDriver::RecreateSocket */
	enum class ERecreateSocketResult : uint8
	{
		NoAction,			// No action taken - this is either a replay, called on the server, or the NetDriver doesn't support socket recreation
		NotReady,			// Net Address Resolution is in the middle of resolving, so no socket recreation can take place
		AlreadyInProgress,	// Socket recreation is already in progress
		BeganRecreate,		// Socket recreation was successfully kicked off
		Error				// There was an error
	};

	const TCHAR* LexToString(ERecreateSocketResult Value);
}

namespace UE::Net::Private
{
	/** Callback triggered when setting a new socket has completed (which may not return immediately, due to multithreading) */
	using FSetSocketComplete = TUniqueFunction<void()>;
}


UCLASS(MinimalAPI, transient, config=Engine)
class UIpNetDriver : public UNetDriver
{
	friend class FPacketIterator;

    GENERATED_BODY()

public:
	/** Should port unreachable messages be logged */
    UPROPERTY(Config)
    uint32 LogPortUnreach:1;

	/** Does the game allow clients to remain after receiving ICMP port unreachable errors (handles flakey connections) */
    UPROPERTY(Config)
    uint32 AllowPlayerPortUnreach:1;

	/** Whether to exit if binding a port failed. */
	UPROPERTY(Config)
	uint32 bExitOnBindFailure:1;

	/** Number of ports which will be tried if current one is not available for binding (i.e. if told to bind to port N, will try from N to N+MaxPortCountToTry inclusive) */
	UPROPERTY(Config)
	uint32 MaxPortCountToTry;

	/** If pausing socket receives, the time at which this should end */
	float PauseReceiveEnd;

	/** Base constructor */
	UE_API UIpNetDriver(const FObjectInitializer& ObjectInitializer);

	//~ Begin UNetDriver Interface.
	UE_API virtual bool IsAvailable() const override;
	UE_API virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual bool InitConnect( FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error ) override;
	UE_API virtual bool InitListen( FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error ) override;
	UE_API virtual void TickDispatch( float DeltaTime ) override;
	UE_API virtual void LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	UE_API virtual FString LowLevelGetNetworkNumber() override;
	UE_API virtual void LowLevelDestroy() override;
	UE_API virtual class ISocketSubsystem* GetSocketSubsystem() override;
	virtual bool IsNetResourceValid(void) override
	{
		return GetSocket() != nullptr;
	}
	//~ End UNetDriver Interface

private:
	/**
	 * Process packets not associated with a NetConnection, performing handshaking and NetConnection creation or remapping as necessary.
	 *
	 * @param PacketRef			A view of the received packet (may output a new packet view, possibly using WorkingBuffer)
	 * @param WorkingBuffer		A buffer for storing processed packet data (may be the buffer the input ReceivedPacketRef points to)
	 * @return					If a new NetConnection is created, returns the net connection
	 */
	UE_API UNetConnection* ProcessConnectionlessPacket(FReceivedPacketView& PacketRef, const FPacketBufferView& WorkingBuffer);

	/** Clears all references to sockets held by this net driver. */
	UE_API void ClearSockets();

public:
	//~ Begin UIpNetDriver Interface.

	/**
	 * Returns the current FSocket to be used with this NetDriver. This is useful in the cases of resolution as it will always point to the Socket that's currently being
	 * used with the current resolution attempt.
	 *
	 * @return a pointer to the socket to use.
	 */
	UE_API virtual FSocket* GetSocket();

	/**
	* Set the current NetDriver's Socket/LocalAddr to the given socket (typically after Net Address Resolution).
	* This will automatically pass the socket back through all NetConnection's, and trigger safe/deferred cleanup of the old socket.
	*
	* @param NewSocket	The socket pointer to set this NetDriver's socket to
	*/
	UE_API void SetSocketAndLocalAddress(const TSharedPtr<FSocket>& SharedSocket);

	/**
	 * Returns the port number to use when a client is creating a socket.
	 * Platforms that can't use the default of 0 (system-selected port) may override
	 * this function.
	 *
	 * @return The port number to use for client sockets. Base implementation returns 0.
	 */
	UE_API virtual int GetClientPort();

	/**
	 * Creates a new Socket for the NetDriver/NetConnection's, based on the address/binding of the existing socket, with a new ephemeral port.
	 * Used to attempt recovery for 'half-broken' connections, where (for whatever reason) only one side of the connection is receiving packets.
	 *
	 * Also usable to test the 'restart handshake' feature, using the 'net RecreateSocket' command.
	 *
	 * @param OverridePort		optional port number to use with the new socket, otherwise falls back to GetClientPort()
	 * 
	 * @return	Returns the action taken (e.g. successfully kicked off recreation, or whether this failed or is already in progress etc.)
	 */
	UE_API UE::Net::ERecreateSocketResult RecreateSocket(int32 OverridePort = INDEX_NONE);

	/** Returns the last time socket recreation was kicked off or completed */
	double GetLastRecreateSocketTime() const
	{
		return LastRecreateSocketTime;
	}

protected:
	/**
	 * Creates a socket set up for communication using the given protocol. This allows for explicit creation instead of inferring type for you.
	 *
	 * @param ProtocolType	an FName that represents the protocol to allocate the new socket under. Typically set to None or a value in FNetworkProtocolTypes
	 * @return				an FSocket if creation succeeded, nullptr if creation failed.
	 */
	UE_API virtual FUniqueSocket CreateSocketForProtocol(const FName& ProtocolType);

	/**
	 * Creates, initializes and binds a socket using the given bind address information.
	 *
	 * @param BindAddr				the address to bind the new socket to, will also create the socket using the address protocol using CreateSocketForProtocol
	 * @param Port					the port number to use with the given bind address.
	 * @param bReuseAddressAndPort	if true, will set the socket to be bound even if the address is in use
	 * @param DesiredRecvSize		the max size of the recv buffer for the socket
	 * @param DesiredSendSize		the max size of the sending buffer for the socket
	 * @param Error					a string reference that will be populated with any error messages should an error occur
	 *
	 * @return if the socket could be created and bound with all the appropriate options, a pointer to the new socket is given, otherwise null
	 */
	UE_API virtual FUniqueSocket CreateAndBindSocket(TSharedRef<FInternetAddr> BindAddr, int32 Port, bool bReuseAddressAndPort, int32 DesiredRecvSize, int32 DesiredSendSize, FString& Error);
	//~ End UIpNetDriver Interface.

	/** Called by NetDriver subclasses, to specify whether or not they support recreating the active socket */
	void SetSupportsRecreateSocket(bool bInSupportsRecreateSocket)
	{
		bSupportsRecreateSocket = bInSupportsRecreateSocket;
	}

public:
	//~ Begin FExec Interface
#if UE_ALLOW_EXEC_COMMANDS
	UE_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog ) override;
#endif
	//~ End FExec Interface


	/** Exec command handlers */

	UE_API bool HandleSocketsCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);
	UE_API bool HandlePauseReceiveCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);
	UE_API bool HandleRecreateSocketCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

#if !UE_BUILD_SHIPPING
	/**
	* Tests a scenario where a connected client suddenly starts sending traffic on a different port (can happen
	* due to a rare router bug.
	*
	* @param NumConnections - number of connections to test sending traffic on a different port.
	*/
	UE_API void TestSuddenPortChange(uint32 NumConnections);
#endif


	/** @return IP connection to server */
	UE_API class UIpConnection* GetServerConnection();

	/** @return The address resolution timeout value */
	float GetResolutionTimeoutValue() const { return ResolutionConnectionTimeout; }

private:
	/**
	 * Whether or not a socket receive failure Error indicates a blocking error, which should 'break;' the receive loop
	 */
	static inline bool IsRecvFailBlocking(ESocketErrors Error)
	{
		// SE_ECONNABORTED is for LAN cable pulls on some platforms, SE_ENETDOWN is to prevent a hang on another platform
		return Error == SE_NO_ERROR || Error == SE_EWOULDBLOCK || Error == SE_ECONNABORTED || Error == SE_ENETDOWN;
	};

	/**
	 * When a new pre-connection IP is encountered, do rate-limited logging/tracking for it.
	 *
	 * @param InAddr	The IP address to log/track
	 */
	UE_API void TrackAndLogNewIP(const FInternetAddr& InAddr);

	/**
	 * Removes an IP from new IP tracking, after successful connection.
	 *
	 * @param InAddr	The IP address to remove from tracking
	 */
	UE_API void RemoveFromNewIPTracking(const FInternetAddr& InAddr);

	/**
	 * Resets new pre-connection IP tracking and timers.
	 */
	UE_API void ResetNewIPTracking();

	/**
	 * When new pre-connection IP tracking is enabled, ticks the tracking
	 *
	 * @param DeltaTime		The time since the last global tick
	 */
	UE_API void TickNewIPTracking(float DeltaTime);


	/**
	 * The current state of the NetDriver socket
	 */
	enum class ESocketState : uint8
	{
		None,
		Resolving,		// Net Address Resolution is in the middle of resolving a working socket
		Recreating,		// Socket Recreation is in the middle of creating a new socket
		Ready,			// The current socket is fully ready to use
		Error			// The current socket is in an error state
	};


	/** Returns the current state of the NetDriver socket */
	ESocketState GetSocketState() const
	{
		return SocketState;
	}

	/**
	 * Triggered when socket recreation completes (which may occur after a delay, due to e.g. updating the Receive Thread socket)
	 */
	UE_API void OnRecreateSocketComplete();

	/**
	 * Internal function for setting a socket and propagating it to all NetConnection's and the Receive Thread etc..
	 *
	 * @param InSocket			The new socket
	 * @param InSetCallback		Callback triggered when the new socket is fully set
	 */
	UE_API void SetSocket_Internal(const TSharedPtr<FSocket>& InSocket, UE::Net::Private::FSetSocketComplete InSetCallback=nullptr);


public:
	// Callback for platform handling when networking is taking a long time in a single frame (by default over 1 second).
	// It may get called multiple times in a single frame if additional processing after a previous alert exceeds the threshold again
	DECLARE_MULTICAST_DELEGATE(FOnNetworkProcessingCausingSlowFrame);
	static UE_API FOnNetworkProcessingCausingSlowFrame OnNetworkProcessingCausingSlowFrame;

private:
	/** Number of bytes that will be passed to FSocket::SetReceiveBufferSize when initializing a server. */
	UPROPERTY(Config)
	uint32 ServerDesiredSocketReceiveBufferBytes;

	/** Number of bytes that will be passed to FSocket::SetSendBufferSize when initializing a server. */
	UPROPERTY(Config)
	uint32 ServerDesiredSocketSendBufferBytes;

	/** Number of bytes that will be passed to FSocket::SetReceiveBufferSize when initializing a client. */
	UPROPERTY(Config)
	uint32 ClientDesiredSocketReceiveBufferBytes;

	/** Number of bytes that will be passed to FSocket::SetSendBufferSize when initializing a client. */
	UPROPERTY(Config)
	uint32 ClientDesiredSocketSendBufferBytes;

	/** Maximum time in seconds the TickDispatch can loop on received socket data*/
	UPROPERTY(Config)
	double MaxSecondsInReceive = 0.0;

	/** Nb of packets to wait before testing if the receive time went over MaxSecondsInReceive */
	UPROPERTY(Config)
	int32 NbPacketsBetweenReceiveTimeTest = 0;

	/** The amount of time to wait in seconds until we consider a connection to a resolution result as timed out */
	UPROPERTY(Config)
	float ResolutionConnectionTimeout;

	/** Represents a packet received and/or error encountered by the receive thread, if enabled, queued for the game thread to process. */
	struct FReceivedPacket
	{
		/** The content of the packet as received from the socket. */
		TArray<uint8> PacketBytes;

		/** Address from which the packet was received. */
		TSharedPtr<FInternetAddr> FromAddress;

		/** The error triggered by the socket RecvFrom call. */
		ESocketErrors Error;

		/** FPlatformTime::Seconds() at which this packet and/or error was received. Can be used for more accurate ping calculations. */
		double PlatformTimeSeconds;

		FReceivedPacket()
			: Error(SE_NO_ERROR)
			, PlatformTimeSeconds(0.0)
		{
		}
	};

	/** Runnable object representing the receive thread, if enabled. */
	class FReceiveThreadRunnable : public FRunnable
	{
	public:
		FReceiveThreadRunnable(UIpNetDriver* InOwningNetDriver);

		virtual uint32 Run() override;

		/**
		 * Execute commands in OwnerEventQueue on the Game Thread
		 */
		void PumpOwnerEventQueue();

		/**
		 * Sets the socket for the Receive Thread, and triggers a callback (if specified) on the Game Thread when this is complete.
		 *
		 * @param InGameThreadSocket		The socket specified by the Game Thread
		 * @param InGameThreadSetCallback	The callback to trigger on the Game Thread when the socket is set
		 */
		void SetSocket(const TSharedPtr<FSocket>& InGameThreadSocket, UE::Net::Private::FSetSocketComplete InGameThreadSetCallback=nullptr);

	public:
		/** Thread-safe queue of received packets. The Run() function is the producer, UIpNetDriver::TickDispatch on the game thread is the consumer. */
		TCircularQueue<FReceivedPacket> ReceiveQueue;

		/** Running flag. The Run() function will return shortly after setting this to false. */
		TAtomic<bool> bIsRunning;

	private:
		bool DispatchPacket(FReceivedPacket&& IncomingPacket, int32 NbBytesRead);

		/**
		 * Execute commands in CommandQueue on the Receive Thread
		 */
		void PumpCommandQueue();

		/** Receive Thread implementation for 'SetSocket' */
		void SetSocketImpl(const TSharedPtr<FSocket>& InGameThreadSocket, UE::Net::Private::FSetSocketComplete InGameThreadSetCallback=nullptr);

		/**
		 * Clears the Receive Thread socket, and resets the shared socket pointer on the Game Thread.
		 */
		void ClearSocket();

	private:
		/** Command queue from Game Thread to Receive Thread */
		TSpscQueue<TUniqueFunction<void()>> CommandQueue;

		/** Event queue from the Receive Thread to the Game Thread (may be generalized to 'Owning' thread in future) */
		TSpscQueue<TUniqueFunction<void()>> OwnerEventQueue;

		/** The NetDriver which owns the receive thread */
		UIpNetDriver* OwningNetDriver;

		/** The socket subsystem used by the receive thread */
		ISocketSubsystem* SocketSubsystem;

		/** Shared pointer for the socket, owned by the Game Thread */
		TSharedPtr<FSocket> Socket;
	};

	/** Receive thread runnable object. */
	TUniquePtr<FReceiveThreadRunnable> SocketReceiveThreadRunnable;

	/** Receive thread object. */
	TUniquePtr<FRunnableThread> SocketReceiveThread;

	/** The preallocated state/buffers, for efficiently executing RecvMulti */
	TUniquePtr<FRecvMulti> RecvMultiState;

	/** Underlying socket communication */
	TSharedPtr<FSocket> SocketPrivate;

	/** The state of the NetDriver socket (e.g. ready, resolving, recreating etc.) */
	ESocketState SocketState = ESocketState::None;

	/** Whether or not this NetDriver supports recreating the socket */
	bool bSupportsRecreateSocket = true;

	/** The last time socket recreation was kicked off or completed. */
	double LastRecreateSocketTime = 0.0;


	/** New IP aggregated logging entry */
	struct FAggregatedIP
	{
		/** Hash of the IP */
		uint32 IPHash;

		/** IP:Port converted to string */
		FString IPStr;
	};

	/** Hashes for new pre-connection IP's that are being tracked */
	TArray<uint32> NewIPHashes;

	/** Number of times each 'NewIPHashes' IP was encountered, in the current tracking period */
	TArray<uint32> NewIPHitCount;

	/** List of IP's queued for aggregated logging */
	TArray<FAggregatedIP> AggregatedIPsToLog;

	/** Whether or not IP hash tracking or aggregated IP limits have been reached. Disables all further IP logging, for the current tracking period */
	bool bExceededIPAggregationLimit = false;

	/** Countdown timer for the current IP aggregation tracking period */
	double NextAggregateIPLogCountdown = 0.0;

	/** NetConnection specific address resolution */
	TPimplPtr<UE::Net::Private::FNetDriverAddressResolution> Resolver;
};

#undef UE_API
