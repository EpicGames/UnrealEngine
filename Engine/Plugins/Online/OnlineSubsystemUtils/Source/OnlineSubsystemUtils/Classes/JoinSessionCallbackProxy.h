// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "Net/OnlineBlueprintCallProxyBase.h"
#include "JoinSessionCallbackProxy.generated.h"

struct FBlueprintSessionResult;

class APlayerController;

UCLASS(MinimalAPI)
class UJoinSessionCallbackProxy : public UOnlineBlueprintCallProxyBase
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful join
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when there is an unsuccessful join
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// Joins a remote session with the default online subsystem
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online|Session")
	static UJoinSessionCallbackProxy* JoinSession(UObject* WorldContextObject, class APlayerController* PlayerController, const FBlueprintSessionResult& SearchResult);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

private:
	// Internal callback when the join completes, calls out to the public success/failure callbacks
	void OnCompleted(FName SessionName, EOnJoinSessionCompleteResult::Type Result);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The search result we are sttempting to join
	FOnlineSessionSearchResult OnlineSearchResult;

	// The delegate executed by the online subsystem
	FOnJoinSessionCompleteDelegate Delegate;

	// Handle to the registered FOnJoinSessionComplete delegate
	FDelegateHandle DelegateHandle;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
