// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "LeaderboardBlueprintLibrary.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class APlayerController;

/**
 * A beacon host used for taking reservations for an existing game session
 */
UCLASS(MinimalAPI, meta=(ScriptName="LeaderboardLibrary"))
class ULeaderboardBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Writes an integer value to the specified leaderboard */
	UFUNCTION(BlueprintCallable, Category = "Online|Leaderboard")
	static UE_API bool WriteLeaderboardInteger(APlayerController* PlayerController, FName StatName, int32 StatValue);

private:
	static bool WriteLeaderboardObject(APlayerController* PlayerController, class FOnlineLeaderboardWrite& WriteObject);
};

#undef UE_API
