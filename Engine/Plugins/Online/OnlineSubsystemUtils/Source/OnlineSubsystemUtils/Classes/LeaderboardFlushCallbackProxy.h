// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "LeaderboardFlushCallbackProxy.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLeaderboardFlushed, FName, SessionName);

UCLASS(MinimalAPI)
class ULeaderboardFlushCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful leaderboard flush
	UPROPERTY(BlueprintAssignable)
	FOnLeaderboardFlushed OnSuccess;

	// Called when there is an unsuccessful leaderboard flush
	UPROPERTY(BlueprintAssignable)
	FOnLeaderboardFlushed OnFailure;

	// Called to perform the query internally
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static ULeaderboardFlushCallbackProxy* CreateProxyObjectForFlush(class APlayerController* PlayerController, FName SessionName);

public:
	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface

private:
	/** Called by the leaderboard system when the flush is finished */
	void OnFlushCompleted(FName SessionName, bool bWasSuccessful);

	/** Unregisters our delegate from the leaderboard system */
	void RemoveDelegate();

	/** Triggers the flush for the specified session */
	void TriggerFlush(class APlayerController* PlayerController, FName InSessionName);

private:
	/** Delegate called when a leaderboard has been successfully flushed */
	FOnLeaderboardFlushCompleteDelegate LeaderboardFlushCompleteDelegate;

	/** OnLeaderboardFlushComplete delegate handle */
	FDelegateHandle LeaderboardFlushCompleteDelegateHandle;

	/** Did we fail immediately? */
	bool bFailedToEvenSubmit;
};
