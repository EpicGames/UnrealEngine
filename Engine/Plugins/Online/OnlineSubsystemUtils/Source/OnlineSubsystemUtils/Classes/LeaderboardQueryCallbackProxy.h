// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/TimerHandle.h"
#include "Interfaces/OnlineLeaderboardInterface.h"
#include "LeaderboardQueryCallbackProxy.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FLeaderboardQueryResult, int32, LeaderboardValue);

UCLASS(MinimalAPI)
class ULeaderboardQueryCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when there is a successful leaderboard query
	UPROPERTY(BlueprintAssignable)
	FLeaderboardQueryResult OnSuccess;

	// Called when there is an unsuccessful leaderboard query
	UPROPERTY(BlueprintAssignable)
	FLeaderboardQueryResult OnFailure;

	// Queries a leaderboard for an integer value
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", DisplayName="Read Leaderboard Integer"), Category="Online|Leaderboard")
	static ULeaderboardQueryCallbackProxy* CreateProxyObjectForIntQuery(class APlayerController* PlayerController, const FString& StatName);

public:
	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface

private:
	/** Called by the leaderboard system when the read is finished */
	void OnStatsRead_Delayed();
	void OnStatsRead(bool bWasSuccessful);

	/** Unregisters our delegate from the leaderboard system */
	void RemoveDelegate();

	/** Triggers the query for a specifed user; the ReadObject must already be set up */
	void TriggerQuery(class APlayerController* PlayerController, const FString& InStatName, EOnlineKeyValuePairDataType::Type StatType);

private:
	/** Delegate called when a leaderboard has been successfully read */
	FOnLeaderboardReadCompleteDelegate LeaderboardReadCompleteDelegate;

	/** LeaderboardReadComplete delegate handle */
	FDelegateHandle LeaderboardReadCompleteDelegateHandle;

	/** The leaderboard read request */
	TSharedPtr<class FOnlineLeaderboardRead, ESPMode::ThreadSafe> ReadObject;

	/** Did we fail immediately? */
	bool bFailedToEvenSubmit;

	/** Name of the stat being queried */
	FString StatName;

	// Pointer to the world, needed to delay the results slightly
	TWeakObjectPtr<UWorld> WorldPtr;

	// Did the read succeed?
	bool bSavedWasSuccessful;
	int32 SavedValue;

	FTimerHandle OnStatsRead_DelayedTimerHandle;
};
