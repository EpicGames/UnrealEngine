// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintAsyncActionBase.h"
#include "LogoutCallbackProxy.generated.h"

class APlayerController;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnlineLogoutResult, APlayerController*, PlayerController);

UCLASS(MinimalAPI)
class ULogoutCallbackProxy : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	ULogoutCallbackProxy(const FObjectInitializer& ObjectInitializer);

	// Called when the logout completed successfully
	UPROPERTY(BlueprintAssignable)
	FOnlineLogoutResult OnSuccess;

	// Called when the logout completed unsuccessfully
	UPROPERTY(BlueprintAssignable)
	FOnlineLogoutResult OnFailure;

	// Logs out of the online service
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online")
	static ULogoutCallbackProxy* Logout(UObject* WorldContextObject, class APlayerController* PlayerController);

	// UBlueprintAsyncActionBase interface
	virtual void Activate() override;
	// End of UBlueprintAsyncActionBase interface

private:
	// Internal callback when the login UI closes, calls out to the public success/failure callbacks
	void OnLogoutCompleted(int LocalUserNum, bool bWasSuccessful);

	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	FDelegateHandle OnLogoutCompleteDelegateHandle;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
