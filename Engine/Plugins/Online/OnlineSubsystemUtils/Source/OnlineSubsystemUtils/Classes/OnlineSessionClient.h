// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Everything a local player will use to manage online sessions.
 */

#pragma once

#include "Interfaces/OnlineSessionDelegates.h"
#include "OnlineSessionSettings.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "GameFramework/OnlineSession.h"
#include "OnlineSessionClient.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FJoinabilitySettings;

class UGameInstance;
class UNetDriver;

UCLASS(MinimalAPI, config=Game)
class UOnlineSessionClient : public UOnlineSession
{
	GENERATED_UCLASS_BODY()

protected:
	/** Delegate for destroying a session after previously ending it */
	FOnEndSessionCompleteDelegate OnEndForJoinSessionCompleteDelegate;
	/** Delegate for joining a new session after previously destroying it */
	FOnDestroySessionCompleteDelegate OnDestroyForJoinSessionCompleteDelegate;
	/** Delegate for returning to main menu after cleaning up */
	FOnDestroySessionCompleteDelegate OnDestroyForMainMenuCompleteDelegate;
	/** Delegate after joining a session */
	FOnJoinSessionCompleteDelegate OnJoinSessionCompleteDelegate;
	/** Delegate for accepting session invites */
	FOnSessionUserInviteAcceptedDelegate OnSessionUserInviteAcceptedDelegate;

	// Handles to the above delegates
	FDelegateHandle OnSessionInviteAcceptedDelegateHandle;
	FDelegateHandle OnEndForJoinSessionCompleteDelegateHandle;
	FDelegateHandle OnDestroyForJoinSessionCompleteDelegateHandle;
	FDelegateHandle OnDestroyForMainMenuCompleteDelegateHandle;
	FDelegateHandle OnJoinSessionCompleteDelegateHandle;

	/** Handle to outstanding start session call */
	FDelegateHandle StartSessionCompleteHandle;
	/** Handle to outstanding end session call */
	FDelegateHandle EndSessionCompleteHandle;
	/** Delegate handle that stores delegate for when an invite is accepted by a user */
	FDelegateHandle OnSessionUserInviteAcceptedDelegateHandle;

	/** Cached invite/search result while in the process of tearing down an existing session */
	FOnlineSessionSearchResult CachedSessionResult;
	/** Is this join from an invite */
	UPROPERTY(Transient)
	bool bIsFromInvite;
	/** Have we started returning to main menu already */
	UPROPERTY(Transient)
	bool bHandlingDisconnect;

	/** @return the current game instance */
	UE_API virtual UGameInstance* GetGameInstance() const;

	/** @return the current game world */
	UE_API virtual UWorld* GetWorld() const override;

	UE_API virtual IOnlineSessionPtr GetSessionInt();

	/**
	 * Chance for the session client to handle the disconnect
     *
     * @param World world involved in disconnect (possibly NULL for pending travel)
     * @param NetDriver netdriver involved in disconnect
     *
     * @return true if disconnect was handled, false for general engine handling
	 */
	UE_API virtual bool HandleDisconnectInternal(UWorld* World, UNetDriver* NetDriver);

	/**
	 * Transition from ending a session to destroying a session
	 *
	 * @param SessionName session that just ended
	 * @param bWasSuccessful was the end session attempt successful
	 */
	UE_API void OnEndForJoinSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Ends an existing session of a given name
	 *
	 * @param SessionName name of session to end
	 * @param Delegate delegate to call at session end
	 */
	UE_API void EndExistingSession(FName SessionName, FOnEndSessionCompleteDelegate& Delegate);

	/** 
	 * Delegate called when StartSession has completed 
	 *
	 * @param InSessionName name of session involved
	 * @param bWasSuccessful true if the call was successful, false otherwise
	 */
	UE_API virtual void OnStartSessionComplete(FName InSessionName, bool bWasSuccessful);

	/**
	 * Delegate called when EndSession has completed
	 *
	 * @param InSessionName name of session involved
	 * @param bWasSuccessful true if the call was successful, false otherwise
	 */
	UE_API virtual void OnEndSessionComplete(FName InSessionName, bool bWasSuccessful);

private:
	/**
	 * Implementation of EndExistingSession
	 *
	 * @param SessionName name of session to end
	 * @param Delegate delegate to call at session end
	 * @return Handle to the added delegate.
	 */
	FDelegateHandle EndExistingSession_Impl(FName SessionName, FOnEndSessionCompleteDelegate& Delegate);

protected:
	/**
	 * Transition from destroying a session to joining a new one of the same name
	 *
  	 * @param SessionName name of session recently destroyed
 	 * @param bWasSuccessful was the destroy attempt successful
	 */
	UE_API void OnDestroyForJoinSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Transition from destroying a session to returning to the main menu
	 *
  	 * @param SessionName name of session recently destroyed
 	 * @param bWasSuccessful was the destroy attempt successful
	 */
	UE_API void OnDestroyForMainMenuComplete(FName SessionName, bool bWasSuccessful);
	
	/**
	 * Destroys an existing session of a given name
	 *
     * @param SessionName name of session to destroy
	 * @param Delegate delegate to call at session destruction
	 */
	UE_API void DestroyExistingSession(FName SessionName, FOnDestroySessionCompleteDelegate& Delegate);

	/**
	 * Implementation of DestroyExistingSession
	 *
	 * @param OutResult Handle to the added delegate.
	 * @param SessionName name of session to destroy
	 * @param Delegate delegate to call at session destruction
	 */
	UE_API void DestroyExistingSession_Impl(FDelegateHandle& OutResult, FName SessionName, FOnDestroySessionCompleteDelegate& Delegate);

protected:

	/**
	* Called from GameInstance when the user accepts an invite
	*
	* @param bWasSuccess true if invite was accepted successfully
	* @param ControllerId the controller index of the user being invited
	* @param UserId the user being invited
	* @param InviteResult the search/settings result for the session we're joining via invite
	*/
	UE_API void OnSessionUserInviteAccepted(const bool bWasSuccess, const int32 ControllerId, FUniqueNetIdPtr UserId, const FOnlineSessionSearchResult& InviteResult) override;

	/**
	 * Delegate fired when the joining process for an online session has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	UE_API void OnJoinSessionComplete(FName SessionName, EOnJoinSessionCompleteResult::Type Result);

	/**
	 * Join a session of a given name after potentially tearing down an existing one
	 *
	 * @param LocalUserNum local user id
	 * @param SessionName name of session to join
	 * @param SearchResult the session to join
	 */
	UE_API virtual void JoinSession(FName SessionName, const FOnlineSessionSearchResult& SearchResult);

public:

	// UOnlineSession interface begin
	UE_API virtual void RegisterOnlineDelegates() override;
	UE_API virtual void ClearOnlineDelegates() override;
	UE_API virtual void HandleDisconnect(UWorld *World, UNetDriver *NetDriver) override;
	UE_API virtual void StartOnlineSession(FName SessionName) override;
	UE_API virtual void EndOnlineSession(FName SessionName) override;
	// UOnlineSession interface end

	/** 
	 * Update the session settings on the client 
	 *
	 * @param World reference to the current world
	 * @param Settings settings to apply to the session
	 */
	UE_API virtual void SetInviteFlags(UWorld* World, const FJoinabilitySettings& Settings);
};

#undef UE_API
