// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/OnlineBlueprintCallProxyBase.h"
#include "QuitMatchCallbackProxy.generated.h"

namespace EMPMatchOutcome { enum Outcome : int; }

class APlayerController;

UCLASS(MinimalAPI)
class UQuitMatchCallbackProxy : public UOnlineBlueprintCallProxyBase
{
public:
	GENERATED_UCLASS_BODY()

	virtual ~UQuitMatchCallbackProxy();

	// Called when there is a successful query
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnSuccess;

	// Called when there is an unsuccessful query
	UPROPERTY(BlueprintAssignable)
	FEmptyOnlineDelegate OnFailure;

	// Quits the turn based match
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UQuitMatchCallbackProxy* QuitMatch(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, EMPMatchOutcome::Outcome Outcome, int32 TurnTimeoutInSeconds);

	// UOnlineBlueprintCallProxyBase interface
	virtual void Activate() override;
	// End of UOnlineBlueprintCallProxyBase interface

	void QuitMatchDelegate(FString MatchID, bool Succeeded);

private:
	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;

	// The MatchID of the match to quit
	FString MatchID;

	// The outcome (won/lost/quit/etc.) of the match
	EMPMatchOutcome::Outcome Outcome;

	// If the match isn't over, this will be how much time the next player will have to complete their turn
	int32 TurnTimeoutInSeconds;
};
