// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintAsyncActionBase.h"
#include "Online/CoreOnlineFwd.h"
#include "ShowLoginUICallbackProxy.generated.h"

class APlayerController;
struct FOnlineError;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnlineShowLoginUIResult, APlayerController*, PlayerController);

UCLASS(MinimalAPI)
class UShowLoginUICallbackProxy : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	UShowLoginUICallbackProxy(const FObjectInitializer& ObjectInitializer);

	// Called when there is a successful query
	UPROPERTY(BlueprintAssignable)
	FOnlineShowLoginUIResult OnSuccess;

	// Called when there is an unsuccessful query
	UPROPERTY(BlueprintAssignable)
	FOnlineShowLoginUIResult OnFailure;

	// Shows the login UI for the currently active online subsystem, if the subsystem supports a login UI.
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", WorldContext="WorldContextObject"), Category = "Online")
	static UShowLoginUICallbackProxy* ShowExternalLoginUI(UObject* WorldContextObject, class APlayerController* InPlayerController);

	// UBlueprintAsyncActionBase interface
	virtual void Activate() override;
	// End of UBlueprintAsyncActionBase interface

private:
	// Internal callback when the login UI closes, calls out to the public success/failure callbacks
	void OnShowLoginUICompleted(FUniqueNetIdPtr UniqueId, int LocalUserNum, const FOnlineError& Error);

	// The player controller triggering things
	TWeakObjectPtr<APlayerController> PlayerControllerWeakPtr;

	// The world context object in which this call is taking place
	UObject* WorldContextObject;
};
