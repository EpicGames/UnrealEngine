// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconClient.h"
#include "TestBeaconClient.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

/**
 * A beacon client used for making reservations with an existing game session
 */
UCLASS(MinimalAPI, transient, notplaceable, config=Engine)
class ATestBeaconClient : public AOnlineBeaconClient
{
	GENERATED_UCLASS_BODY()

	//~ Begin AOnlineBeaconClient Interface
	UE_API virtual void OnFailure() override;
	//~ End AOnlineBeaconClient Interface

	/** Send a ping RPC to the client */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientPing();

	/** Send a pong RPC to the host */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerPong();
};

#undef UE_API
