// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHostObject.h"
#include "TestBeaconHost.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class AOnlineBeaconClient;

/**
 * A beacon host used for taking reservations for an existing game session
 */
UCLASS(MinimalAPI, transient, notplaceable, config=Engine)
class ATestBeaconHost : public AOnlineBeaconHostObject
{
	GENERATED_UCLASS_BODY()

	//~ Begin AOnlineBeaconHost Interface 
	UE_API virtual AOnlineBeaconClient* SpawnBeaconActor(class UNetConnection* ClientConnection) override;
	UE_API virtual void OnClientConnected(class AOnlineBeaconClient* NewClientActor, class UNetConnection* ClientConnection) override;
	//~ End AOnlineBeaconHost Interface 

	UE_API virtual bool Init();
};

#undef UE_API
