// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "TurnBasedBlueprintLibrary.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class APlayerController;

// Library of synchronous achievement calls
UCLASS(MinimalAPI, meta=(ScriptName="TurnBasedLibrary"))
class UTurnBasedBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintCallable, meta = (WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UE_API void GetIsMyTurn(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, /*out*/ bool& bIsMyTurn);

	UFUNCTION(BlueprintCallable, meta = (WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
    static UE_API void GetMyPlayerIndex(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, /*out*/ int32& PlayerIndex);

	UFUNCTION(BlueprintCallable, meta = (WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UE_API void RegisterTurnBasedMatchInterfaceObject(UObject* WorldContextObject, APlayerController* PlayerController, UObject* Object);

	UFUNCTION(BlueprintCallable, meta = (WorldContext = "WorldContextObject"), Category = "Online|TurnBased")
	static UE_API void GetPlayerDisplayName(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, int32 PlayerIndex, /*out*/ FString& PlayerDisplayName);
};

#undef UE_API
