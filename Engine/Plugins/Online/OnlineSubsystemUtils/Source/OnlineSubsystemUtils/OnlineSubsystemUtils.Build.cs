// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineSubsystemUtils : ModuleRules
{
	public OnlineSubsystemUtils(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDefinitions.Add("ONLINESUBSYSTEMUTILS_PACKAGE=1");

        bool bIsWindowsPlatformBuild = Target.Platform.IsInGroup(UnrealPlatformGroup.Windows);

        if (bIsWindowsPlatformBuild)
        {
            AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11Audio");
        }

        if (Target.bCompileAgainstEngine)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Engine"
				}
            );

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Voice",
					"AudioMixer"
				}
			);
        }

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ImageCore",
				"Sockets",
				"PacketHandler",
				"Json",
				"SignalProcessing",
				"AudioMixerCore",
				"DeveloperSettings",
				"OnlineServicesInterface"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreOnline",
				"CoreUObject",
				"NetCore"
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
				}
			);
		}

		PublicDependencyModuleNames.Add("OnlineSubsystem");

		SetupIrisSupport(Target);
	}
}
