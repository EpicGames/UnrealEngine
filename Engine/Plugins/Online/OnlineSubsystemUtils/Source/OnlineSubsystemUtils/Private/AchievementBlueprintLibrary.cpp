// Copyright Epic Games, Inc. All Rights Reserved.

#include "AchievementBlueprintLibrary.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystemBPCallHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AchievementBlueprintLibrary)

UAchievementBlueprintLibrary::UAchievementBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAchievementBlueprintLibrary::GetCachedAchievementProgress(UObject* WorldContextObject, APlayerController* PlayerController, FName AchievementID, /*out*/ bool& bFoundID, /*out*/ float& Progress)
{
	bFoundID = false;
	Progress = 0.0f;

	FOnlineSubsystemBPCallHelper Helper(TEXT("GetCachedAchievementProgress"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerController);

	if (Helper.IsValid())
	{
		IOnlineAchievementsPtr Achievements = Helper.OnlineSub->GetAchievementsInterface();
		if (Achievements.IsValid())
		{
			FOnlineAchievement AchievementStatus;
			if (Achievements->GetCachedAchievement(*Helper.UserID, AchievementID.ToString(), AchievementStatus) == EOnlineCachedResult::Success)
			{
				bFoundID = true;
				Progress = float(AchievementStatus.Progress);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Achievements not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}
}

void UAchievementBlueprintLibrary::GetCachedAchievementDescription(UObject* WorldContextObject, APlayerController* PlayerController, FName AchievementID, /*out*/ bool& bFoundID, /*out*/ FText& Title, /*out*/ FText& LockedDescription, /*out*/ FText& UnlockedDescription, /*out*/ bool& bHidden)
{
	bFoundID = false;
	Title = FText::GetEmpty();
	LockedDescription = FText::GetEmpty();
	UnlockedDescription = FText::GetEmpty();
	bHidden = false;

	FOnlineSubsystemBPCallHelper Helper(TEXT("GetCachedAchievementDescription"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerController);

	if (Helper.IsValid())
	{
		IOnlineAchievementsPtr Achievements = Helper.OnlineSub->GetAchievementsInterface();
		if (Achievements.IsValid())
		{
			FOnlineAchievementDesc AchievementDescription;
			if (Achievements->GetCachedAchievementDescription(AchievementID.ToString(), AchievementDescription) == EOnlineCachedResult::Success)
			{
				bFoundID = true;
				Title = AchievementDescription.Title;
				LockedDescription = AchievementDescription.LockedDesc;
				UnlockedDescription = AchievementDescription.UnlockedDesc;
				bHidden = AchievementDescription.bIsHidden;
				//@TODO: UnlockTime - FDateTime is not exposed to Blueprints right now, see TTP 315420
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Achievements not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}
}

