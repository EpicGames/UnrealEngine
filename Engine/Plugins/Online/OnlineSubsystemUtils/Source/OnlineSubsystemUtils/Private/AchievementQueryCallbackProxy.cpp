// Copyright Epic Games, Inc. All Rights Reserved.

#include "AchievementQueryCallbackProxy.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AchievementQueryCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UAchievementQueryCallbackProxy

UAchievementQueryCallbackProxy::UAchievementQueryCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WorldContextObject(nullptr)
{
}

UAchievementQueryCallbackProxy* UAchievementQueryCallbackProxy::CacheAchievements(UObject* WorldContextObject, class APlayerController* PlayerController)
{
	UAchievementQueryCallbackProxy* Proxy = NewObject<UAchievementQueryCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->bFetchDescriptions = false;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

UAchievementQueryCallbackProxy* UAchievementQueryCallbackProxy::CacheAchievementDescriptions(UObject* WorldContextObject, class APlayerController* PlayerController)
{
	UAchievementQueryCallbackProxy* Proxy = NewObject<UAchievementQueryCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->bFetchDescriptions = true;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UAchievementQueryCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("CacheAchievements or CacheAchievementDescriptions"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineAchievementsPtr Achievements = Helper.OnlineSub->GetAchievementsInterface();
		if (Achievements.IsValid())
		{
			FOnQueryAchievementsCompleteDelegate QueryFinishedDelegate = FOnQueryAchievementsCompleteDelegate::CreateUObject(this, &ThisClass::OnQueryCompleted);
			
			if (bFetchDescriptions)
			{
				Achievements->QueryAchievementDescriptions(*Helper.UserID, QueryFinishedDelegate);
			}
			else
			{
				Achievements->QueryAchievements(*Helper.UserID, QueryFinishedDelegate);
			}

			// OnQueryCompleted will get called, nothing more to do now
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Achievements not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UAchievementQueryCallbackProxy::OnQueryCompleted(const FUniqueNetId& UserID, bool bSuccess)
{
	if (bSuccess)
	{
		OnSuccess.Broadcast();
	}
	else
	{
		OnFailure.Broadcast();
	}
}

