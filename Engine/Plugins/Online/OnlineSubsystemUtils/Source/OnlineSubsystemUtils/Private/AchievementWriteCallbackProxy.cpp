// Copyright Epic Games, Inc. All Rights Reserved.

#include "AchievementWriteCallbackProxy.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AchievementWriteCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// ULeaderboardQueryCallbackProxy

UAchievementWriteCallbackProxy::UAchievementWriteCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WorldContextObject(nullptr)
{
}

UAchievementWriteCallbackProxy* UAchievementWriteCallbackProxy::WriteProgress(UObject* WorldContextObject, class APlayerController* PlayerController, FString AchievementName, float Progress, int32 UserTag)
{
	UAchievementWriteCallbackProxy* Proxy = NewObject<UAchievementWriteCallbackProxy>();

	Proxy->WriteObject = MakeShareable(new FOnlineAchievementsWrite);
	Proxy->WriteObject->SetFloatStat(AchievementName, Progress);
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->AchievementName = AchievementName;
	Proxy->AchievementProgress = Progress;
	Proxy->UserTag = UserTag;
	Proxy->WorldContextObject = WorldContextObject;

	return Proxy;
}

void UAchievementWriteCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("WriteAchievementObject"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineAchievementsPtr Achievements = Helper.OnlineSub->GetAchievementsInterface();
		if (Achievements.IsValid())
		{
			FOnlineAchievementsWriteRef WriteObjectRef = WriteObject.ToSharedRef();
			FOnAchievementsWrittenDelegate WriteFinishedDelegate = FOnAchievementsWrittenDelegate::CreateUObject(this, &ThisClass::OnAchievementWritten);
			Achievements->WriteAchievements(*Helper.UserID, WriteObjectRef, WriteFinishedDelegate);

			// OnAchievementWritten will get called, nothing more to do
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("WriteAchievementObject - Achievements not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnWriteFailure.Broadcast(AchievementName, AchievementProgress, UserTag);
	
	WriteObject.Reset();
}

void UAchievementWriteCallbackProxy::OnAchievementWritten(const FUniqueNetId& UserID, bool bSuccess)
{
	if (bSuccess)
	{
		OnWriteSuccess.Broadcast(AchievementName, AchievementProgress, UserTag);
	}
	else
	{
		OnWriteFailure.Broadcast(AchievementName, AchievementProgress, UserTag);
	}

	WriteObject.Reset();
}

void UAchievementWriteCallbackProxy::BeginDestroy()
{
	WriteObject.Reset();

	Super::BeginDestroy();
}

