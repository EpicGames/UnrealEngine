// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConnectionCallbackProxy.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemBPCallHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConnectionCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UConnectionCallbackProxy

UConnectionCallbackProxy::UConnectionCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WorldContextObject(nullptr)
{

	OnLoginCompleteDelegate.BindUObject(this, &ThisClass::OnLoginCompleted);

}

UConnectionCallbackProxy* UConnectionCallbackProxy::ConnectToService(UObject* WorldContextObject, class APlayerController* PlayerController)
{
	UConnectionCallbackProxy* Proxy = NewObject<UConnectionCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UConnectionCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("ConnectToService"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineIdentityPtr OnlineIdentity = Helper.OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			const int32 ControllerId = CastChecked<ULocalPlayer>(PlayerControllerWeakPtr.Get()->Player)->GetControllerId();

			if (!OnlineIdentity->OnLoginCompleteDelegates[ControllerId].IsBoundToObject(this))
			{
				OnLoginCompleteDelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(ControllerId, OnLoginCompleteDelegate);
				OnlineIdentity->Login(ControllerId, FOnlineAccountCredentials()); /// Probably need to supply real creds here somehow... doesn't apply for all imple however.
			}
			else
			{
				// already trying to login!
			}
			
			// OnQueryCompleted will get called, nothing more to do now
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Connection control not supported by online subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast(0);
}

void UConnectionCallbackProxy::OnLoginCompleted(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("ConnectToService"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineIdentityPtr OnlineIdentity = Helper.OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, OnLoginCompleteDelegateHandle);
		}
	}

	if (bWasSuccessful)
	{
		OnSuccess.Broadcast(0);
	}
	else
	{
		OnFailure.Broadcast(0);
	}
}

