// Copyright Epic Games, Inc. All Rights Reserved.

#include "CreateSessionCallbackProxy.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CreateSessionCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UCreateSessionCallbackProxy

UCreateSessionCallbackProxy::UCreateSessionCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CreateCompleteDelegate(FOnCreateSessionCompleteDelegate::CreateUObject(this, &ThisClass::OnCreateCompleted))
	, StartCompleteDelegate(FOnStartSessionCompleteDelegate::CreateUObject(this, &ThisClass::OnStartCompleted))
	, NumPublicConnections(1)
{
}

UCreateSessionCallbackProxy* UCreateSessionCallbackProxy::CreateSession(UObject* WorldContextObject, class APlayerController* PlayerController, int32 PublicConnections, bool bUseLAN, bool bUseLobbiesIfAvailable)
{
	UCreateSessionCallbackProxy* Proxy = NewObject<UCreateSessionCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->NumPublicConnections = PublicConnections;
	Proxy->bUseLAN = bUseLAN;
	Proxy->bUseLobbiesIfAvailable = bUseLobbiesIfAvailable;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UCreateSessionCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("CreateSession"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			CreateCompleteDelegateHandle = Sessions->AddOnCreateSessionCompleteDelegate_Handle(CreateCompleteDelegate);

			FOnlineSessionSettings Settings;
			Settings.NumPublicConnections = NumPublicConnections;
			Settings.bShouldAdvertise = true;
			Settings.bAllowJoinInProgress = true;
			Settings.bIsLANMatch = bUseLAN;
			Settings.bUsesPresence = true;
			Settings.bAllowJoinViaPresence = true;
			Settings.bUseLobbiesIfAvailable = bUseLobbiesIfAvailable;

			Sessions->CreateSession(*Helper.UserID, NAME_GameSession, Settings);

			// OnCreateCompleted will get called, nothing more to do now
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Sessions not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UCreateSessionCallbackProxy::OnCreateCompleted(FName SessionName, bool bWasSuccessful)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("CreateSessionCallback"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			Sessions->ClearOnCreateSessionCompleteDelegate_Handle(CreateCompleteDelegateHandle);
			
			if (bWasSuccessful)
			{
				StartCompleteDelegateHandle = Sessions->AddOnStartSessionCompleteDelegate_Handle(StartCompleteDelegate);
				Sessions->StartSession(NAME_GameSession);

				// OnStartCompleted will get called, nothing more to do now
				return;
			}
		}
	}

	if (!bWasSuccessful)
	{
		OnFailure.Broadcast();
	}
}

void UCreateSessionCallbackProxy::OnStartCompleted(FName SessionName, bool bWasSuccessful)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("StartSessionCallback"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			Sessions->ClearOnStartSessionCompleteDelegate_Handle(StartCompleteDelegateHandle);
		}
	}

	if (bWasSuccessful)
	{
		OnSuccess.Broadcast();
	}
	else
	{
		OnFailure.Broadcast();
	}
}

