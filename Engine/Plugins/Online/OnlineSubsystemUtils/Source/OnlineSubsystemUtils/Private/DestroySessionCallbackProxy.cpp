// Copyright Epic Games, Inc. All Rights Reserved.

#include "DestroySessionCallbackProxy.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DestroySessionCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UDestroySessionCallbackProxy

UDestroySessionCallbackProxy::UDestroySessionCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Delegate(FOnDestroySessionCompleteDelegate::CreateUObject(this, &ThisClass::OnCompleted))
{
}

UDestroySessionCallbackProxy* UDestroySessionCallbackProxy::DestroySession(UObject* WorldContextObject, class APlayerController* PlayerController)
{
	UDestroySessionCallbackProxy* Proxy = NewObject<UDestroySessionCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UDestroySessionCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("DestroySession"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			DelegateHandle = Sessions->AddOnDestroySessionCompleteDelegate_Handle(Delegate);
			Sessions->DestroySession(NAME_GameSession);

			// OnCompleted will get called, nothing more to do now
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Sessions not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UDestroySessionCallbackProxy::OnCompleted(FName SessionName, bool bWasSuccessful)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("DestroySessionCallback"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			Sessions->ClearOnDestroySessionCompleteDelegate_Handle(DelegateHandle);
		}
	}

	if (bWasSuccessful)
	{
		OnSuccess.Broadcast();
	}
	else
	{
		OnFailure.Broadcast();
	}
}

