// Copyright Epic Games, Inc. All Rights Reserved.

#include "EndMatchCallbackProxy.h"
#include "Interfaces/OnlineTurnBasedInterface.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"
#include "OnlineSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EndMatchCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UEndMatchCallbackProxy

UEndMatchCallbackProxy::UEndMatchCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WorldContextObject(nullptr)
	, TurnBasedMatchInterface(nullptr)
{
}

UEndMatchCallbackProxy::~UEndMatchCallbackProxy()
{
}

UEndMatchCallbackProxy* UEndMatchCallbackProxy::EndMatch(UObject* WorldContextObject, class APlayerController* PlayerController, TScriptInterface<ITurnBasedMatchInterface> MatchActor, FString MatchID, EMPMatchOutcome::Outcome LocalPlayerOutcome, EMPMatchOutcome::Outcome OtherPlayersOutcome)
{
	UEndMatchCallbackProxy* Proxy = NewObject<UEndMatchCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	Proxy->MatchID = MatchID;
	Proxy->LocalPlayerOutcome = LocalPlayerOutcome;
	Proxy->OtherPlayersOutcome = OtherPlayersOutcome;
	return Proxy;
}

void UEndMatchCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("EndMatch"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr TurnBasedInterface = Helper.OnlineSub->GetTurnBasedInterface();
		if (TurnBasedInterface.IsValid())
		{
			FTurnBasedMatchPtr Match = TurnBasedInterface->GetMatchWithID(MatchID);
			if (Match.IsValid())
			{
				FEndMatchSignature EndMatchDelegate;
				EndMatchDelegate.BindUObject(this, &UEndMatchCallbackProxy::EndMatchDelegate);
				Match->EndMatch(EndMatchDelegate, LocalPlayerOutcome, OtherPlayersOutcome);
				return;
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn based games not supported by online subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UEndMatchCallbackProxy::EndMatchDelegate(FString InMatchID, bool Succeeded)
{
	if (Succeeded)
	{ 
		OnSuccess.Broadcast();
	}
	else
	{
		OnFailure.Broadcast();
	}
}

