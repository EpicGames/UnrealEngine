// Copyright Epic Games, Inc. All Rights Reserved.

#include "EndTurnCallbackProxy.h"
#include "Interfaces/TurnBasedMatchInterface.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "Interfaces/OnlineTurnBasedInterface.h"
#include "OnlineSubsystem.h"
#include "GameFramework/PlayerController.h"
#include "Net/RepLayout.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EndTurnCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UTurnBasedMatchEndTurnCallbackProxy

UEndTurnCallbackProxy::UEndTurnCallbackProxy(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UEndTurnCallbackProxy::~UEndTurnCallbackProxy()
{
}

UEndTurnCallbackProxy* UEndTurnCallbackProxy::EndTurn(UObject* WorldContextObject, class APlayerController* PlayerController, FString MatchID, TScriptInterface<ITurnBasedMatchInterface> TurnBasedMatchInterface)
{
	UEndTurnCallbackProxy* Proxy = NewObject<UEndTurnCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	Proxy->MatchID = MatchID;
	Proxy->TurnBasedMatchInterface = (UTurnBasedMatchInterface*)TurnBasedMatchInterface.GetObject();
	return Proxy;
}

void UEndTurnCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("EndTurn"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr TurnBasedInterface = Helper.OnlineSub->GetTurnBasedInterface();
		if (TurnBasedInterface.IsValid())
		{
			if (TurnBasedMatchInterface != nullptr)
			{
				// TODO: We should cache off the RepLayout by class, just like we do in NetDriver.
				FBitWriter Writer(TurnBasedInterface->GetMatchDataSize());
				FRepLayout::CreateFromClass(TurnBasedMatchInterface->GetClass())->SerializeObjectReplicatedProperties(TurnBasedMatchInterface, Writer);

				FUploadMatchDataSignature UploadMatchDataSignature;
				UploadMatchDataSignature.BindUObject(this, &UEndTurnCallbackProxy::UploadMatchDataDelegate);

				FTurnBasedMatchPtr TurnBasedMatchPtr = TurnBasedInterface->GetMatchWithID(MatchID);
				if (TurnBasedMatchPtr.IsValid())
				{
					TurnBasedMatchPtr->EndTurnWithMatchData(*Writer.GetBuffer(), 0, UploadMatchDataSignature);
					return;
				}

				FString Message = FString::Printf(TEXT("Match ID %s not found"), *MatchID);
				FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("No match data passed in to End Turn."), ELogVerbosity::Warning);
			}
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn based games not supported by online subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UEndTurnCallbackProxy::UploadMatchDataDelegate(FString InMatchID, bool Succeeded)
{
	if (Succeeded)
	{
		OnSuccess.Broadcast();
	}
	else
	{
		OnFailure.Broadcast();
	}
}

