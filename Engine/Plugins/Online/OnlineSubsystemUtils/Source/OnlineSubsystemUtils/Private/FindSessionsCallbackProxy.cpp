// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindSessionsCallbackProxy.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FindSessionsCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UFindSessionsCallbackProxy

UFindSessionsCallbackProxy::UFindSessionsCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Delegate(FOnFindSessionsCompleteDelegate::CreateUObject(this, &ThisClass::OnCompleted))
	, bUseLAN(false)
{
}

UFindSessionsCallbackProxy* UFindSessionsCallbackProxy::FindSessions(UObject* WorldContextObject, class APlayerController* PlayerController, int MaxResults, bool bUseLAN, bool bUseLobbies)
{
	UFindSessionsCallbackProxy* Proxy = NewObject<UFindSessionsCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->bUseLAN = bUseLAN;
	Proxy->bUseLobbies = bUseLobbies;
	Proxy->MaxResults = MaxResults;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UFindSessionsCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("FindSessions"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			DelegateHandle = Sessions->AddOnFindSessionsCompleteDelegate_Handle(Delegate);
			
			SearchObject = MakeShareable(new FOnlineSessionSearch);
			SearchObject->MaxSearchResults = MaxResults;
			SearchObject->bIsLanQuery = bUseLAN;
			if (bUseLobbies)
			{
				SearchObject->QuerySettings.Set(SEARCH_LOBBIES, true, EOnlineComparisonOp::Equals);
			}

			Sessions->FindSessions(*Helper.UserID, SearchObject.ToSharedRef());

			// OnQueryCompleted will get called, nothing more to do now
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Sessions not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	TArray<FBlueprintSessionResult> Results;
	OnFailure.Broadcast(Results);
}

void UFindSessionsCallbackProxy::OnCompleted(bool bSuccess)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("FindSessionsCallback"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			Sessions->ClearOnFindSessionsCompleteDelegate_Handle(DelegateHandle);
		}
	}

	TArray<FBlueprintSessionResult> Results;

	if (bSuccess && SearchObject.IsValid())
	{
		for (auto& Result : SearchObject->SearchResults)
		{
			FBlueprintSessionResult BPResult;
			BPResult.OnlineResult = Result;
			Results.Add(BPResult);
		}

		OnSuccess.Broadcast(Results);
	}
	else
	{
		OnFailure.Broadcast(Results);
	}
}

int32 UFindSessionsCallbackProxy::GetPingInMs(const FBlueprintSessionResult& Result)
{
	return Result.OnlineResult.PingInMs;
}

FString UFindSessionsCallbackProxy::GetServerName(const FBlueprintSessionResult& Result)
{
	return Result.OnlineResult.Session.OwningUserName;
}

int32 UFindSessionsCallbackProxy::GetCurrentPlayers(const FBlueprintSessionResult& Result)
{
	return Result.OnlineResult.Session.SessionSettings.NumPublicConnections - Result.OnlineResult.Session.NumOpenPublicConnections;
}

int32 UFindSessionsCallbackProxy::GetMaxPlayers(const FBlueprintSessionResult& Result)
{
	return Result.OnlineResult.Session.SessionSettings.NumPublicConnections;
}

