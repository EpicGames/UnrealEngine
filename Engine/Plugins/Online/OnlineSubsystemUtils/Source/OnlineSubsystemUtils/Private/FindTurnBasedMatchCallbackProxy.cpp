// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindTurnBasedMatchCallbackProxy.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"
#include "Net/RepLayout.h"
#include "OnlineSubsystem.h"
#include "Interfaces/TurnBasedMatchInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FindTurnBasedMatchCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UFindTurnBasedMatchCallbackProxy

UFindTurnBasedMatchCallbackProxy::UFindTurnBasedMatchCallbackProxy(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
, WorldContextObject(nullptr)
, Delegate(new FFindTurnBasedMatchCallbackProxyMatchmakerDelegate)
{
}

UFindTurnBasedMatchCallbackProxy::~UFindTurnBasedMatchCallbackProxy()
{
}

UFindTurnBasedMatchCallbackProxy* UFindTurnBasedMatchCallbackProxy::FindTurnBasedMatch(UObject* WorldContextObject, class APlayerController* PlayerController, TScriptInterface<ITurnBasedMatchInterface> MatchActor, int32 MinPlayers, int32 MaxPlayers, int32 PlayerGroup, bool ShowExistingMatches)
{
	UFindTurnBasedMatchCallbackProxy* Proxy = NewObject<UFindTurnBasedMatchCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	Proxy->MinPlayers = MinPlayers;
	Proxy->MaxPlayers = MaxPlayers;
	Proxy->PlayerGroup = PlayerGroup;
	Proxy->ShowExistingMatches = ShowExistingMatches;
	Proxy->TurnBasedMatchInterface = (UTurnBasedMatchInterface*)MatchActor.GetObject();
	return Proxy;
}

void UFindTurnBasedMatchCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("FindTurnBasedMatch"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr TurnBasedInterface = Helper.OnlineSub->GetTurnBasedInterface();
		if (TurnBasedInterface.IsValid())
		{
			Delegate->SetFindTurnBasedMatchCallbackProxy(this);
			Delegate->SetTurnBasedInterface(TurnBasedInterface);
			TurnBasedInterface->SetMatchmakerDelegate(Delegate);
			FTurnBasedMatchRequest MatchRequest(MinPlayers, MaxPlayers, PlayerGroup, ShowExistingMatches);
			TurnBasedInterface->ShowMatchmaker(MatchRequest);

			// Results will be handled in the FFindTurnBasedMatchCallbackProxyMatchmakerDelegate object
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn based games not supported by online subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast(FString());
}

FFindTurnBasedMatchCallbackProxyMatchmakerDelegate::FFindTurnBasedMatchCallbackProxyMatchmakerDelegate()
: FTurnBasedMatchmakerDelegate()
, FindTurnBasedMatchCallbackProxy(nullptr)
, TurnBasedInterface(nullptr)
{
}

FFindTurnBasedMatchCallbackProxyMatchmakerDelegate::~FFindTurnBasedMatchCallbackProxyMatchmakerDelegate()
{
}

void FFindTurnBasedMatchCallbackProxyMatchmakerDelegate::OnMatchmakerCancelled()
{
	if (FindTurnBasedMatchCallbackProxy)
	{
		FindTurnBasedMatchCallbackProxy->OnFailure.Broadcast(FString());
	}
}

void FFindTurnBasedMatchCallbackProxyMatchmakerDelegate::OnMatchmakerFailed()
{
	if (FindTurnBasedMatchCallbackProxy)
	{
		FindTurnBasedMatchCallbackProxy->OnFailure.Broadcast(FString());
	}
}

void FFindTurnBasedMatchCallbackProxyMatchmakerDelegate::OnMatchFound(FTurnBasedMatchRef Match)
{
    UE_LOG_ONLINE(Verbose, TEXT("Turn-based match found: %s"), *Match->GetMatchID());
	TArray<uint8> MatchData;
	
	if (Match->GetMatchData(MatchData) && FindTurnBasedMatchCallbackProxy)
	{
		// TODO: We should cache off the RepLayout by class, just like we do in NetDriver.
		FBitReader Reader(MatchData.GetData(), TurnBasedInterface->GetMatchDataSize());
		FRepLayout::CreateFromClass(FindTurnBasedMatchCallbackProxy->GetTurnBasedMatchInterfaceObject()->GetClass())->SerializeObjectReplicatedProperties(FindTurnBasedMatchCallbackProxy->GetTurnBasedMatchInterfaceObject(), Reader);
	}

	if (FindTurnBasedMatchCallbackProxy)
	{
		FindTurnBasedMatchCallbackProxy->OnSuccess.Broadcast(Match->GetMatchID());
	}
}

