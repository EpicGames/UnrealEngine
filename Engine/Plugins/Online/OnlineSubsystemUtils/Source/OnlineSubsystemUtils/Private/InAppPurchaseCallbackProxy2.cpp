// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseCallbackProxy2.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/PlayerController.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "OnlineError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseCallbackProxy2)

//////////////////////////////////////////////////////////////////////////
// UInAppPurchaseCallbackProxy

UInAppPurchaseCallbackProxy2::UInAppPurchaseCallbackProxy2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WorldPtr = nullptr;
	bWasSuccessful = false;
	SavedPurchaseStatus = EInAppPurchaseStatus::Failed;
}


void UInAppPurchaseCallbackProxy2::Trigger(APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest)
{
	bFailedToEvenSubmit = true;
	WorldPtr = nullptr;
	if (PlayerController)
	{
		WorldPtr = PlayerController->GetWorld();
		if (APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState))
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				PurchaseInterface = OnlineSub->GetPurchaseInterface();
				if (PurchaseInterface.IsValid())
				{
					bFailedToEvenSubmit = false;

					// Register the completion callback
					InAppPurchaseCompleteDelegate = FOnPurchaseCheckoutComplete::CreateUObject(this, &UInAppPurchaseCallbackProxy2::OnCheckoutComplete);

					// Set-up, and trigger the transaction through the store interface
					FPurchaseCheckoutRequest CheckoutRequest = FPurchaseCheckoutRequest();
					CheckoutRequest.AddPurchaseOffer("", ProductRequest.ProductIdentifier, 1, ProductRequest.bIsConsumable);
					PurchasingPlayer = (*PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId()).AsShared();
					PurchaseInterface->Checkout(*PurchasingPlayer, CheckoutRequest, InAppPurchaseCompleteDelegate);
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		}

		if (bFailedToEvenSubmit)
		{
			OnPurchaseComplete();
		}
	}
}

void UInAppPurchaseCallbackProxy2::TriggerGetUnprocessedPurchases(APlayerController* PlayerController)
{
	WorldPtr = nullptr;
	APlayerState* PlayerState = nullptr;
	if (PlayerController != nullptr)
	{
		WorldPtr = PlayerController->GetWorld();
		PlayerState = ToRawPtr(PlayerController->PlayerState);
	}

	if (PlayerState != nullptr)
	{
		if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
		{
			PurchaseInterface = OnlineSub->GetPurchaseInterface();
			if (PurchaseInterface.IsValid())
			{
				check(PlayerController);
				PurchasingPlayer = (*PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId()).AsShared();
				OnCheckoutComplete(FOnlineError(true), MakeShared<FPurchaseReceipt>());
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
	}
}

void UInAppPurchaseCallbackProxy2::TriggerGetOwnedPurchases(APlayerController* PlayerController)
{
	bFailedToEvenSubmit = true;
	WorldPtr = nullptr;
	if (PlayerController)
	{
		WorldPtr = PlayerController->GetWorld();
		if (APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState))
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				PurchaseInterface = OnlineSub->GetPurchaseInterface();
				if (PurchaseInterface.IsValid())
				{
					check(PlayerController);
					PurchasingPlayer = (*PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId()).AsShared();
					PurchaseInterface->QueryReceipts(*PurchasingPlayer, false, FOnQueryReceiptsComplete::CreateUObject(this, &UInAppPurchaseCallbackProxy2::OnQueryReceiptsComplete));
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		}

		if (bFailedToEvenSubmit)
		{
			OnPurchaseComplete();
		}
	}
}

void UInAppPurchaseCallbackProxy2::OnCheckoutComplete(const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
{
	
	TArray<FPurchaseReceipt> OutReceipts;
	bWasSuccessful = false;
	bool bWasSuccessfulLocal = Result.WasSuccessful();
	SavedPurchaseStatus = ::PurchaseStatusFromOnlineError(Result);
	if (bWasSuccessfulLocal)
	{
		if (PurchaseInterface.IsValid())
		{
			if (PurchasingPlayer.IsValid())
			{
				// Get Receipts
				PurchaseInterface->GetReceipts(*PurchasingPlayer, OutReceipts);
				for (FPurchaseReceipt& CurrentReceipt : OutReceipts)
				{
					if (CurrentReceipt.TransactionState == EPurchaseTransactionState::Purchased ||
						CurrentReceipt.TransactionState == EPurchaseTransactionState::Restored)
					{
						FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - FinalizingPurchase"), ELogVerbosity::Warning);
						PurchaseInterface->FinalizePurchase(*PurchasingPlayer, CurrentReceipt.TransactionId);
						bWasSuccessful = bWasSuccessfulLocal;
					}
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Invalid or uninitialized PurchasingPlayer"), ELogVerbosity::Warning);
			}
		}
	}

	FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Emptying savedreceipts"), ELogVerbosity::Warning);
	SavedReceipts.Empty();

	FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Iterating OutReceipts"), ELogVerbosity::Warning);
	for(FPurchaseReceipt& CurrentReceipt : OutReceipts)
	{
		if (CurrentReceipt.TransactionState == EPurchaseTransactionState::Purchased ||
			CurrentReceipt.TransactionState == EPurchaseTransactionState::Restored)
		{
			FInAppPurchaseReceiptInfo2& ReceiptInfo = SavedReceipts.AddDefaulted_GetRef();

			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Writing Receipt to savedreceipts"), ELogVerbosity::Warning);
			ReceiptInfo.ItemId = CurrentReceipt.ReceiptOffers[0].LineItems[0].UniqueId;
			ReceiptInfo.ItemName = CurrentReceipt.ReceiptOffers[0].LineItems[0].ItemName;
			ReceiptInfo.ValidationInfo = CurrentReceipt.ReceiptOffers[0].LineItems[0].ValidationInfo;
		}
	}
	FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Calling OnPurchaseCOmplete"), ELogVerbosity::Warning);
	OnPurchaseComplete();
}

void UInAppPurchaseCallbackProxy2::OnPurchaseComplete()
{
	if (UWorld* World = WorldPtr.Get())
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.DelayInAppPurchaseComplete"), STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseComplete, STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateLambda([this]() {

			OnInAppPurchaseComplete_Delayed();

		}),
			GET_STATID(STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseComplete),
			nullptr,
			ENamedThreads::GameThread
			);
	}
}

void UInAppPurchaseCallbackProxy2::OnQueryReceiptsComplete(const FOnlineError& Result)
{
		SavedReceipts.Empty();
		bWasSuccessful = Result.WasSuccessful();
		SavedPurchaseStatus = ::PurchaseStatusFromOnlineError(Result);
		
		OnPurchaseComplete();
}

void UInAppPurchaseCallbackProxy2::OnInAppPurchaseComplete_Delayed()
{
	/** Cached product details of the purchased product */
	if (bWasSuccessful)
	{
		OnSuccess.Broadcast(SavedPurchaseStatus, SavedReceipts);
	}
	else
	{
		OnFailure.Broadcast(SavedPurchaseStatus, SavedReceipts);
	}
}

void UInAppPurchaseCallbackProxy2::BeginDestroy()
{
	Super::BeginDestroy();
}

UInAppPurchaseCallbackProxy2* UInAppPurchaseCallbackProxy2::CreateProxyObjectForInAppPurchase(class APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest)
{
	UInAppPurchaseCallbackProxy2* Proxy = NewObject<UInAppPurchaseCallbackProxy2>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->Trigger(PlayerController, ProductRequest);
	return Proxy;
}

UInAppPurchaseCallbackProxy2* UInAppPurchaseCallbackProxy2::CreateProxyObjectForInAppPurchaseUnprocessedPurchases(class APlayerController* PlayerController)
{
	UInAppPurchaseCallbackProxy2* Proxy = NewObject<UInAppPurchaseCallbackProxy2>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerGetUnprocessedPurchases(PlayerController);
	return Proxy;
}

UInAppPurchaseCallbackProxy2* UInAppPurchaseCallbackProxy2::CreateProxyObjectForInAppPurchaseQueryOwned(class APlayerController* PlayerController)
{
	UInAppPurchaseCallbackProxy2* Proxy = NewObject<UInAppPurchaseCallbackProxy2>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerGetOwnedPurchases(PlayerController);
	return Proxy;
}
