// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseCheckoutCallbackProxy.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/PlayerController.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "Misc/ScopeExit.h"
#include "OnlineError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseCheckoutCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UInAppPurchaseCallbackProxy

UInAppPurchaseCheckoutCallbackProxy::UInAppPurchaseCheckoutCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInAppPurchaseCheckoutCallbackProxy::ClearPreviousResult()
{
	SavedReceipt.Reset();
	SavedPurchaseStatus = EInAppPurchaseStatus::Failed;
	WorldPtr = nullptr;
}


void UInAppPurchaseCheckoutCallbackProxy::TriggerCheckout(APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest)
{
	ClearPreviousResult();

	if (PlayerController == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		return;
	}

	WorldPtr = PlayerController->GetWorld();

	bool bFailedToEvenSubmit = true;
	ON_SCOPE_EXIT
	{
		if (bFailedToEvenSubmit)
		{
			OnPurchaseComplete();
		}
	};

	APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState);

	if (PlayerState == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		return;
	}

	IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr;
	if (OnlineSub == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
		return;
	}

	IOnlinePurchasePtr PurchaseInterface = OnlineSub->GetPurchaseInterface();
	if (!PurchaseInterface.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
		return;
	}

	if( FUniqueNetIdRepl PurchasingPlayer = PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId(); PurchasingPlayer.IsValid())
	{
		bFailedToEvenSubmit = false;

		// Register the completion callback
		FOnPurchaseCheckoutComplete InAppPurchaseCompleteDelegate = FOnPurchaseCheckoutComplete::CreateUObject(this, &UInAppPurchaseCheckoutCallbackProxy::OnCheckoutComplete);

		// Set-up, and trigger the transaction through the store interface
		FPurchaseCheckoutRequest CheckoutRequest = FPurchaseCheckoutRequest();
		CheckoutRequest.AddPurchaseOffer("", ProductRequest.ProductIdentifier, 1, ProductRequest.bIsConsumable);
		PurchaseInterface->Checkout(*PurchasingPlayer, CheckoutRequest, InAppPurchaseCompleteDelegate);
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player UniqueNetId"), ELogVerbosity::Warning);
	}
}

void UInAppPurchaseCheckoutCallbackProxy::OnCheckoutComplete(const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
{
	bool bWasSuccessful = Result.WasSuccessful();
	SavedPurchaseStatus = PurchaseStatusFromOnlineError(Result);

	if (bWasSuccessful)
	{
		FInAppPurchaseReceiptInfo2 ReceiptInfo;

		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Success: Storing receipt"), ELogVerbosity::Log);
		ReceiptInfo.ItemId = Receipt->ReceiptOffers[0].LineItems[0].UniqueId;
		ReceiptInfo.ItemName = Receipt->ReceiptOffers[0].LineItems[0].ItemName;
		ReceiptInfo.ValidationInfo = Receipt->ReceiptOffers[0].LineItems[0].ValidationInfo;
		ReceiptInfo.TransactionIdentifier = Receipt->TransactionId;

		SavedReceipt = MoveTemp(ReceiptInfo);
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnCheckoutComplete - Failure: Emptying saved receipts"), ELogVerbosity::Log);
		SavedReceipt.Reset();
	}

	OnPurchaseComplete();
}

void UInAppPurchaseCheckoutCallbackProxy::OnPurchaseComplete()
{
	if (UWorld* World = WorldPtr.Get())
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.DelayInAppPurchaseComplete"), STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseComplete, STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateLambda([this]() {
				if (SavedReceipt.IsSet())
				{
					OnSuccess.Broadcast(SavedPurchaseStatus, SavedReceipt.GetValue());
				}
				else
				{
					OnFailure.Broadcast(SavedPurchaseStatus, FInAppPurchaseReceiptInfo2());
				}
			}),
			GET_STATID(STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseComplete),
			nullptr,
			ENamedThreads::GameThread
			);
	}
}

UInAppPurchaseCheckoutCallbackProxy* UInAppPurchaseCheckoutCallbackProxy::CreateProxyObjectForInAppPurchaseCheckout(class APlayerController* PlayerController, const FInAppPurchaseProductRequest2& ProductRequest)
{
	UInAppPurchaseCheckoutCallbackProxy* Proxy = NewObject<UInAppPurchaseCheckoutCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerCheckout(PlayerController, ProductRequest);
	return Proxy;
}