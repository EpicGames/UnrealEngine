// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseDataTypes.h"

#include "OnlineError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseDataTypes)

EInAppPurchaseStatus PurchaseStatusFromOnlineError(const FOnlineError& OnlineError)
{
	if (OnlineError.bSucceeded)
	{
		return EInAppPurchaseStatus::Purchased;
	}
	else if(OnlineError.ErrorCode.Equals(TEXT("com.epicgames.purchase.failure")))
	{
		return EInAppPurchaseStatus::Failed;
	}
	else if (OnlineError.ErrorCode.Equals(TEXT("com.epicgames.catalog_helper.user_cancelled")))
	{
		return EInAppPurchaseStatus::Canceled;
	}
	else if (OnlineError.ErrorCode.Equals(TEXT("com.epicgames.purchase.deferred")))
	{
		return EInAppPurchaseStatus::Deferred;
	}
	else if (OnlineError.ErrorCode.Equals(TEXT("com.epicgames.purchase.invalid")))
	{
		return EInAppPurchaseStatus::Invalid;
	}
	
	return EInAppPurchaseStatus::Failed;
}
