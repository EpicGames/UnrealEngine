// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseFinalizeProxy.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "InAppPurchaseCheckoutCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseFinalizeProxy)

//////////////////////////////////////////////////////////////////////////
// UInAppPurchaseFinalizeProxy

UInAppPurchaseFinalizeProxy::UInAppPurchaseFinalizeProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UInAppPurchaseFinalizeProxy::Trigger(const FInAppPurchaseReceiptInfo2& InAppPurchaseReceipt, APlayerController* PlayerController)
{
	APlayerState* PlayerState = nullptr;
	if (PlayerController == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseFinalizeProxy::Trigger - Invalid player controller"), ELogVerbosity::Warning);
		return;
	}

	PlayerState = ToRawPtr(PlayerController->PlayerState);

	if (PlayerState == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseFinalizeProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		return;
	}

	IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr;
	if (OnlineSub == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseFinalizeProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
		return;
	}

	IOnlinePurchasePtr PurchaseInterface = OnlineSub->GetPurchaseInterface();
	if (!PurchaseInterface.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseFinalizeProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
		return;
	}

	if (FUniqueNetIdRepl PurchasingPlayer = PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId(); PurchasingPlayer.IsValid())
	{
		PurchaseInterface->FinalizePurchase(*PurchasingPlayer, InAppPurchaseReceipt.TransactionIdentifier, InAppPurchaseReceipt.ValidationInfo);
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseFinalizeProxy::Trigger - In-App Purchase finalized"), ELogVerbosity::Log);
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseFinalizeProxy::Trigger - Innvalid player UniqueNetId"), ELogVerbosity::Warning);					
	}
}

UInAppPurchaseFinalizeProxy* UInAppPurchaseFinalizeProxy::CreateProxyObjectForInAppPurchaseFinalize(const FInAppPurchaseReceiptInfo2& InAppPurchaseReceipt, class APlayerController* PlayerController)
{
	UInAppPurchaseFinalizeProxy* Proxy = NewObject<UInAppPurchaseFinalizeProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->Trigger(InAppPurchaseReceipt, PlayerController);
	return Proxy;
}