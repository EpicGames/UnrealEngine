// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseQueryCallbackProxy2.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/PlayerController.h"
#include "OnlineSubsystem.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseQueryCallbackProxy2)

//////////////////////////////////////////////////////////////////////////
// UInAppPurchaseQueryCallbackProxy

UInAppPurchaseQueryCallbackProxy2::UInAppPurchaseQueryCallbackProxy2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInAppPurchaseQueryCallbackProxy2::TriggerQuery(APlayerController* PlayerController, const TArray<FString>& ProductIdentifiers)
{
	bool bFailedToEvenSubmit = true;

	WorldPtr = nullptr;
	if (PlayerController)
	{
		WorldPtr = PlayerController->GetWorld();
		if (APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState))
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				IOnlineStoreV2Ptr StoreInterface = OnlineSub->GetStoreV2Interface();
				if (StoreInterface.IsValid())
				{
					FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - GetUniqueNetIdFromCachedControllerId"), ELogVerbosity::Log);
					check(PlayerController);
					FUniqueNetIdRepl QueryingPlayer = PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId();

					if (QueryingPlayer.IsValid())
					{
						FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Querying Store Interface"), ELogVerbosity::Log);
						bFailedToEvenSubmit = false;
						StoreInterface->QueryOffersById(*QueryingPlayer, ProductIdentifiers, FOnQueryOnlineStoreOffersComplete::CreateUObject(this, &UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead));
					}
					else
					{
						FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Invalid UniqueNetId"), ELogVerbosity::Warning);
					}
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - In App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Invalid player state"), ELogVerbosity::Warning);
		}

		if (bFailedToEvenSubmit)
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Failed to even submit"), ELogVerbosity::Warning);
			OnInAppPurchaseRead(false, TArray<FUniqueOfferId>(), FString());
		}
	}
}

void UInAppPurchaseQueryCallbackProxy2::CreateProxyProductInformation(TArray<FOnlineStoreOfferRef>& SourceArray, TArray<FOnlineProxyStoreOffer>& TargetArray)
{
	FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - CreateProxyInformation"), ELogVerbosity::Log);
	TargetArray.Empty();
	for (FOnlineStoreOfferRef SourceProduct : SourceArray)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - CreateProxyInformation In Loop"), ELogVerbosity::Log);
		FOnlineProxyStoreOffer CurrentProduct;

		CurrentProduct.OfferId = SourceProduct->OfferId;
		CurrentProduct.Title = SourceProduct->Title;
		CurrentProduct.Description = SourceProduct->Description;
		CurrentProduct.LongDescription = SourceProduct->LongDescription;
		CurrentProduct.RegularPriceText = SourceProduct->RegularPriceText;
		CurrentProduct.RegularPrice = int32(SourceProduct->RegularPrice);
		CurrentProduct.PriceText = SourceProduct->PriceText;
		CurrentProduct.NumericPrice = int32(SourceProduct->NumericPrice);
		CurrentProduct.CurrencyCode = SourceProduct->CurrencyCode;
		CurrentProduct.ReleaseDate = SourceProduct->ReleaseDate;
		CurrentProduct.ExpirationDate = SourceProduct->ExpirationDate;
		CurrentProduct.DiscountType = (EOnlineProxyStoreOfferDiscountType)SourceProduct->DiscountType;
		CurrentProduct.DynamicFields = SourceProduct->DynamicFields;

		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Adding Product"), ELogVerbosity::Log);
		TargetArray.Add(CurrentProduct);
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::TriggerQuery - Done Adding Product"), ELogVerbosity::Log);
	}
}

void UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead(bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& Error)
{
	FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead - Start"), ELogVerbosity::Log);
	bSavedWasSuccessful = false;
	if (bWasSuccessful)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead - WasSuccessful"), ELogVerbosity::Log);
		if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
		{
			IOnlineStoreV2Ptr StoreInterface = OnlineSub->GetStoreV2Interface();
			if (StoreInterface.IsValid())
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead - Interface is good"), ELogVerbosity::Log);
				TArray<FOnlineStoreOfferRef> LocalProductInformation;
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead - Getting Cached Offers"), ELogVerbosity::Log);
				StoreInterface->GetOffers(LocalProductInformation);
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead - Converting to proxy store offer"), ELogVerbosity::Log);
				CreateProxyProductInformation(LocalProductInformation, SavedProductInformation);
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead - Done"), ELogVerbosity::Log);
				bSavedWasSuccessful = true;
			}
		}
	}

	if (UWorld* World = WorldPtr.Get())
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.DelayInAppPurchaseRead"), STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseRead, STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateLambda([this](){

				OnInAppPurchaseRead_Delayed();

			}),
			GET_STATID(STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseRead), 
			nullptr, 
			ENamedThreads::GameThread
		);
	}
}

void UInAppPurchaseQueryCallbackProxy2::OnInAppPurchaseRead_Delayed()
{
	if (bSavedWasSuccessful)
	{
		OnSuccess.Broadcast(SavedProductInformation);
	}
	else
	{
		OnFailure.Broadcast(SavedProductInformation);
	}
}

void UInAppPurchaseQueryCallbackProxy2::BeginDestroy()
{
	Super::BeginDestroy();
}

UInAppPurchaseQueryCallbackProxy2* UInAppPurchaseQueryCallbackProxy2::CreateProxyObjectForInAppPurchaseQuery(class APlayerController* PlayerController, const TArray<FString>& ProductIdentifiers)
{
	UInAppPurchaseQueryCallbackProxy2* Proxy = NewObject<UInAppPurchaseQueryCallbackProxy2>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerQuery(PlayerController, ProductIdentifiers);
	return Proxy;
}

