// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseReceiptsCallbackProxy.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/PlayerController.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "Misc/ScopeExit.h"
#include "OnlineError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseReceiptsCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UInAppPurchaseCallbackProxy

UInAppPurchaseReceiptsCallbackProxy::UInAppPurchaseReceiptsCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInAppPurchaseReceiptsCallbackProxy::ClearPreviousResult()
{
	bWasSuccessful = false;
	SavedPurchaseStatus = EInAppPurchaseStatus::Failed;
	SavedReceipts.Empty();
	WorldPtr = nullptr;
}

void UInAppPurchaseReceiptsCallbackProxy::TriggerGetKnownReceipts(APlayerController* PlayerController)
{
	ClearPreviousResult();

	if (PlayerController == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		return;
	}

	WorldPtr = PlayerController->GetWorld();
	APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState);

	if (PlayerState == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		return;
	}

	IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr;
	if (OnlineSub == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
		return;
	}

	IOnlinePurchasePtr PurchaseInterface = OnlineSub->GetPurchaseInterface();
	if (!PurchaseInterface.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
		return;
	}

	if (FUniqueNetIdRepl PurchasingPlayer = PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId(); PurchasingPlayer.IsValid())
	{
		OnQueryReceiptsComplete(FOnlineError(true), PurchasingPlayer->AsShared());
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player UniqueNetId"), ELogVerbosity::Warning);
	}
}

void UInAppPurchaseReceiptsCallbackProxy::TriggerGetOwnedProducts(APlayerController* PlayerController, bool bRestorePurchases)
{
	ClearPreviousResult();

	if (PlayerController == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid  player controller"), ELogVerbosity::Warning);
		return;
	}

	WorldPtr = PlayerController->GetWorld();
	bool bFailedToEvenSubmit = true;
	ON_SCOPE_EXIT
	{
		if (bFailedToEvenSubmit)
		{
			OnQueryComplete();
		}
	};

	APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState);

	if (PlayerState == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player state"), ELogVerbosity::Warning);
		return;
	}

	IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr;
	if (OnlineSub == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
		return;
	}

	IOnlinePurchasePtr PurchaseInterface = OnlineSub->GetPurchaseInterface();
	if (!PurchaseInterface.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
		return;
	}
	if (FUniqueNetIdRepl PurchasingPlayer = PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId(); PurchasingPlayer.IsValid())
	{
		bFailedToEvenSubmit = false;
		PurchaseInterface->QueryReceipts(*PurchasingPlayer, bRestorePurchases, FOnQueryReceiptsComplete::CreateUObject(this, &UInAppPurchaseReceiptsCallbackProxy::OnQueryReceiptsComplete, FUniqueNetIdWeakPtr(PurchasingPlayer->AsShared())));
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid player UniqueNetId"), ELogVerbosity::Warning);
	}
}

void UInAppPurchaseReceiptsCallbackProxy::OnQueryComplete()
{
	if (UWorld* World = WorldPtr.Get())
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.DelayInAppQueryReceiptsComplete"), STAT_FSimpleDelegateGraphTask_DelayInAppQueryReceiptsComplete, STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateLambda([this]() {
				if (bWasSuccessful)
				{
					OnSuccess.Broadcast(SavedPurchaseStatus, SavedReceipts);
				}
				else
				{
					OnFailure.Broadcast(SavedPurchaseStatus, SavedReceipts);
				}
			}),
			GET_STATID(STAT_FSimpleDelegateGraphTask_DelayInAppQueryReceiptsComplete),
			nullptr,
			ENamedThreads::GameThread
			);
	}
}

void UInAppPurchaseReceiptsCallbackProxy::OnQueryReceiptsComplete(const FOnlineError& Result, FUniqueNetIdWeakPtr PurchasingPlayerWeak)
{
	bWasSuccessful = Result.WasSuccessful();
	SavedPurchaseStatus = PurchaseStatusFromOnlineError(Result);

	if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
	{
		IOnlinePurchasePtr PurchaseInterface = OnlineSub->GetPurchaseInterface();
		if (PurchaseInterface.IsValid())
		{
			if (FUniqueNetIdPtr PurchasingPlayer = PurchasingPlayerWeak.Pin())
			{
				TArray<FPurchaseReceipt> Receipts;
				PurchaseInterface->GetReceipts(*PurchasingPlayer, Receipts);

				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnQueryReceiptsComplete - Storing Receipts"), ELogVerbosity::Log);

				for (const auto& Receipt : Receipts)
				{
					FInAppPurchaseReceiptInfo2& ReceiptInfo = SavedReceipts.AddDefaulted_GetRef();

					ReceiptInfo.ItemId = Receipt.ReceiptOffers[0].LineItems[0].UniqueId;
					ReceiptInfo.ItemName = Receipt.ReceiptOffers[0].LineItems[0].ItemName;
					ReceiptInfo.ValidationInfo = Receipt.ReceiptOffers[0].LineItems[0].ValidationInfo;
					ReceiptInfo.TransactionIdentifier = Receipt.TransactionId;
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::OnQueryReceiptsComplete - Unknown PurchaseingPlayer"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseCallbackProxy::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
	}

	OnQueryComplete();
}

UInAppPurchaseReceiptsCallbackProxy* UInAppPurchaseReceiptsCallbackProxy::CreateProxyObjectForInAppPurchaseGetKnownReceipts(class APlayerController* PlayerController)
{
	UInAppPurchaseReceiptsCallbackProxy* Proxy = NewObject<UInAppPurchaseReceiptsCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerGetKnownReceipts(PlayerController);
	return Proxy;
}

UInAppPurchaseReceiptsCallbackProxy* UInAppPurchaseReceiptsCallbackProxy::CreateProxyObjectForInAppPurchaseQueryOwnedProducts(class APlayerController* PlayerController)
{
	UInAppPurchaseReceiptsCallbackProxy* Proxy = NewObject<UInAppPurchaseReceiptsCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerGetOwnedProducts(PlayerController, false);
	return Proxy;
}

UInAppPurchaseReceiptsCallbackProxy* UInAppPurchaseReceiptsCallbackProxy::CreateProxyObjectForInAppPurchaseRestoreOwnedProducts(class APlayerController* PlayerController)
{
	UInAppPurchaseReceiptsCallbackProxy* Proxy = NewObject<UInAppPurchaseReceiptsCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerGetOwnedProducts(PlayerController, true);
	return Proxy;
}
