// Copyright Epic Games, Inc. All Rights Reserved.

#include "InAppPurchaseRestoreCallbackProxy2.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/PlayerController.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "InAppPurchaseCallbackProxy2.h"
#include "OnlineError.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InAppPurchaseRestoreCallbackProxy2)

//////////////////////////////////////////////////////////////////////////
// UInAppPurchaseRestoreCallbackProxy

UInAppPurchaseRestoreCallbackProxy2::UInAppPurchaseRestoreCallbackProxy2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WorldPtr = nullptr;
}


void UInAppPurchaseRestoreCallbackProxy2::Trigger(const TArray<FInAppPurchaseProductRequest2>& ConsumableProductFlags, APlayerController* PlayerController)
{
	bWasSuccessful = false;
	bool bFailedToEvenSubmit = true;

	WorldPtr = nullptr;
	if (PlayerController)
	{
		WorldPtr = PlayerController->GetWorld();
		if (APlayerState* PlayerState = ToRawPtr(PlayerController->PlayerState))
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				PurchaseInterface = OnlineSub->GetPurchaseInterface();
				if (PurchaseInterface.IsValid())
				{
					bFailedToEvenSubmit = false;

					check(PlayerController);
					PurchasingPlayer = (*PlayerController->GetLocalPlayer()->GetUniqueNetIdFromCachedControllerId()).AsShared();
					PurchaseInterface->QueryReceipts(*PurchasingPlayer, true, FOnQueryReceiptsComplete::CreateUObject(this, &UInAppPurchaseRestoreCallbackProxy2::OnQueryReceiptsComplete));
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseRestoreCallbackProxy2::Trigger - In-App Purchases are not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseRestoreCallbackProxy2::Trigger - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseRestoreCallbackProxy2::Trigger - Invalid player state"), ELogVerbosity::Warning);
		}

		if (bFailedToEvenSubmit)
		{
			bWasSuccessful = false;
			OnInAppPurchaseRestoreComplete();
		}
	}
}

void UInAppPurchaseRestoreCallbackProxy2::OnQueryReceiptsComplete(const FOnlineError& Result)
{
	
	bWasSuccessful = Result.WasSuccessful();
	SavedPurchaseStatus = ::PurchaseStatusFromOnlineError(Result);
	check(PurchaseInterface.IsValid());

	// Get Receipts
	TArray<FPurchaseReceipt> OutReceipts;
	PurchaseInterface->GetReceipts(*PurchasingPlayer, OutReceipts);
	for (FPurchaseReceipt& CurrentReceipt : OutReceipts)
	{
		if (CurrentReceipt.TransactionState == EPurchaseTransactionState::Restored)
		{
			SavedPurchaseStatus = EInAppPurchaseStatus::Restored;
		}
	}

	SavedReceipts.Empty();

	FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseRestoreCallbackProxy2::OnQueryReceiptsComplete - Iterating OutReceipts"), ELogVerbosity::Log);
	for (FPurchaseReceipt& CurrentReceipt : OutReceipts)
	{
		if (CurrentReceipt.TransactionState == EPurchaseTransactionState::Restored)
		{
			FInAppPurchaseRestoreInfo2& ReceiptInfo = SavedReceipts.AddDefaulted_GetRef();

			FFrame::KismetExecutionMessage(TEXT("UInAppPurchaseRestoreCallbackProxy2::OnQueryReceiptsComplete - Writing Receipt to savedreceipts"), ELogVerbosity::Log);
			ReceiptInfo.ItemId = CurrentReceipt.ReceiptOffers[0].LineItems[0].UniqueId;
			ReceiptInfo.ItemName = CurrentReceipt.ReceiptOffers[0].LineItems[0].ItemName;
			ReceiptInfo.ValidationInfo = CurrentReceipt.ReceiptOffers[0].LineItems[0].ValidationInfo;
		}
	}

	OnInAppPurchaseRestoreComplete();
}

void UInAppPurchaseRestoreCallbackProxy2::OnInAppPurchaseRestoreComplete()
{
	if (UWorld* World = WorldPtr.Get())
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.DelayInAppPurchaseRestoreComplete"), STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseRestoreComplete, STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateLambda([this](){

				OnInAppPurchaseRestoreComplete_Delayed();

			}),
			GET_STATID(STAT_FSimpleDelegateGraphTask_DelayInAppPurchaseRestoreComplete), 
			nullptr, 
			ENamedThreads::GameThread
		);
    }
}


void UInAppPurchaseRestoreCallbackProxy2::OnInAppPurchaseRestoreComplete_Delayed()
{    
	if (bWasSuccessful)
	{
		OnSuccess.Broadcast(SavedPurchaseStatus, SavedReceipts);
	}
	else
	{
		OnFailure.Broadcast(SavedPurchaseStatus, SavedReceipts);
	}
}

void UInAppPurchaseRestoreCallbackProxy2::BeginDestroy()
{
	Super::BeginDestroy();
}


UInAppPurchaseRestoreCallbackProxy2* UInAppPurchaseRestoreCallbackProxy2::CreateProxyObjectForInAppPurchaseRestore(const TArray<FInAppPurchaseProductRequest2>& ConsumableProductFlags, class APlayerController* PlayerController)
{
	UInAppPurchaseRestoreCallbackProxy2* Proxy = NewObject<UInAppPurchaseRestoreCallbackProxy2>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->Trigger(ConsumableProductFlags, PlayerController);
	return Proxy;
}