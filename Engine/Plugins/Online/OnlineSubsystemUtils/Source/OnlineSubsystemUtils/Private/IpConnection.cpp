// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IpConnection.cpp: Unreal IP network connection.
Notes:
	* See \msdev\vc98\include\winsock.h and \msdev\vc98\include\winsock2.h 
	  for Winsock WSAE* errors returned by Windows Sockets.
=============================================================================*/

#include "IpConnection.h"
#include "Async/TaskGraphInterfaces.h"
#include "IpNetDriver.h"
#include "Engine/Engine.h"
#include "AutoRTFM.h"

#include "Sockets.h"
#include "Net/NetworkProfiler.h"
#include "Net/DataChannel.h"

#include "Net/Core/Misc/PacketAudit.h"
#include "NetAddressResolution.h"
#include "PacketHandler.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IpConnection)


/*-----------------------------------------------------------------------------
	Declarations.
-----------------------------------------------------------------------------*/

// Size of a UDP header.
#define IP_HEADER_SIZE     (20)
#define UDP_HEADER_SIZE    (IP_HEADER_SIZE+8)

DECLARE_CYCLE_STAT(TEXT("IpConnection InitRemoteConnection"), Stat_IpConnectionInitRemoteConnection, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("IpConnection Socket SendTo"), STAT_IpConnection_SendToSocket, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("IpConnection WaitForSendTasks"), STAT_IpConnection_WaitForSendTasks, STATGROUP_Net);

TAutoConsoleVariable<int32> CVarNetIpConnectionUseSendTasks(
	TEXT("net.IpConnectionUseSendTasks"),
	0,
	TEXT("If true, the IpConnection will call the socket's SendTo function in a task graph task so that it can run off the game thread."));

TAutoConsoleVariable<int32> CVarNetIpConnectionDisableResolution(
	TEXT("net.IpConnectionDisableResolution"),
	0,
	TEXT("If enabled, any future ip connections will not use resolution methods."),
	ECVF_Default | ECVF_Cheat);


namespace UE::Net::Private
{
#if !UE_BUILD_SHIPPING
	static int32 GNetDebugInitialConnect = 1;

	static FAutoConsoleVariableRef CVarNetDebugInitialConnect(
		TEXT("net.DebugInitialConnect"),
		GNetDebugInitialConnect,
		TEXT("When enabled, periodically logs socket-level send stats clientside, until a packet is successfully received to verify connection."));

	static float GNetDebugInitialConnectLogFrequency = 10.0;

	static FAutoConsoleVariableRef CVarNetDebugInitialConnectLogFrequency(
		TEXT("net.DebugInitialConnectLogFrequency"),
		GNetDebugInitialConnectLogFrequency,
		TEXT("The amount of time, in seconds, between initial connect debug logging."));

	static int32 GNetBlockSend = 0;

	static FAutoConsoleVariableRef CVarNetBlockSend(
		TEXT("net.BlockSend"),
		GNetBlockSend,
		TEXT("When enabled, blocks packet sends on NetConnection's."));
#endif

	static int32 GNetRecreateSocketCooldown = 10;
	static float GNetRecreateSocketTimeoutThreshold = 0.f;

	static FAutoConsoleVariableRef CVarNetRecreateSocketCooldown(
		TEXT("net.RecreateSocketCooldown"),
		GNetRecreateSocketCooldown,
		TEXT("The minimum amount of time, in seconds, between socket recreation attempts."));

	static FAutoConsoleVariableRef CVarNetRecreateSocketTimeoutThreshold(
		TEXT("net.RecreateSocketTimeoutThreshold"),
		GNetRecreateSocketTimeoutThreshold,
		TEXT("The amount of time, in seconds, without receiving a packet or alternatively without a send ack, before triggering socket recreation. ")
		TEXT("(0.0 = off)"));

	// Relates to the warning message which fires when trying to run HandlerComponents in parallel that do not support parallel execution
	// This latch makes sure we only first the warning once across all threads
	static std::atomic<bool> bHasLoggedHandlerParallelSupportWarning = false;
}


/**
 * UIpConnection
 */

UIpConnection::UIpConnection(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	SocketErrorDisconnectDelay(5.f),
	SocketError_SendDelayStartTime(0.f),
	SocketError_RecvDelayStartTime(0.f),
	Resolver(MakePimpl<UE::Net::Private::FNetConnectionAddressResolution>())
{
	// Auto add address resolution disable flags if the cvar is set.
	if (!!CVarNetIpConnectionDisableResolution.GetValueOnAnyThread())
	{
		DisableAddressResolution();
	}
}

void UIpConnection::InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	// Pass the call up the chain
	Super::InitBase(InDriver, InSocket, InURL, InState, 
		// Use the default packet size/overhead unless overridden by a child class
		(InMaxPacket == 0 || InMaxPacket > MAX_PACKET_SIZE) ? MAX_PACKET_SIZE : InMaxPacket,
		InPacketOverhead == 0 ? UDP_HEADER_SIZE : InPacketOverhead);

	if (!Resolver->IsAddressResolutionEnabled())
	{
		SocketPrivate = InSocket->AsShared();
	}

	if (CVarNetEnableCongestionControl.GetValueOnAnyThread() > 0)
	{
		NetworkCongestionControl.Emplace(CurrentNetSpeed, uint32(FNetPacketNotify::SequenceHistoryT::Size));
	}
}

void UIpConnection::InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	InitBase(InDriver, InSocket, InURL, InState, 
		// Use the default packet size/overhead unless overridden by a child class
		(InMaxPacket == 0 || InMaxPacket > MAX_PACKET_SIZE) ? MAX_PACKET_SIZE : InMaxPacket,
		InPacketOverhead == 0 ? UDP_HEADER_SIZE : InPacketOverhead);


	bool bResolverInit = Resolver->InitLocalConnection(InDriver->GetSocketSubsystem(), InSocket, InURL);

	if (!bResolverInit)
	{
		Close();

		return;
	}

	RemoteAddr = Resolver->GetRemoteAddr();

	// Initialize our send bunch
	InitSendBuffer();
}

void UIpConnection::InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	SCOPE_CYCLE_COUNTER(Stat_IpConnectionInitRemoteConnection);

	// Listeners don't need to perform address resolution
	Resolver->DisableAddressResolution();

	InitBase(InDriver, InSocket, InURL, InState, 
		// Use the default packet size/overhead unless overridden by a child class
		(InMaxPacket == 0 || InMaxPacket > MAX_PACKET_SIZE) ? MAX_PACKET_SIZE : InMaxPacket,
		InPacketOverhead == 0 ? UDP_HEADER_SIZE : InPacketOverhead);

	RemoteAddr = InRemoteAddr.Clone();
	URL.Host = RemoteAddr->ToString(false);

	// Initialize our send bunch
	InitSendBuffer();

	// This is for a client that needs to log in, setup ClientLoginState and ExpectedClientLoginMsgType to reflect that
	SetClientLoginState( EClientLoginState::LoggingIn );
	SetExpectedClientLoginMsgType( NMT_Hello );
}

void UIpConnection::Tick(float DeltaSeconds)
{
	using namespace UE::Net::Private;

	if (CVarNetIpConnectionUseSendTasks.GetValueOnAnyThread() != 0)
	{
		ISocketSubsystem* const SocketSubsystem = Driver->GetSocketSubsystem();
		TArray<FSocketSendResult> ResultsCopy;

		{
			FScopeLock ScopeLock(&SocketSendResultsCriticalSection);

			if (SocketSendResults.Num())
			{
				ResultsCopy = MoveTemp(SocketSendResults);
			}
		}
		
		for (const FSocketSendResult& Result : ResultsCopy)
		{
			HandleSocketSendResult(Result, SocketSubsystem);
		}
	}


	if (!TryConsumeAddressResolution())
	{
		// TryConsumeAddressResolution only advances the state machine from TryNextAddress
		// (via HasPendingAddressResults guard), never from Connected/Error (safe to call again).
		ECheckAddressResolutionResult CheckResult = Resolver->CheckAddressResolution();
		if (CheckResult == ECheckAddressResolutionResult::Connected)
		{
			UIpNetDriver* IpDriver = Cast<UIpNetDriver>(Driver);

			// Set the right object now that we have a connection
			IpDriver->SetSocketAndLocalAddress(Resolver->GetResolutionSocket());
		}
		else if (CheckResult == ECheckAddressResolutionResult::Error)
		{
			// Host name resolution just now failed.
			SetConnectionState(USOCK_Closed);
			Close(ENetCloseResult::AddressResolutionFailed);
		}
	}

	// Deferred cleanup of old sockets also waits until packets are received on the new socket (which happens after restart handshake process),
	// in order to avoid getting kicked by the server, due to closing the old socket too early and triggering ICMP unreachable errors
	if (DeferredCleanupSockets.Num() > 0 && ((LastReceiveRealtime - DeferredCleanupTimeCheck) >= 1.0) &&
		DeferredCleanupSockets.Num() == DeferredCleanupReadyCount)
	{
		DeferredCleanupReadyCount = 0;

		DeferredCleanupSockets.Reset();
	}

	Super::Tick(DeltaSeconds);

#if !UE_BUILD_SHIPPING
	if (!!GNetDebugInitialConnect && InTotalPackets == 0 && Driver != nullptr && Driver->ServerConnection != nullptr)
	{
		const double CurTime = Driver->GetElapsedTime();
		const double ElapsedLogTime = CurTime - InitialConnectLastLogTime;

		if (ElapsedLogTime >= static_cast<double>(GNetDebugInitialConnectLogFrequency))
		{
			const int32 SendCount = InitialConnectSocketSendCount - InitialConnectLastLogSocketSendCount;

			UE_LOGF(LogNet, Log, "Initial Connect Diagnostics: Sent '%i' packets in last '%f' seconds, no packets received yet.",
					SendCount, ElapsedLogTime);

			InitialConnectLastLogSocketSendCount = InitialConnectSocketSendCount;
			InitialConnectLastLogTime = CurTime;
		}
	}
#endif

	if (UIpNetDriver* IpDriver = Cast<UIpNetDriver>(Driver))
	{
		if (GNetRecreateSocketTimeoutThreshold != 0.f && GetConnectionState() != EConnectionState::USOCK_Closed && !IsReplay() &&
			!IpDriver->IsServer())
		{
			const double ElapsedTime = Driver->GetElapsedTime();
			const double LastRecreateSocketTime = IpDriver->GetLastRecreateSocketTime();
			const double InitialConnectMultiplier = (ElapsedTime < Driver->InitialConnectTimeout && LastRecreateSocketTime == 0.0) ? 2.0 : 1.0;
			const double RecreateCooldown = static_cast<double>(GNetRecreateSocketCooldown) * InitialConnectMultiplier;
			const double CooldownCompareTimeDiff = ElapsedTime - FMath::Max(GetConnectTime(), LastRecreateSocketTime);
			const double ThresholdCompareTimeDiff = ElapsedTime - FMath::Max(LastReceiveTime, GetLastRecvAckTime());

			if (CooldownCompareTimeDiff > RecreateCooldown && ThresholdCompareTimeDiff > GNetRecreateSocketTimeoutThreshold)
			{
				UE::Net::ERecreateSocketResult Result = IpDriver->RecreateSocket();

				UE_LOGF(LogNet, Warning,
						"Passed socket recreate threshold. LastReceiveTimeDiff: %f, LastRecvAckTimeDiff: %f, Threshold: %f, Result: %ls",
						(ElapsedTime - LastReceiveTime), (ElapsedTime - GetLastRecvAckTime()), GNetRecreateSocketTimeoutThreshold,
						ToCStr(UE::Net::LexToString(Result)));
			}
		}
	}
}

void UIpConnection::CleanUp()
{
	using namespace UE::Net::Private;

	Super::CleanUp();

	WaitForSendTasks();

	// Sockets must be cleaned up after send tasks complete, as they can still be in use
	Resolver->CleanupResolutionSockets();

	DeferredCleanupReadyCount = 0;

	DeferredCleanupSockets.Empty();
	SocketPrivate.Reset();
}

bool UIpConnection::TryConsumeAddressResolution(const TCHAR* LogSuffix)
{
	using namespace UE::Net::Private;

	// Only call CheckAddressResolution() when we know there are pending results (TryNextAddress state).
	// It returns the current ResolutionState and transitions to the next, so calling it here would
	// consume Error/Connected before the caller can observe and handle them via their own call.
	if (!Resolver->HasPendingAddressResults())
	{
		return false;
	}

	ECheckAddressResolutionResult CheckResult = Resolver->CheckAddressResolution();

	if (CheckResult == ECheckAddressResolutionResult::TryFirstAddress || CheckResult == ECheckAddressResolutionResult::TryNextAddress)
	{
		SetSocket_Local(Resolver->GetResolutionSocket());
		RemoteAddr = Resolver->GetRemoteAddr();

		UE_LOGF(LogNet, Log, "Address resolution complete, connection socket assigned (resolved address: %ls)%ls",
			*LowLevelGetRemoteAddress(true), LogSuffix);

		// Reset any timers
		LastReceiveTime = Driver->GetElapsedTime();
		LastReceiveRealtime = FPlatformTime::Seconds();
		LastGoodPacketRealtime = FPlatformTime::Seconds();

		// Reinit the buffer
		InitSendBuffer();

		// Resend initial packets again (only need to do this if this connection does not have packethandlers)
		// Otherwise most packethandlers will do retry (the stateless handshake being one example)
		if (CheckResult == ECheckAddressResolutionResult::TryNextAddress && !Handler.IsValid())
		{
			FWorldContext* WorldContext = GEngine->GetWorldContextFromPendingNetGameNetDriver(Driver);
			if (WorldContext != nullptr && WorldContext->PendingNetGame != nullptr)
			{
				WorldContext->PendingNetGame->SendInitialJoin();
			}
		}

		return true;
	}

	return false;
}

void UIpConnection::SetSocket_Local(const TSharedPtr<FSocket>& InSocket)
{
	TSharedPtr<FSocket> ResolutionSocket = Resolver->GetResolutionSocket();

	if (ResolutionSocket.IsValid() && InSocket != ResolutionSocket)
	{
		SafeDeferredSocketCleanup(ResolutionSocket);
		Resolver->CleanupResolutionSockets();
	}

	if (SocketPrivate.IsValid() && InSocket != SocketPrivate && SocketPrivate != ResolutionSocket)
	{
		SafeDeferredSocketCleanup(SocketPrivate);
	}

	SocketPrivate = InSocket;
}

// This needs to be removed when the main multithreading changes are added (or implemented in line with them)
void UIpConnection::SafeDeferredSocketCleanup(const TSharedPtr<FSocket>& InSocket)
{
	if (!DeferredCleanupSockets.Contains(InSocket))
	{
		DeferredCleanupSockets.Add(InSocket);
		DeferredCleanupTimeCheck = LastReceiveRealtime;

		if (CVarNetIpConnectionUseSendTasks.GetValueOnGameThread() != 0 && LastSendTask.IsValid())
		{
			DECLARE_CYCLE_STAT(TEXT("IpConnection SocketCleanup task"), STAT_IpConnection_SocketCleanupTask, STATGROUP_TaskGraphTasks);

			FGraphEventArray Prerequisites;

			Prerequisites.Add(LastSendTask);

			LastSendTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this]()
				{
					this->DeferredCleanupReadyCount++;
				},
				GET_STATID(STAT_IpConnection_SocketCleanupTask), &Prerequisites);
		}
		else
		{
			DeferredCleanupReadyCount++;
		}
	}
}

void UIpConnection::HandleConnectionTimeout(const FString& ErrorStr)
{
	using namespace UE::Net::Private;

	const EEAddressResolutionHandleResult HandleResult = Resolver->NotifyTimeout();

	if (HandleResult == EEAddressResolutionHandleResult::CallerShouldHandle)
	{
		Super::HandleConnectionTimeout(ErrorStr);
	}
	else
	{
		bPendingDestroy = false;
	}
}

void UIpConnection::WaitForSendTasks()
{
	if (CVarNetIpConnectionUseSendTasks.GetValueOnGameThread() != 0 && LastSendTask.IsValid())
	{
		check(IsInGameThread());

		SCOPE_CYCLE_COUNTER(STAT_IpConnection_WaitForSendTasks);
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(LastSendTask, ENamedThreads::GameThread);

		DeferredCleanupSockets.Reset();
	}
}

void UIpConnection::LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	using namespace UE::Net::Private;

	uint8* DataToSend = static_cast<uint8*>(Data);

	// If the remote addr hasn't been set, we need to wait for it.
	if (!RemoteAddr.IsValid() || !RemoteAddr->IsValid())
	{
		return;
	}

#if !UE_BUILD_SHIPPING
	if (GNetBlockSend)
	{
		return;
	}
#endif

	if (Resolver->ShouldBlockSend())
	{
		UE_LOGF(LogNet, Verbose, "Skipping send task as we are waiting on the next resolution step");
		return;
	}

	if (AutoRTFM::IsTransactional() && Driver->ShouldDeferTransactionalSend())
	{
		// Inside an AutoRTFM transaction, we must postpone network sends until commit time, to account for the
		// possibility of aborting the transaction. (An abort will prevent any network traffic from occurring.)
		// We aren't allowed to send packets out of order, so we explicitly jump into the closed here; this ensures
		// that OnCommit takes effect for _all_ transactional packets, not just packets sent in the closed.
		const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
		{
			int32 CountBytes = FMath::DivideAndRoundUp(CountBits, 8);

			AutoRTFM::OnCommit([this, 
								DataCopy = TArray<uint8>(DataToSend, CountBytes),
								CountBits,
								TraitsCopy = Traits]() mutable
			{
				SendToRemote(DataCopy.GetData(), CountBits, TraitsCopy);
			});
		});

		check(Status == AutoRTFM::ETransactionStatus::Executing);
	}
	else
	{
		SendToRemote(DataToSend, CountBits, Traits);
	}
}

void UIpConnection::SendToRemote(uint8* DataToSend, int32 CountBits, FOutPacketTraits& Traits) {
	using namespace UE::Net::Private;

#if !UE_BUILD_SHIPPING
	TSharedPtr<FInternetAddr> OrigAddr;

	if (GCurrentDuplicateIP.IsValid() && RemoteAddr->CompareEndpoints(*GCurrentDuplicateIP))
	{
		OrigAddr = RemoteAddr;

		TSharedRef<FInternetAddr> NewAddr = OrigAddr->Clone();
		int32 NewPort = NewAddr->GetPort() - 9876;

		NewAddr->SetPort(NewPort >= 0 ? NewPort : (65536 + NewPort));

		RemoteAddr = NewAddr;
	}

	ON_SCOPE_EXIT
	{
		if (OrigAddr.IsValid())
		{
			RemoteAddr = OrigAddr;
		}
	};
#endif

	// Process any packet modifiers, such as compression and encryption.
	if (Handler.IsValid() && !Handler->GetRawSend())
	{
#if WITH_SERVER_CODE
		//Flag a warning if we are trying to run non-parallel Handlers in a ParallelPhase
		UE_AUTORTFM_OPEN 
		{
			const UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem();
			if(ReplicationSystem && ReplicationSystem->IsInParallelPhase())
			{
				bool bExpected = false;
				if(!Handler->CanRunHandlerComponentsOnParallelOutgoingConnections() && bHasLoggedHandlerParallelSupportWarning.compare_exchange_strong(bExpected, true, std::memory_order_relaxed))
				{
					UE_LOGF(LogNet, Warning, "UIpConnection is trying to run non-parallel-compatible HandlerComponents in parallel. These HandlerComponents will be updated in a sequential order instead of simultaneously:");
					Handler->LogHandlerComponentList();
				}
			}
		};
#endif // WITH_SERVER_CODE

		const ProcessedPacket ProcessedData = Handler->Outgoing(DataToSend, CountBits, Traits);

		if (!ProcessedData.bError)
		{
			DataToSend = ProcessedData.Data;
			CountBits = ProcessedData.CountBits;
		}
		else
		{
			CountBits = 0;
		}
	}

	bool bBlockSend = false;
	int32 CountBytes = FMath::DivideAndRoundUp(CountBits, 8);

#if !UE_BUILD_SHIPPING
	LowLevelSendDel.ExecuteIfBound(DataToSend, CountBytes, bBlockSend);
#endif

	if (!bBlockSend)
	{
		// Send to remote.
		FSocketSendResult SendResult;
		CLOCK_CYCLES(Driver->SendCycles);

		if (CountBytes > MaxPacket)
		{
			UE_LOGF(LogNet, Warning, "UIpConnection::LowLevelSend: CountBytes > MaxPacketSize! Count: %i, MaxPacket: %i %ls", CountBytes, MaxPacket, *Describe());
		}

		FPacketAudit::NotifyLowLevelSend(DataToSend, CountBytes, CountBits);

		if (CountBytes > 0)
		{
			const bool bNotifyOnSuccess = (SocketErrorDisconnectDelay > 0.f) && (SocketError_SendDelayStartTime != 0.f);
			FSocket* CurSocket = GetSocket();

			if (CVarNetIpConnectionUseSendTasks.GetValueOnAnyThread() != 0)
			{
				DECLARE_CYCLE_STAT(TEXT("IpConnection SendTo task"), STAT_IpConnection_SendToTask, STATGROUP_TaskGraphTasks);

				FGraphEventArray Prerequisites;
				if (LastSendTask.IsValid())
				{
					Prerequisites.Add(LastSendTask);
				}

				ISocketSubsystem* const SocketSubsystem = Driver->GetSocketSubsystem();
				
				LastSendTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this, Packet = TArray<uint8>(DataToSend, CountBytes), SocketSubsystem, bNotifyOnSuccess]
				{
					FSocket* CurSocket = GetSocket();

					if (CurSocket != nullptr)
					{
						bool bWasSendSuccessful = false;
						UIpConnection::FSocketSendResult Result;

						{
							SCOPE_CYCLE_COUNTER(STAT_IpConnection_SendToSocket);
							bWasSendSuccessful = CurSocket->SendTo(Packet.GetData(), Packet.Num(), Result.BytesSent, *RemoteAddr);
						}

#if !UE_BUILD_SHIPPING
						if (!!GNetDebugInitialConnect && InTotalPackets == 0 && bWasSendSuccessful)
						{
							InitialConnectSocketSendCount++;
						}
#endif

						if (!bWasSendSuccessful && SocketSubsystem)
						{
							Result.Error = SocketSubsystem->GetLastErrorCode();
						}

						if (!bWasSendSuccessful || (bNotifyOnSuccess && Result.Error == SE_NO_ERROR))
						{
							FScopeLock ScopeLock(&SocketSendResultsCriticalSection);
							SocketSendResults.Add(MoveTemp(Result));
						}
					}
				},
				GET_STATID(STAT_IpConnection_SendToTask), &Prerequisites);

				// Always flush this profiler data now. Technically this could be incorrect if the send in the task fails,
				// but this keeps the bookkeeping simpler for now.
				NETWORK_PROFILER(GNetworkProfiler.FlushOutgoingBunches(this));
				NETWORK_PROFILER(GNetworkProfiler.TrackSocketSendTo(CurSocket->GetDescription(), DataToSend, uint16(CountBytes), uint16(NumPacketIdBits), uint16(NumBunchBits), uint16(NumAckBits), uint16(NumPaddingBits), this));
			}
			else
			{
				bool bWasSendSuccessful = false;
				{
					SCOPE_CYCLE_COUNTER(STAT_IpConnection_SendToSocket);
					bWasSendSuccessful = CurSocket->SendTo(DataToSend, CountBytes, SendResult.BytesSent, *RemoteAddr);
				}

				if (bWasSendSuccessful)
				{
					UNCLOCK_CYCLES(Driver->SendCycles);
					NETWORK_PROFILER(GNetworkProfiler.FlushOutgoingBunches(this));
					NETWORK_PROFILER(GNetworkProfiler.TrackSocketSendTo(CurSocket->GetDescription(), DataToSend, uint16(SendResult.BytesSent), uint16(NumPacketIdBits), uint16(NumBunchBits), uint16(NumAckBits), uint16(NumPaddingBits), this));

					if (bNotifyOnSuccess)
					{
						HandleSocketSendResult(SendResult, nullptr);
					}

#if !UE_BUILD_SHIPPING
					if (!!GNetDebugInitialConnect && InTotalPackets == 0)
					{
						InitialConnectSocketSendCount++;
					}
#endif
				}
				else if (ISocketSubsystem* const SocketSubsystem = Driver->GetSocketSubsystem())
				{
					SendResult.Error = SocketSubsystem->GetLastErrorCode();

					HandleSocketSendResult(SendResult, SocketSubsystem);
				}
			}
		}
	}
}

void UIpConnection::ReceivedRawPacket(void* Data, int32 Count)
{
	UE_CLOGF(SocketError_RecvDelayStartTime > 0.0, LogNet, Log,
			"UIpConnection::ReceivedRawPacket: Recovered from socket errors. %ls Connection", ToCStr(Describe()));
	
	// We received data successfully, reset our error counters.
	SocketError_RecvDelayStartTime = 0.0;

	// Set that we've gotten packet from the server, this begins destruction of the other elements.
	if (Resolver->IsAddressResolutionEnabled() && !Resolver->IsAddressResolutionComplete())
	{
		// We only want to write this once, because we don't want to waste cycles trying to clean up nothing.
		Resolver->NotifyAddressResolutionConnected();
	}

	Super::ReceivedRawPacket(Data, Count);
}

float UIpConnection::GetTimeoutValue()
{
	// Include support for no-timeouts
#if !UE_BUILD_SHIPPING
	check(Driver);
	if (Driver->bNoTimeouts)
	{
		return UNetConnection::GetTimeoutValue();
	}
#endif

	if (Resolver->IsAddressResolutionConnecting())
	{
		UIpNetDriver* IpDriver = Cast<UIpNetDriver>(Driver);

		if (IpDriver == nullptr || IpDriver->GetResolutionTimeoutValue() == 0.0f)
		{
			return Driver->InitialConnectTimeout;
		}

		return IpDriver->GetResolutionTimeoutValue();
	}

	return UNetConnection::GetTimeoutValue();
}

void UIpConnection::HandleSocketSendResult(const FSocketSendResult& Result, ISocketSubsystem* SocketSubsystem)
{
	using namespace UE::Net::Private;

	if (Result.Error == SE_NO_ERROR)
	{
		UE_CLOGF(SocketError_SendDelayStartTime > 0.f, LogNet, Log, "UIpConnection::HandleSocketSendResult: Recovered from socket errors. %ls Connection", *Describe());

		// We sent data successfully, reset our error counters.
		SocketError_SendDelayStartTime = 0.0;
	}
	else if (Result.Error != SE_EWOULDBLOCK)
	{
		check(SocketSubsystem);

		if (SocketErrorDisconnectDelay > 0.f)
		{
			const double Time = Driver->GetElapsedTime();
			if (SocketError_SendDelayStartTime == 0.0)
			{
				UE_LOGF(LogNet, Log, "UIpConnection::HandleSocketSendResult: Socket->SendTo failed with error %i (%ls). %ls Connection beginning close timeout (Timeout = %f).",
					static_cast<int32>(Result.Error),
					SocketSubsystem->GetSocketError(Result.Error),
					*Describe(),
					SocketErrorDisconnectDelay);

				SocketError_SendDelayStartTime = Time;
				return;
			}
			else if ((Time - SocketError_SendDelayStartTime) < SocketErrorDisconnectDelay)
			{
				// Our delay hasn't elapsed yet. Just suppress further warnings until we either recover or are disconnected.
				return;
			}
		}

		// Broadcast the error only on the first occurrence
		if (!GetPendingCloseDueToSocketSendFailure())
		{
			const EEAddressResolutionHandleResult HandleResult = Resolver->NotifySendError();

			// Request the connection to be disconnected during next tick() since we got a critical socket failure, the actual disconnect is postponed 		
			// to avoid issues with the call Close() causing issues with reentrant code paths in DataChannel::SendBunch() and FlushNet()
			SetPendingCloseDueToSocketSendFailure();

			if (HandleResult == EEAddressResolutionHandleResult::CallerShouldHandle)
			{
				FString ErrorString = FString::Printf(TEXT("UIpNetConnection::HandleSocketSendResult: Socket->SendTo failed with error %i (%s). ")
														TEXT("%s Connection will be closed during next Tick()!"),
														static_cast<int32>(Result.Error), ToCStr(SocketSubsystem->GetSocketError(Result.Error)),
														ToCStr(Describe()));

				GEngine->BroadcastNetworkFailure(Driver->GetWorld(), Driver, ENetworkFailure::ConnectionLost, ErrorString);
			}
			else
			{
				SocketError_SendDelayStartTime = 0.f;
			}
		}
	}
}

void UIpConnection::HandleSocketRecvError(class UNetDriver* NetDriver, const FString& ErrorString)
{
	using namespace UE::Net::Private;

	check(NetDriver);

	if (SocketErrorDisconnectDelay > 0.f)
	{
		const double Time = NetDriver->GetElapsedTime();
		if (SocketError_RecvDelayStartTime == 0.0)
		{
			UE_LOGF(LogNet, Log, "%ls. %ls Connection beginning close timeout (Timeout = %f).",
				*ErrorString,
				*Describe(),
				SocketErrorDisconnectDelay);

			SocketError_RecvDelayStartTime = Time;
			return;
		}
		else if ((Time - SocketError_RecvDelayStartTime) < SocketErrorDisconnectDelay)
		{
			return;
		}
	}


	const EEAddressResolutionHandleResult HandleResult = Resolver->NotifyReceiveError();

	if (HandleResult == EEAddressResolutionHandleResult::CallerShouldHandle)
	{
		// For now, this is only called on clients when the ServerConnection fails.
		// Because of that, on failure we'll shut down the NetDriver.
		GEngine->BroadcastNetworkFailure(NetDriver->GetWorld(), NetDriver, ENetworkFailure::ConnectionLost, ErrorString);
		NetDriver->Shutdown();
	}
	else
	{
		SocketErrorDisconnectDelay = 0.0f;
	}
}

void UIpConnection::DisableAddressResolution()
{
	Resolver->DisableAddressResolution();
}

FString UIpConnection::LowLevelGetRemoteAddress(bool bAppendPort)
{
	return (RemoteAddr.IsValid()) ? RemoteAddr->ToString(bAppendPort) : TEXT("");
}

FString UIpConnection::LowLevelDescribe()
{
	TSharedRef<FInternetAddr> LocalAddr = Driver->GetSocketSubsystem()->CreateInternetAddr();

	if (FSocket* CurSocket = GetSocket())
	{
		CurSocket->GetAddress(*LocalAddr);
	}

	return FString::Printf
	(
		TEXT("url=%s remote=%s local=%s uniqueid=%s state: %s"),
		*URL.Host,
		(RemoteAddr.IsValid() ? *RemoteAddr->ToString(true) : TEXT("nullptr")),
		*LocalAddr->ToString(true),
		(PlayerId.IsValid() ? *PlayerId->ToDebugString() : TEXT("nullptr")),
		LexToString(GetConnectionState())
	);
}

