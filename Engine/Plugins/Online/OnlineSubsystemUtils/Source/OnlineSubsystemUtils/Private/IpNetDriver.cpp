// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IpNetDriver.cpp: Unreal IP network driver.
Notes:
	* See \msdev\vc98\include\winsock.h and \msdev\vc98\include\winsock2.h 
	  for Winsock WSAE* errors returned by Windows Sockets.
=============================================================================*/

#include "IpNetDriver.h"
#include "HAL/RunnableThread.h"
#include "Misc/CommandLine.h"
#include "Engine/World.h"
#include "UObject/Package.h"
#include "PacketHandlers/StatelessConnectHandlerComponent.h"
#include "IpConnection.h"

#include "Net/Core/Misc/PacketAudit.h"
#include "Misc/ScopeExit.h"

#include "Sockets.h"
#include "Serialization/ArchiveCountMem.h"
#include "NetAddressResolution.h"
#include <limits> // IWYU pragma: keep

#include UE_INLINE_GENERATED_CPP_BY_NAME(IpNetDriver)


/*-----------------------------------------------------------------------------
	Declarations.
-----------------------------------------------------------------------------*/

DECLARE_CYCLE_STAT(TEXT("IpNetDriver Add new connection"), Stat_IpNetDriverAddNewConnection, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("IpNetDriver Socket RecvFrom"), STAT_IpNetDriver_RecvFromSocket, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("IpNetDriver Destroy WaitForReceiveThread"), STAT_IpNetDriver_Destroy_WaitForReceiveThread, STATGROUP_Net);

UIpNetDriver::FOnNetworkProcessingCausingSlowFrame UIpNetDriver::OnNetworkProcessingCausingSlowFrame;

// Time before the alarm delegate is called (in seconds)
float GIpNetDriverMaxDesiredTimeSliceBeforeAlarmSecs = 1.0f;

FAutoConsoleVariableRef GIpNetDriverMaxDesiredTimeSliceBeforeAlarmSecsCVar(
	TEXT("n.IpNetDriverMaxFrameTimeBeforeAlert"),
	GIpNetDriverMaxDesiredTimeSliceBeforeAlarmSecs,
	TEXT("Time to spend processing networking data in a single frame before an alert is raised (in seconds)\n")
	TEXT("It may get called multiple times in a single frame if additional processing after a previous alert exceeds the threshold again\n")
	TEXT(" default: 1 s"));

// Time before the time taken in a single frame is printed out (in seconds)
float GIpNetDriverLongFramePrintoutThresholdSecs = 10.0f;

FAutoConsoleVariableRef GIpNetDriverLongFramePrintoutThresholdSecsCVar(
	TEXT("n.IpNetDriverMaxFrameTimeBeforeLogging"),
	GIpNetDriverLongFramePrintoutThresholdSecs,
	TEXT("Time to spend processing networking data in a single frame before an output log warning is printed (in seconds)\n")
	TEXT(" default: 10 s"));

TAutoConsoleVariable<int32> CVarNetIpNetDriverUseReceiveThread(
	TEXT("net.IpNetDriverUseReceiveThread"),
	0,
	TEXT("If true, the IpNetDriver will call the socket's RecvFrom function on a separate thread (not the game thread)"));

TAutoConsoleVariable<int32> CVarNetIpNetDriverReceiveThreadQueueMaxPackets(
	TEXT("net.IpNetDriverReceiveThreadQueueMaxPackets"),
	1024,
	TEXT("If net.IpNetDriverUseReceiveThread is true, the maximum number of packets that can be waiting in the queue. Additional packets received will be dropped."));

TAutoConsoleVariable<int32> CVarNetIpNetDriverReceiveThreadPollTimeMS(
	TEXT("net.IpNetDriverReceiveThreadPollTimeMS"),
	250,
	TEXT("If net.IpNetDriverUseReceiveThread is true, the number of milliseconds to use as the timeout value for FSocket::Wait on the receive thread. A negative value means to wait indefinitely (FSocket::Shutdown should cancel it though)."));

TAutoConsoleVariable<int32> CVarNetUseRecvMulti(
	TEXT("net.UseRecvMulti"),
	0,
	TEXT("If true, and if running on a Unix/Linux platform, multiple packets will be retrieved from the socket with one syscall, ")
		TEXT("improving performance and also allowing retrieval of timestamp information."));

TAutoConsoleVariable<int32> CVarRecvMultiCapacity(
	TEXT("net.RecvMultiCapacity"),
	2048,
	TEXT("When RecvMulti is enabled, this is the number of packets it is allocated to handle per call - ")
		TEXT("bigger is better (especially under a DDoS), but keep an eye on memory cost."));

TAutoConsoleVariable<int32> CVarNetUseRecvTimestamps(
	TEXT("net.UseRecvTimestamps"),
	0,
	TEXT("If true and if net.UseRecvMulti is also true, on a Unix/Linux platform, ")
		TEXT("the kernel timestamp will be retrieved for each packet received, providing more accurate ping calculations."));

TAutoConsoleVariable<float> CVarRcvThreadSleepTimeForWaitableErrorsInSeconds(
	TEXT("net.RcvThreadSleepTimeForWaitableErrorsInSeconds"),
	0.0f, // When > 0 => sleep. When == 0 => yield (if platform supports it). When < 0 => disabled
	TEXT("Time the receive thread will sleep when a waitable error is returned by a socket operation."));

TAutoConsoleVariable<int32> CVarRcvThreadShouldSleepForLongRecvErrors(
	TEXT("net.RcvThreadShouldSleepForLongRecvErrors"),
	0,
	TEXT("Whether or not the receive thread should sleep for RecvFrom errors which are expected to last a long time. ")
		TEXT("0 = don't sleep, 1 = sleep, 2 = exit receive thread.")
	);


#if !UE_BUILD_SHIPPING
TAutoConsoleVariable<int32> CVarNetDebugDualIPs(
	TEXT("net.DebugDualIPs"),
	0,
	TEXT("If true, will duplicate every packet received, and process with a new (deterministic) IP, ")
		TEXT("to emulate receiving client packets from dual IP's - which can happen under real-world network conditions")
		TEXT("(only supports a single client on the server)."));

TSharedPtr<FInternetAddr> GCurrentDuplicateIP;
#endif

namespace IPNetDriverInternal
{
	/** The maximum number of times to individually log a specific IP pre-connection, before aggregating further logs */
	static int32 MaxIPHitLogs = 4;

	static FAutoConsoleVariableRef CVarMaxIPHitLogs(
		TEXT("net.MaxIPHitLogs"),
		MaxIPHitLogs,
		TEXT("The maximum number of times to individually log a specific IP pre-connection, before aggregating further logs."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var)
		{
			// Safe to update static here, but not CVar system cached value
			MaxIPHitLogs = FMath::Clamp(MaxIPHitLogs, 0, std::numeric_limits<int32>::max());
		}));

	/** The maximum number of IP's to include in aggregated pre-connection logging, before such logging is disabled altogether */
	static int32 MaxAggregateIPLogs = 16;

	static FAutoConsoleVariableRef CVarMaxAggregateIPLogs(
		TEXT("net.MaxAggregateIPLogs"),
		MaxAggregateIPLogs,
		TEXT("The maximum number of IP's to include in aggregated pre-connection logging, before such logging is disabled altogether ")
		TEXT("(Min: 1, Max: 128)."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var)
		{
			// Safe to update static here, but not CVar system cached value
			MaxAggregateIPLogs = FMath::Clamp(MaxAggregateIPLogs, 1, 128);
		}));


	/** The maximum number of IP hashes to track, for pre-connection logging */
	constexpr int32 MaxIPHashes = 256;

	/** The base time interval within which pre-connection IP tracking is performed */
	constexpr double AggregateIPLogInterval = 15.0;

	/** The random time interval variance added to AggregateIPLogInterval */
	constexpr double AggregateIPLogIntervalVariance = 5.0;


	bool ShouldSleepOnWaitError(ESocketErrors SocketError)
	{
		return SocketError == ESocketErrors::SE_NO_ERROR || SocketError == ESocketErrors::SE_EWOULDBLOCK || SocketError == ESocketErrors::SE_TRY_AGAIN;
	}

	/**
	 * Receive errors which are expected to last a long time and risk spinning the receive thread, which may or may not be recoverable
	 */
	bool IsLongRecvError(ESocketErrors SocketError)
	{
		return SocketError == SE_ENETDOWN;
	}
}

namespace UE::Net
{
	const TCHAR* LexToString(ERecreateSocketResult Value)
	{
		switch (Value)
		{
		case ERecreateSocketResult::NoAction: return TEXT("NoAction");
		case ERecreateSocketResult::NotReady: return TEXT("NotReady");
		case ERecreateSocketResult::AlreadyInProgress: return TEXT("AlreadyInProgress");
		case ERecreateSocketResult::BeganRecreate: return TEXT("BeganRecreate");
		case ERecreateSocketResult::Error: return TEXT("Error");

		default:
			check(false);

			return TEXT("Unknown");
		}
	}
}

/**
 * FPacketItrator
 *
 * Encapsulates the NetDriver TickDispatch code required for executing all variations of packet receives
 * (FSocket::RecvFrom, FSocket::RecvMulti, and the Receive Thread),
 * as well as implementing/abstracting-away some of the outermost (non-NetConnection-related) parts of the DDoS detection code,
 * and code for timing receives/iterations (which affects control flow).
 */
class FPacketIterator
{
	friend class UIpNetDriver;
	
private:
	struct FCachedPacket
	{
		/** Whether socket receive succeeded. Don't rely on the Error field for this, due to implementation/platform uncertainties. */
		bool bRecvSuccess;

		/** Pre-allocated Data field, for storing packets of any expected size */
		TArray<uint8, TFixedAllocator<MAX_PACKET_SIZE>> Data;

		/** Receive address for the packet */
		TSharedPtr<FInternetAddr> Address;

		/** OS-level timestamp for the packet receive, if applicable */
		double PacketTimestamp;

		/** Error if receiving a packet failed */
		ESocketErrors Error;
	};


private:
	FPacketIterator(UIpNetDriver* InDriver)
		: FPacketIterator(InDriver, InDriver->RecvMultiState.Get(), FPlatformTime::Seconds(),
							(InDriver->MaxSecondsInReceive > 0.0 && InDriver->NbPacketsBetweenReceiveTimeTest > 0))
	{
	}

	FPacketIterator(UIpNetDriver* InDriver, FRecvMulti* InRMState, double InStartReceiveTime, bool bInCheckReceiveTime)
		: bBreak(false)
		, IterationCount(0)
		, Driver(InDriver)
		, DDoS(InDriver->DDoS)
		, SocketSubsystem(InDriver->GetSocketSubsystem())
		, SocketReceiveThreadRunnable(InDriver->SocketReceiveThreadRunnable.Get())
		, CurrentPacket()
#if !UE_BUILD_SHIPPING
		, bDebugDualIPs(CVarNetDebugDualIPs.GetValueOnAnyThread() != 0)
		, DuplicatePacket()
#endif
		, RMState(InRMState)
		, bUseRecvMulti(CVarNetUseRecvMulti.GetValueOnAnyThread() != 0 && InRMState != nullptr)
		, RecvMultiIdx(0)
		, RecvMultiPacketCount(0)
		, StartReceiveTime(InStartReceiveTime)
		, bCheckReceiveTime(bInCheckReceiveTime)
		, CheckReceiveTimePacketCount(bInCheckReceiveTime ? InDriver->NbPacketsBetweenReceiveTimeTest : 0)
		, NumIterationUntilTimeTest(CheckReceiveTimePacketCount)
		, BailOutTime(InStartReceiveTime + InDriver->MaxSecondsInReceive)
		, bSlowFrameChecks(UIpNetDriver::OnNetworkProcessingCausingSlowFrame.IsBound())
		, AlarmTime(InStartReceiveTime + GIpNetDriverMaxDesiredTimeSliceBeforeAlarmSecs)
	{
		if (!bUseRecvMulti && SocketSubsystem != nullptr)
		{
			CurrentPacket.Address = SocketSubsystem->CreateInternetAddr();
		}

#if !UE_BUILD_SHIPPING
		if (bDebugDualIPs && !bUseRecvMulti)
		{
			DuplicatePacket = MakeUnique<FCachedPacket>();
		}
#endif

		AdvanceCurrentPacket();
	}

	~FPacketIterator()
	{
		const float DeltaReceiveTime = float(FPlatformTime::Seconds() - StartReceiveTime);

		if (DeltaReceiveTime > GIpNetDriverLongFramePrintoutThresholdSecs)
		{
			UE_LOGF(LogNet, Warning, "Took too long to receive packets. Time: %2.2f %ls", DeltaReceiveTime, *Driver->GetName());
		}
	}

	FORCEINLINE FPacketIterator& operator++()
	{
		IterationCount++;
		AdvanceCurrentPacket();

		return *this;
	}

	FORCEINLINE explicit operator bool() const
	{
		return !bBreak;
	}


	/**
	 * Retrieves the packet information from the current iteration. Avoid calling more than once, per iteration.
	 *
	 * @param OutPacket		Outputs a view to the received packet data
	 * @return				Returns whether or not receiving was successful for the current packet
	 */
	bool GetCurrentPacket(FReceivedPacketView& OutPacket)
	{
		bool bRecvSuccess = false;

		if (bUseRecvMulti)
		{
			RMState->GetPacket(RecvMultiIdx, OutPacket);
			bRecvSuccess = true;
		}
		else
		{
			OutPacket.DataView = {CurrentPacket.Data.GetData(), CurrentPacket.Data.Num(), ECountUnits::Bytes};
			OutPacket.Error = CurrentPacket.Error;
			OutPacket.Address = CurrentPacket.Address;
			bRecvSuccess = CurrentPacket.bRecvSuccess;
		}

#if !UE_BUILD_SHIPPING
		if (IsDuplicatePacket() && OutPacket.Address.IsValid())
		{
			TSharedRef<FInternetAddr> NewAddr = OutPacket.Address->Clone();

			NewAddr->SetPort((NewAddr->GetPort() + 9876) & 0xFFFF);

			OutPacket.Address = NewAddr;
			GCurrentDuplicateIP = NewAddr;
		}
#endif

		return bRecvSuccess;
	}

	/**
	 * Retrieves the packet timestamp information from the current iteration. As above, avoid calling more than once.
	 *
	 * @param ForConnection		The connection we are retrieving timestamp information for
	 */
	void GetCurrentPacketTimestamp(UNetConnection* ForConnection)
	{
		FPacketTimestamp CurrentTimestamp;
		bool bIsLocalTimestamp = false;
		bool bSuccess = false;

		if (bUseRecvMulti)
		{
			RMState->GetPacketTimestamp(RecvMultiIdx, CurrentTimestamp);
			bIsLocalTimestamp = false;
			bSuccess = true;
		}
		else if (CurrentPacket.PacketTimestamp != 0.0)
		{
			CurrentTimestamp.Timestamp = FTimespan::FromSeconds(CurrentPacket.PacketTimestamp);
			bIsLocalTimestamp = true;
			bSuccess = true;
		}

		if (bSuccess)
		{
			ForConnection->SetPacketOSReceiveTime(CurrentTimestamp, bIsLocalTimestamp);
		}
	}

	/**
	 * Returns a view of the iterator's packet buffer, for updating packet data as it's processed, and generating new packet views
	 */
	FPacketBufferView GetWorkingBuffer()
	{
		return { CurrentPacket.Data.GetData(), MAX_PACKET_SIZE };
	}

	/**
	 * Advances the current packet to the next iteration
	 */
	void AdvanceCurrentPacket()
	{
		// @todo: Remove the slow frame checks, eventually - potential DDoS and platform constraint
		if (bSlowFrameChecks)
		{
			const double CurrentTime = FPlatformTime::Seconds();

			if (CurrentTime > AlarmTime)
			{
				Driver->OnNetworkProcessingCausingSlowFrame.Broadcast();

				AlarmTime = CurrentTime + GIpNetDriverMaxDesiredTimeSliceBeforeAlarmSecs;
			}
		}

		if (bCheckReceiveTime && IterationCount > 0)
		{
			--NumIterationUntilTimeTest;
			if (NumIterationUntilTimeTest <= 0)
			{
				// Restart the countdown until the next time check
				NumIterationUntilTimeTest = CheckReceiveTimePacketCount;

				const double CurrentTime = FPlatformTime::Seconds();

				if (CurrentTime > BailOutTime)
				{
					// NOTE: For RecvMulti, this will mass-dump packets, leading to packetloss. Avoid using with RecvMulti.
					bBreak = true;

					UE_LOGF(LogNet, Verbose, "Stopping packet reception after processing for more than %f seconds. %ls",
							Driver->MaxSecondsInReceive, *Driver->GetName());
				}
			}
		}

		if (!bBreak)
		{
#if !UE_BUILD_SHIPPING
			if (IsDuplicatePacket())
			{
				CurrentPacket = *DuplicatePacket;
			}
			else
#endif
			if (bUseRecvMulti)
			{
				if (RecvMultiPacketCount == 0 || ((RecvMultiIdx + 1) >= RecvMultiPacketCount))
				{
					AdvanceRecvMultiState();
				}
				else
				{
					RecvMultiIdx++;
				}

				// At this point, bBreak will be set, or RecvMultiPacketCount will be > 0
			}
			else
			{
				bBreak = !ReceiveSinglePacket();

#if !UE_BUILD_SHIPPING
				if (bDebugDualIPs && !bBreak)
				{
					(*DuplicatePacket) = CurrentPacket;
				}
#endif
			}
		}
	}

	/**
	 * Receives a single packet from the network socket, outputting to the CurrentPacket buffer.
	 *
	 * @return				Whether or not a packet or an error was successfully received
	 */
	bool ReceiveSinglePacket()
	{
		bool bReceivedPacketOrError = false;

		CurrentPacket.bRecvSuccess = false;
		CurrentPacket.Data.SetNumUninitialized(0, EAllowShrinking::No);

		if (CurrentPacket.Address.IsValid())
		{
			CurrentPacket.Address->SetAnyAddress();
		}

		CurrentPacket.PacketTimestamp = 0.0;
		CurrentPacket.Error = SE_NO_ERROR;

		while (true)
		{
			bReceivedPacketOrError = false;

			if (SocketReceiveThreadRunnable != nullptr)
			{
				// Very-early-out - the NetConnection per frame time limit, limits all packet processing
				// @todo #JohnB: This DDoS detection code will be redundant, as it's performed in the Receive Thread in a coming refactor
				if (DDoS.ShouldBlockNetConnPackets())
				{
					// Approximate due to threading
					uint32 DropCountApprox = SocketReceiveThreadRunnable->ReceiveQueue.Count();

					SocketReceiveThreadRunnable->ReceiveQueue.Empty();

					if (DropCountApprox > 0)
					{
						DDoS.IncDroppedPacketCounter(DropCountApprox);
					}
				}
				else
				{
					UIpNetDriver::FReceivedPacket IncomingPacket;
					const bool bHasPacket = SocketReceiveThreadRunnable->ReceiveQueue.Dequeue(IncomingPacket);

					if (bHasPacket)
					{
						if (IncomingPacket.FromAddress.IsValid())
						{
							CurrentPacket.Address = IncomingPacket.FromAddress.ToSharedRef();
						}

						ESocketErrors CurError = IncomingPacket.Error;
						bool bReceivedPacket = CurError == SE_NO_ERROR;

						CurrentPacket.bRecvSuccess = bReceivedPacket;
						CurrentPacket.PacketTimestamp = IncomingPacket.PlatformTimeSeconds;
						CurrentPacket.Error = CurError;
						bReceivedPacketOrError = bReceivedPacket;

						if (bReceivedPacket)
						{
							int32 BytesRead = IncomingPacket.PacketBytes.Num();

							if (IncomingPacket.PacketBytes.Num() <= MAX_PACKET_SIZE)
							{
								CurrentPacket.Data.SetNumUninitialized(BytesRead, EAllowShrinking::No);

								FMemory::Memcpy(CurrentPacket.Data.GetData(), IncomingPacket.PacketBytes.GetData(), BytesRead);
							}
							else
							{
								UE_LOGF(LogNet, Warning, "IpNetDriver receive thread received a packet of %d bytes, which is larger than the data buffer size of %d bytes.",
										BytesRead, MAX_PACKET_SIZE);

								continue;
							}
						}
						// Received an error
						else if (!UIpNetDriver::IsRecvFailBlocking(CurError))
						{
							bReceivedPacketOrError = true;
						}
					}
				}
			}
			else if (Driver->GetSocket() != nullptr && SocketSubsystem != nullptr)
			{
				SCOPE_CYCLE_COUNTER(STAT_IpNetDriver_RecvFromSocket);

				int32 BytesRead = 0;
				bool bReceivedPacket = Driver->GetSocket()->RecvFrom(CurrentPacket.Data.GetData(), MAX_PACKET_SIZE, BytesRead, *CurrentPacket.Address);

				CurrentPacket.bRecvSuccess = bReceivedPacket;
				bReceivedPacketOrError = bReceivedPacket;

				if (bReceivedPacket)
				{
					// Fixed allocator, so no risk of realloc from copy-then-resize
					CurrentPacket.Data.SetNumUninitialized(BytesRead, EAllowShrinking::No);
				}
				else
				{
					ESocketErrors CurError = SocketSubsystem->GetLastErrorCode();

					CurrentPacket.Error = CurError;
					CurrentPacket.Data.SetNumUninitialized(0, EAllowShrinking::No);

					// Received an error
					if (!UIpNetDriver::IsRecvFailBlocking(CurError))
					{
						bReceivedPacketOrError = true;
					}
				}

				// Very-early-out - the NetConnection per frame time limit, limits all packet processing
				if (bReceivedPacketOrError && DDoS.ShouldBlockNetConnPackets())
				{
					if (bReceivedPacket)
					{
						DDoS.IncDroppedPacketCounter();
					}

					continue;
				}
			}

			// While loop only exists to allow 'continue' for DDoS and invalid packet code, above
			break;
		}

		return bReceivedPacketOrError;
	}

	/**
	 * Load a fresh batch of RecvMulti packets
	 */
	void AdvanceRecvMultiState()
	{
		RecvMultiIdx = 0;
		RecvMultiPacketCount = 0;

		bBreak = Driver->GetSocket() == nullptr;

		while (!bBreak)
		{
			SCOPE_CYCLE_COUNTER(STAT_IpNetDriver_RecvFromSocket);

			if (Driver->GetSocket() == nullptr)
			{
				break;
			}

			bool bRecvMultiOk = Driver->GetSocket()->RecvMulti(*RMState);

			if (!bRecvMultiOk)
			{
				ESocketErrors RecvMultiError = (SocketSubsystem != nullptr ? SocketSubsystem->GetLastErrorCode() : SE_NO_ERROR);

				if (UIpNetDriver::IsRecvFailBlocking(RecvMultiError))
				{
					bBreak = true;
					break;
				}
				else
				{
					// When the Linux recvmmsg syscall encounters an error after successfully receiving at least one packet,
					// it won't return an error until called again, but this error can be overwritten before recvmmsg is called again.
					// This makes the error handling for recvmmsg unreliable. Continue until the socket blocks.

					// Continue is safe, as 0 packets have been received
					continue;
				}
			}

			// Extreme-early-out. NetConnection per frame time limit, limits all packet processing - RecvMulti drops all packets at once
			if (DDoS.ShouldBlockNetConnPackets())
			{
				int32 NumDropped = RMState->GetNumPackets();

				DDoS.IncDroppedPacketCounter(NumDropped);

				// Have a threshold, to stop the RecvMulti syscall spinning with low packet counts - let the socket buffer build up
				if (NumDropped > 10)
				{
					continue;
				}
				else
				{
					bBreak = true;
					break;
				}
			}

			RecvMultiPacketCount = RMState->GetNumPackets();

			break;
		}
	}

#if !UE_BUILD_SHIPPING
	/**
	 * Whether or not the current packet being iterated, is a duplicate of the previous packet
	 */
	FORCEINLINE bool IsDuplicatePacket() const
	{
		// When doing Dual IP debugging, every other packet is a duplicate of the previous packet
		return bDebugDualIPs && (IterationCount % 2) == 1;
	}
#endif


private:
	/** Specified internally, when the packet iterator should break/stop (no packets, DDoS limits triggered, etc.) */
	bool bBreak;

	/** The number of packets iterated thus far */
	int64 IterationCount;


	/** Cached reference to the NetDriver, and NetDriver variables/values */

	UIpNetDriver* const Driver;

	FDDoSDetection& DDoS;

	ISocketSubsystem* const SocketSubsystem;

	UIpNetDriver::FReceiveThreadRunnable* const SocketReceiveThreadRunnable;

	/** Stores information for the current packet being received (when using single-receive mode) */
	FCachedPacket CurrentPacket;

#if !UE_BUILD_SHIPPING
	/** Whether or not to enable Dual IP debugging */
	const bool bDebugDualIPs;

	/** When performing Dual IP tests, a duplicate copy of every packet, is stored here */
	TUniquePtr<FCachedPacket> DuplicatePacket;
#endif

	/** Stores information for receiving packets using RecvMulti */
	FRecvMulti* const RMState;

	/** Whether or not RecvMulti is enabled/supported */
	const bool bUseRecvMulti;

	/** The RecvMulti index of the next packet to be received (if RecvMultiPacketCount > 0) */
	int32 RecvMultiIdx;

	/** The number of packets waiting to be read from the FRecvMulti state */
	int32 RecvMultiPacketCount;

	/** The time at which packet iteration/receiving began */
	const double StartReceiveTime;

	/** Whether or not to perform receive time limit checks */
	const bool bCheckReceiveTime;

	/** Receive time is checked every 'x' number of packets */
	const int32 CheckReceiveTimePacketCount;

	/** The number of packets left to process until we check if we went over our time budget */
	int32 NumIterationUntilTimeTest;

	/** The time at which to bail out of the receive loop, if it's time limited */
	const double BailOutTime;

	/** Whether or not checks for slow frames are active */
	const bool bSlowFrameChecks;

	/** Cached time at which to trigger a slow frame alarm */
	double AlarmTime;
};

class FIpConnectionHelper
{
private:
	friend UIpNetDriver;

	static void HandleSocketRecvError(UIpNetDriver* Driver, UIpConnection* Connection, const FString& ErrorString)
	{
		Connection->HandleSocketRecvError(Driver, ErrorString);
	}

	static void SetSocket_Local(UIpConnection* Connection, const TSharedPtr<FSocket>& InSocket)
	{
		Connection->SetSocket_Local(InSocket);
	}
};


/**
 * UIpNetDriver
 */

UIpNetDriver::UIpNetDriver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PauseReceiveEnd(0.f)
	, ServerDesiredSocketReceiveBufferBytes(0x20000)
	, ServerDesiredSocketSendBufferBytes(0x20000)
	, ClientDesiredSocketReceiveBufferBytes(0x8000)
	, ClientDesiredSocketSendBufferBytes(0x8000)
	, RecvMultiState(nullptr)
	, Resolver(MakePimpl<UE::Net::Private::FNetDriverAddressResolution>())
{
	using namespace IPNetDriverInternal;

	NewIPHashes.Empty(MaxIPHashes);
	NewIPHitCount.Empty(MaxIPHashes);
	AggregatedIPsToLog.Empty(MaxAggregateIPLogs);
}

bool UIpNetDriver::IsAvailable() const
{
	// IP driver always valid for now
	return true;
}

ISocketSubsystem* UIpNetDriver::GetSocketSubsystem()
{
	return ISocketSubsystem::Get();
}

FUniqueSocket UIpNetDriver::CreateSocketForProtocol(const FName& ProtocolType)
{
	// Create UDP socket and enable broadcasting.
	ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();

	if (SocketSubsystem == NULL)
	{
		UE_LOGF(LogNet, Warning, "UIpNetDriver::CreateSocket: Unable to find socket subsystem");
		return NULL;
	}

	return SocketSubsystem->CreateUniqueSocket(NAME_DGram, TEXT("Unreal"), ProtocolType);
}

int UIpNetDriver::GetClientPort()
{
	return 0;
}

FUniqueSocket UIpNetDriver::CreateAndBindSocket(TSharedRef<FInternetAddr> BindAddr, int32 Port, bool bReuseAddressAndPort, int32 DesiredRecvSize, int32 DesiredSendSize, FString& Error)
{
	ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();
	if (SocketSubsystem == nullptr)
	{
		Error = TEXT("Unable to find socket subsystem");
		return nullptr;
	}

	// Create the socket that we will use to communicate with
	FUniqueSocket NewSocket = CreateSocketForProtocol(BindAddr->GetProtocolType());

	if (!NewSocket.IsValid())
	{
		Error = FString::Printf(TEXT("%s: socket failed (%i)"), SocketSubsystem->GetSocketAPIName(), (int32)SocketSubsystem->GetLastErrorCode());
		return nullptr;
	}

	/* Make sure to cleanly destroy any sockets we do not mean to use. */
	ON_SCOPE_EXIT
	{
		if (Error.IsEmpty() == false)
		{
			NewSocket.Reset();
		}
	};

	if (SocketSubsystem->RequiresChatDataBeSeparate() == false && NewSocket->SetBroadcast() == false)
	{
		Error = FString::Printf(TEXT("%s: setsockopt SO_BROADCAST failed (%i)"), SocketSubsystem->GetSocketAPIName(), (int32)SocketSubsystem->GetLastErrorCode());
		return nullptr;
	}

	if (NewSocket->SetReuseAddr(bReuseAddressAndPort) == false)
	{
		UE_LOGF(LogNet, Log, "setsockopt with SO_REUSEADDR failed");
	}

	if (NewSocket->SetRecvErr() == false)
	{
		UE_LOGF(LogNet, Log, "setsockopt with IP_RECVERR failed");
	}

	int32 ActualRecvSize(0);
	int32 ActualSendSize(0);
	NewSocket->SetReceiveBufferSize(DesiredRecvSize, ActualRecvSize);
	NewSocket->SetSendBufferSize(DesiredSendSize, ActualSendSize);
	UE_LOGF(LogInit, Log, "%ls: Socket queue. Rx: %i (config %i) Tx: %i (config %i)", SocketSubsystem->GetSocketAPIName(),
		ActualRecvSize, DesiredRecvSize, ActualSendSize, DesiredSendSize);

	// Bind socket to our port.
	BindAddr->SetPort(Port);

	int32 AttemptPort = BindAddr->GetPort();
	int32 BoundPort = SocketSubsystem->BindNextPort(NewSocket.Get(), *BindAddr, MaxPortCountToTry + 1, 1);
	if (BoundPort == 0)
	{
		Error = FString::Printf(TEXT("%s: binding to port %i failed (%i)"), SocketSubsystem->GetSocketAPIName(), AttemptPort,
			(int32)SocketSubsystem->GetLastErrorCode());

		if (bExitOnBindFailure)
		{
			UE_LOGF(LogNet, Fatal, "Fatal error: %ls", *Error);
		}

		return nullptr;
	}

	if (NewSocket->SetNonBlocking() == false)
	{
		Error = FString::Printf(TEXT("%s: SetNonBlocking failed (%i)"), SocketSubsystem->GetSocketAPIName(),
			(int32)SocketSubsystem->GetLastErrorCode());
		return nullptr;
	}

	return NewSocket;
}

void UIpNetDriver::SetSocketAndLocalAddress(const TSharedPtr<FSocket>& SharedSocket)
{
	// Must be called even if the current socket is already SharedSocket (for when Net Address Resolution resolves the current socket)
	SetSocket_Internal(SharedSocket);

	if (SocketPrivate.IsValid())
	{
		// Allocate any LocalAddrs if they haven't been allocated yet.
		if (!LocalAddr.IsValid())
		{
			LocalAddr = GetSocketSubsystem()->CreateInternetAddr();
		}

		SocketPrivate->GetAddress(*LocalAddr);
	}
}

void UIpNetDriver::ClearSockets()
{
	SocketPrivate.Reset();
	Resolver->ClearSockets();
}

bool UIpNetDriver::InitBase( bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error )
{
	using namespace UE::Net::Private;

	if (!Super::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		return false;
	}

	ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();
	if (SocketSubsystem == nullptr)
	{
		UE_LOGF(LogNet, Warning, "Unable to find socket subsystem");
		return false;
	}

	const int32 BindPort = bInitAsClient ? GetClientPort() : URL.Port;
	// Increase socket queue size, because we are polling rather than threading
	// and thus we rely on the OS socket to buffer a lot of data.
	const int32 DesiredRecvSize = bInitAsClient ? ClientDesiredSocketReceiveBufferBytes : ServerDesiredSocketReceiveBufferBytes;
	const int32 DesiredSendSize = bInitAsClient ? ClientDesiredSocketSendBufferBytes : ServerDesiredSocketSendBufferBytes;
	const EInitBindSocketsFlags InitBindFlags = bInitAsClient ? EInitBindSocketsFlags::Client : EInitBindSocketsFlags::Server;
	FCreateAndBindSocketFunc CreateAndBindSocketsFunc = [this, BindPort, bReuseAddressAndPort, DesiredRecvSize, DesiredSendSize]
									(TSharedRef<FInternetAddr> BindAddr, FString& Error) -> FUniqueSocket
		{
			return this->CreateAndBindSocket(BindAddr, BindPort, bReuseAddressAndPort, DesiredRecvSize, DesiredSendSize, Error);
		};

	bool bInitBindSocketsSuccess = Resolver->InitBindSockets(MoveTemp(CreateAndBindSocketsFunc), InitBindFlags, SocketSubsystem, Error);

	if (!bInitBindSocketsSuccess)
	{
		UE_LOGF(LogNet, Error, "InitBindSockets failed: %ls", ToCStr(Error));

		return false;
	}

	
	// If the cvar is set and the socket subsystem supports it, create the receive thread.
	if (CVarNetIpNetDriverUseReceiveThread.GetValueOnAnyThread() != 0 && SocketSubsystem->IsSocketWaitSupported())
	{
		SocketReceiveThreadRunnable = MakeUnique<FReceiveThreadRunnable>(this);
		SocketReceiveThread.Reset(FRunnableThread::Create(SocketReceiveThreadRunnable.Get(), *FString::Printf(TEXT("IpNetDriver Receive Thread: %s"), *NetDriverName.ToString())));
	}

	SetSocketAndLocalAddress(Resolver->GetFirstSocket());

	bool bRecvMultiEnabled = CVarNetUseRecvMulti.GetValueOnAnyThread() != 0;
	bool bRecvThreadEnabled = CVarNetIpNetDriverUseReceiveThread.GetValueOnAnyThread() != 0;

	if (bRecvMultiEnabled && !bRecvThreadEnabled)
	{
		bool bSupportsRecvMulti = SocketSubsystem->IsSocketRecvMultiSupported();

		if (bSupportsRecvMulti)
		{
			bool bRetrieveTimestamps = CVarNetUseRecvTimestamps.GetValueOnAnyThread() != 0;

			if (bRetrieveTimestamps)
			{
				Resolver->SetRetrieveTimestamp(true);
			}

			ERecvMultiFlags RecvMultiFlags = bRetrieveTimestamps ? ERecvMultiFlags::RetrieveTimestamps : ERecvMultiFlags::None;
			int32 MaxRecvMultiPackets = FMath::Max(32, CVarRecvMultiCapacity.GetValueOnAnyThread());

			RecvMultiState = SocketSubsystem->CreateRecvMulti(MaxRecvMultiPackets, MAX_PACKET_SIZE, RecvMultiFlags);

			FArchiveCountMem MemArc(nullptr);

			RecvMultiState->CountBytes(MemArc);

			UE_LOGF(LogNet, Log, "NetDriver RecvMulti state size: %zu, Retrieve Timestamps: %i", MemArc.GetMax(),
					(uint32)bRetrieveTimestamps);
		}
		else
		{
			UE_LOGF(LogNet, Warning, "NetDriver could not enable RecvMulti, as current socket subsystem does not support it.");
		}
	}
	else if (bRecvMultiEnabled && bRecvThreadEnabled)
	{
		UE_LOGF(LogNet, Warning, "NetDriver RecvMulti is not yet supported with the Receive Thread enabled.");
	}

	// Success.
	return true;
}

bool UIpNetDriver::InitConnect( FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_IpNetDriver_InitConnect);

	using namespace UE::Net::Private;

	ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();
	if (SocketSubsystem == nullptr)
	{
		UE_LOGF(LogNet, Warning, "Unable to find socket subsystem");
		return false;
	}

	if( !InitBase( true, InNotify, ConnectURL, false, Error ) )
	{
		UE_LOGF(LogNet, Warning, "Failed to init net driver ConnectURL: %ls: %ls", *ConnectURL.ToString(), *Error);
		return false;
	}

	// Create new connection.
	ServerConnection = NewObject<UNetConnection>(GetTransientPackage(), NetConnectionClass);

	ServerConnection->InitLocalConnection(this, SocketPrivate.Get(), ConnectURL, USOCK_Pending);

	if (!Resolver->InitConnect(ServerConnection, SocketSubsystem, GetSocket(), ConnectURL))
	{
		Error = TEXT("Address resolution failed during InitConnect");
		return false;
	}

	UIpConnection* IpServerConnection = Cast<UIpConnection>(ServerConnection);

	if (FNetConnectionAddressResolution* ConnResolver = FNetDriverAddressResolution::GetConnectionResolver(IpServerConnection))
	{
		if (ConnResolver->IsAddressResolutionEnabled() && !ConnResolver->IsAddressResolutionComplete())
		{
			SocketState = ESocketState::Resolving;
		}
	}
	
	UE_LOGF(LogNet, Log, "Game client on port %i, rate %i", ConnectURL.Port, ServerConnection->CurrentNetSpeed );
	CreateInitialClientChannels();

	return true;
}

bool UIpNetDriver::InitListen( FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error )
{
	if( !InitBase( false, InNotify, LocalURL, bReuseAddressAndPort, Error ) )
	{
		UE_LOGF(LogNet, Warning, "Failed to init net driver ListenURL: %ls: %ls", *LocalURL.ToString(), *Error);
		return false;
	}

	InitConnectionlessHandler();

	// Update result URL.
	//LocalURL.Host = LocalAddr->ToString(false);
	LocalURL.Port = LocalAddr->GetPort();
	UE_LOGF(LogNet, Log, "%ls IpNetDriver listening on port %i", *GetDescription(), LocalURL.Port );

	return true;
}

void UIpNetDriver::TickDispatch(float DeltaTime)
{
	LLM_SCOPE_BYTAG(NetDriver);

	Super::TickDispatch( DeltaTime );

	const bool bUsingReceiveThread = SocketReceiveThreadRunnable.IsValid();

	if (bUsingReceiveThread)
	{
		SocketReceiveThreadRunnable->PumpOwnerEventQueue();
	}

#if !UE_BUILD_SHIPPING
	PauseReceiveEnd = (PauseReceiveEnd != 0.f && PauseReceiveEnd - (float)FPlatformTime::Seconds() > 0.f) ? PauseReceiveEnd : 0.f;

	if (PauseReceiveEnd != 0.f)
	{
		return;
	}
#endif

	// Set the context on the world for this driver's level collection.
	const int32 FoundCollectionIndex = World ? World->GetLevelCollections().IndexOfByPredicate([this](const FLevelCollection& Collection)
	{
		return Collection.GetNetDriver() == this;
	}) : INDEX_NONE;

	FScopedLevelCollectionContextSwitch LCSwitch(FoundCollectionIndex, World);


	DDoS.PreFrameReceive(DeltaTime);

	ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();
	bool bRetrieveTimestamps = CVarNetUseRecvTimestamps.GetValueOnAnyThread() != 0;

	// Process all incoming packets
	for (FPacketIterator It(this); It; ++It)
	{
		FReceivedPacketView ReceivedPacket;
		FInPacketTraits& ReceivedTraits = ReceivedPacket.Traits;
		bool bOk = It.GetCurrentPacket(ReceivedPacket);
		const TSharedRef<const FInternetAddr> FromAddr = ReceivedPacket.Address.ToSharedRef();
		UNetConnection* Connection = nullptr;
		UIpConnection* const MyServerConnection = GetServerConnection();

		if (bOk)
		{
			// Immediately stop processing (continuing to next receive), for empty packets (usually a DDoS)
			if (ReceivedPacket.DataView.NumBits() == 0)
			{
				DDoS.IncBadPacketCounter();
				continue;
			}

			FPacketAudit::NotifyLowLevelReceive((uint8*)ReceivedPacket.DataView.GetData(), ReceivedPacket.DataView.NumBytes());
		}
		else
		{
			if (IsRecvFailBlocking(ReceivedPacket.Error))
			{
				break;
			}
			else if (ReceivedPacket.Error != SE_ECONNRESET && ReceivedPacket.Error != SE_UDP_ERR_PORT_UNREACH)
			{
				// MalformedPacket: Client tried receiving a packet that exceeded the maximum packet limit
				// enforced by the server
				if (ReceivedPacket.Error == SE_EMSGSIZE)
				{
					DDoS.IncBadPacketCounter();

					if (MyServerConnection)
					{
						if (MyServerConnection->RemoteAddr->CompareEndpoints(*FromAddr))
						{
							Connection = MyServerConnection;
						}
						else
						{
							UE_LOGF(LogNet, Log, "Received packet with bytes > max MTU from an incoming IP address that doesn't match expected server address: Actual: %ls Expected: %ls",
								*FromAddr->ToString(true),
								MyServerConnection->RemoteAddr.IsValid() ? *MyServerConnection->RemoteAddr->ToString(true) : TEXT("Invalid"));
							continue;
						}
					}

					if (Connection != nullptr)
					{
						UE_LOGF(LogSecurity, Warning, "%ls: Malformed_Packet: Received Packet with bytes > max MTU", *Connection->RemoteAddressToString());
					}
				}
				else
				{
					DDoS.IncErrorPacketCounter();
				}

				FString ErrorString = FString::Printf(TEXT("UIpNetDriver::TickDispatch: Socket->RecvFrom: %i (%s) from %s"),
					static_cast<int32>(ReceivedPacket.Error),
					SocketSubsystem->GetSocketError(ReceivedPacket.Error),
					*FromAddr->ToString(true));


				// This should only occur on clients - on servers it leaves the NetDriver in an invalid/vulnerable state
				if (MyServerConnection != nullptr)
				{
					// TODO: Maybe we should check to see whether or not the From address matches the server?
					// If not, we could forward errors incorrectly, causing the connection to shut down.

					FIpConnectionHelper::HandleSocketRecvError(this, MyServerConnection, ErrorString);
					break;
				}
				else
				{
					// TODO: Should we also forward errors to connections here?
					// If we did, instead of just shutting down the NetDriver completely we could instead
					// boot the given connection.
					// May be DDoS concerns with the cost of looking up the connections for malicious packets
					// from sources that won't have connections.
					UE_CLOGF(!DDoS.CheckLogRestrictions(), LogNet, Warning, "%ls", *ErrorString);
				}

				// Unexpected packet errors should continue to the next iteration, rather than block all further receives this tick
				continue;
			}
		}


		// Figure out which socket the received data came from.
		if (MyServerConnection)
		{
			if (MyServerConnection->RemoteAddr->CompareEndpoints(*FromAddr))
			{
				Connection = MyServerConnection;
			}
			else
			{
				UE_LOGF(LogNet, Warning, "Incoming ip address doesn't match expected server address: Actual: %ls Expected: %ls",
					*FromAddr->ToString(true),
					MyServerConnection->RemoteAddr.IsValid() ? *MyServerConnection->RemoteAddr->ToString(true) : TEXT("Invalid"));
			}
		}

		if (Connection == nullptr)
		{
			if (TObjectPtr<UNetConnection>* ConnectionMapValue = MappedClientConnections.Find(FromAddr))
			{
				UNetConnection* FoundConnection = *ConnectionMapValue;
				if (FoundConnection)
				{
					if (ensureMsgf(FoundConnection->Driver, TEXT("Found invalid cleaned-up connection in Map: %s"), *Connection->Describe()))
					{
						Connection = FoundConnection;
						check(CastChecked<UIpConnection>(Connection)->RemoteAddr->CompareEndpoints(*FromAddr));
					}
				}
				else
				{
					ReceivedTraits.bFromRecentlyDisconnected = true;
				}
			}
		}


		if( bOk == false )
		{
			if( Connection )
			{
				if( Connection != GetServerConnection() )
				{
					// We received an ICMP port unreachable from the client, meaning the client is no longer running the game
					// (or someone is trying to perform a DoS attack on the client)

					// rcg08182002 Some buggy firewalls get occasional ICMP port
					// unreachable messages from legitimate players. Still, this code
					// will drop them unceremoniously, so there's an option in the .INI
					// file for servers with such flakey connections to let these
					// players slide...which means if the client's game crashes, they
					// might get flooded to some degree with packets until they timeout.
					// Either way, this should close up the usual DoS attacks.
					if ((Connection->GetConnectionState() != USOCK_Open) || (!AllowPlayerPortUnreach))
					{
						if (LogPortUnreach)
						{
							UE_LOGF(LogNet, Log, "Received ICMP port unreachable from client %ls.  Disconnecting.",
								*FromAddr->ToString(true));
						}
						Connection->CleanUp();
					}
				}
			}
			else
			{
				ReceivedTraits.bFromRecentlyDisconnected ? DDoS.IncDisconnPacketCounter() : DDoS.IncNonConnPacketCounter();

				if (LogPortUnreach && !DDoS.CheckLogRestrictions())
				{
					UE_LOGF(LogNet, Log, "Received ICMP port unreachable from %ls.  No matching connection found.",
						*FromAddr->ToString(true));
				}
			}
		}
		else
		{
			bool bIgnorePacket = false;

			// If we didn't find a client connection, maybe create a new one.
			if (Connection == nullptr)
			{
				if (DDoS.IsDDoSDetectionEnabled())
				{
					// If packet limits were reached, stop processing
					if (DDoS.ShouldBlockNonConnPackets())
					{
						DDoS.IncDroppedPacketCounter();
						continue;
					}


					ReceivedTraits.bFromRecentlyDisconnected ? DDoS.IncDisconnPacketCounter() : DDoS.IncNonConnPacketCounter();

					DDoS.CondCheckNonConnQuotasAndLimits();
				}

				// Determine if allowing for client/server connections
				const bool bAcceptingConnection = Notify != nullptr && Notify->NotifyAcceptingConnection() == EAcceptConnection::Accept;

				if (bAcceptingConnection)
				{
					if (!DDoS.CheckLogRestrictions() && !bExceededIPAggregationLimit)
					{
						TrackAndLogNewIP(FromAddr.Get());
					}

					FPacketBufferView WorkingBuffer = It.GetWorkingBuffer();

					Connection = ProcessConnectionlessPacket(ReceivedPacket, WorkingBuffer);
					bIgnorePacket = ReceivedPacket.DataView.NumBytes() == 0;
				}
				else
				{
					UE_LOGF(LogNet, VeryVerbose, "NotifyAcceptingConnection denied from: %ls", *FromAddr->ToString(true));
				}
			}

			// Send the packet to the connection for processing.
			if (Connection != nullptr && !bIgnorePacket)
			{
				if (DDoS.IsDDoSDetectionEnabled())
				{
					DDoS.IncNetConnPacketCounter();
					DDoS.CondCheckNetConnLimits();
				}

				if (bRetrieveTimestamps)
				{
					It.GetCurrentPacketTimestamp(Connection);
				}

				Connection->ReceivedRawPacket((uint8*)ReceivedPacket.DataView.GetData(), ReceivedPacket.DataView.NumBytes());
			}
		}
	}

	if (NewIPHashes.Num() > 0)
	{
		TickNewIPTracking(DeltaTime);
	}

	DDoS.PostFrameReceive();
}

FSocket* UIpNetDriver::GetSocket()
{
	using namespace UE::Net::Private;

	UIpConnection* IpServerConnection = Cast<UIpConnection>(ServerConnection);

	if (IpServerConnection != nullptr && FNetDriverAddressResolution::GetConnectionResolver(IpServerConnection)->IsAddressResolutionEnabled())
	{
		// Null here triggers the SocketNotReady path in handshake
		return IpServerConnection->GetSocket();
	}

	return SocketPrivate.Get();
}

UNetConnection* UIpNetDriver::ProcessConnectionlessPacket(FReceivedPacketView& PacketRef, const FPacketBufferView& WorkingBuffer)
{
	UNetConnection* ReturnVal = nullptr;
	TSharedPtr<StatelessConnectHandlerComponent> StatelessConnect;
	const TSharedPtr<const FInternetAddr>& Address = PacketRef.Address;
	FString IncomingAddress = Address->ToString(true);
	bool bPassedChallenge = false;
	bool bRestartedHandshake = false;
	bool bIgnorePacket = true;

	if (Notify != nullptr && ConnectionlessHandler.IsValid() && StatelessConnectComponent.IsValid())
	{
		StatelessConnect = StatelessConnectComponent.Pin();

		EIncomingResult Result = ConnectionlessHandler->IncomingConnectionless(PacketRef);

		if (Result == EIncomingResult::Success)
		{
			bPassedChallenge = StatelessConnect->HasPassedChallenge(Address, bRestartedHandshake);

			if (bPassedChallenge)
			{
				if (bRestartedHandshake)
				{
					UE_LOGF(LogNet, Log, "Finding connection to update to new address: %ls", *IncomingAddress);

					TSharedPtr<StatelessConnectHandlerComponent> CurComp;
					UIpConnection* FoundConn = nullptr;

					for (UNetConnection* const CurConn : ClientConnections)
					{
						CurComp = CurConn != nullptr ? CurConn->StatelessConnectComponent.Pin() : nullptr;

						if (CurComp.IsValid() && StatelessConnect->DoesRestartedHandshakeMatch(*CurComp))
						{
							FoundConn = Cast<UIpConnection>(CurConn);
							break;
						}
					}

					if (FoundConn != nullptr)
					{
						TSharedRef<FInternetAddr> RemoteAddrRef = FoundConn->RemoteAddr.ToSharedRef();

						// @todo: There needs to be a proper/standardized copy API for this. Also in IpConnection.cpp
						bool bIsValid = false;

						RemoveFromNewIPTracking(RemoteAddrRef.Get());
						
						// Remove the old address mapping
						MappedClientConnections.Remove(RemoteAddrRef);

						const FString OldAddress = RemoteAddrRef->ToString(true);

						RemoteAddrRef->SetIp(*Address->ToString(false), bIsValid);
						if (RemoteAddrRef->IsPortValid(Address->GetPort()))
						{
							RemoteAddrRef->SetPort(Address->GetPort());
						}

						MappedClientConnections.Add(RemoteAddrRef, FoundConn);

						// Make sure we didn't just invalidate a RecentlyDisconnectedClients entry, with the same address
						int32 RecentDisconnectIdx = RecentlyDisconnectedClients.IndexOfByPredicate(
							[&RemoteAddrRef](const FDisconnectedClient& CurElement)
							{
								return *RemoteAddrRef == *CurElement.Address;
							});

						if (RecentDisconnectIdx != INDEX_NONE)
						{
							RecentlyDisconnectedClients.RemoveAt(RecentDisconnectIdx);
						}

						// Reinitialize the stateless handshake component with the current address, to allow challenge ack retries
						CurComp->InitFromConnectionless(StatelessConnect.Get());

						ReturnVal = FoundConn;

						// We shouldn't need to log IncomingAddress, as the UNetConnection should dump it with it's description.
						UE_LOGF(LogNet, Log, "Updated IP address for connection. Connection = %ls, Old Address = %ls", *FoundConn->Describe(), *OldAddress);
					}
					else
					{
						UE_LOGF(LogNet, Log, "Failed to find an existing connection with a matching cookie. Restarted Handshake failed.");
					}
				}


				int32 NewCountBytes = PacketRef.DataView.NumBytes();
				uint8* WorkingData = WorkingBuffer.Buffer.GetData();

				if (NewCountBytes > 0)
				{
					const uint8* NewData = PacketRef.DataView.GetData();

					if (NewData != WorkingData)
					{
						FMemory::Memcpy(WorkingData, NewData, NewCountBytes);
					}

					bIgnorePacket = false;
				}

				PacketRef.DataView = {WorkingData, NewCountBytes, ECountUnits::Bytes};
			}
		}
	}
#if !UE_BUILD_SHIPPING
	else if (FParse::Param(FCommandLine::Get(), TEXT("NoPacketHandler")))
	{
		UE_CLOGF(!DDoS.CheckLogRestrictions(), LogNet, Log, "Accepting connection without handshake, due to '-NoPacketHandler'.")

		bIgnorePacket = false;
		bPassedChallenge = true;
	}
#endif
	else
	{
		UE_LOGF(LogNet, Log, "Invalid Notify (%i) or ConnectionlessHandler (%i) or StatelessConnectComponent (%i); can't accept connections.",
			(int32)(Notify != nullptr), (int32)(ConnectionlessHandler.IsValid()), (int32)(StatelessConnectComponent.IsValid()));
	}

	if (bPassedChallenge)
	{
		if (!bRestartedHandshake)
		{
			SCOPE_CYCLE_COUNTER(Stat_IpNetDriverAddNewConnection);

			UE_LOGF(LogNet, Log, "Server accepting post-challenge connection from: %ls", *IncomingAddress);

			ReturnVal = NewObject<UIpConnection>(GetTransientPackage(), NetConnectionClass);
			check(ReturnVal != nullptr);

			ReturnVal->InitRemoteConnection(this, SocketPrivate.Get(), World ? World->URL : FURL(), *Address, USOCK_Open);

			// Set the initial packet sequence from the handshake data
			if (StatelessConnect.IsValid())
			{
				int32 ServerSequence = 0;
				int32 ClientSequence = 0;

				StatelessConnect->GetChallengeSequence(ServerSequence, ClientSequence);

				ReturnVal->InitSequence(ClientSequence, ServerSequence);
			}

			if (ReturnVal->Handler.IsValid())
			{
				ReturnVal->Handler->BeginHandshaking();
			}

			Notify->NotifyAcceptedConnection(ReturnVal);

			AddClientConnection(ReturnVal);
			RemoveFromNewIPTracking(*Address.Get());
		}

		if (StatelessConnect.IsValid())
		{
			StatelessConnect->ResetChallengeData();
		}
	}
	else
	{
		UE_LOGF(LogNet, VeryVerbose, "Server failed post-challenge connection from: %ls", *IncomingAddress);
	}

	if (bIgnorePacket)
	{
		PacketRef.DataView = {PacketRef.DataView.GetData(), 0, ECountUnits::Bits};
	}

	return ReturnVal;
}

void UIpNetDriver::LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	if (Address.IsValid() && Address->IsValid())
	{
#if !UE_BUILD_SHIPPING
		if (GCurrentDuplicateIP.IsValid() && Address->CompareEndpoints(*GCurrentDuplicateIP))
		{
			TSharedRef<FInternetAddr> NewAddr = Address->Clone();
			int32 NewPort = NewAddr->GetPort() - 9876;

			NewAddr->SetPort(NewPort >= 0 ? NewPort : (65536 + NewPort));

			Address = NewAddr;
		}
#endif

		const uint8* DataToSend = reinterpret_cast<uint8*>(Data);

		if (ConnectionlessHandler.IsValid())
		{
			const ProcessedPacket ProcessedData =
					ConnectionlessHandler->OutgoingConnectionless(Address, (uint8*)DataToSend, CountBits, Traits);

			if (!ProcessedData.bError)
			{
				DataToSend = ProcessedData.Data;
				CountBits = ProcessedData.CountBits;
			}
			else
			{
				CountBits = 0;
			}
		}


		int32 BytesSent = 0;

		if (CountBits > 0)
		{
			CLOCK_CYCLES(SendCycles);
			GetSocket()->SendTo(DataToSend, FMath::DivideAndRoundUp(CountBits, 8), BytesSent, *Address);
			UNCLOCK_CYCLES(SendCycles);
		}


		// @todo: Can't implement these profiling events (require UNetConnections)
		//NETWORK_PROFILER(GNetworkProfiler.FlushOutgoingBunches(/* UNetConnection */));
		//NETWORK_PROFILER(GNetworkProfiler.TrackSocketSendTo(Socket->GetDescription(),Data,BytesSent,NumPacketIdBits,NumBunchBits,
							//NumAckBits,NumPaddingBits, /* UNetConnection */));
	}
	else
	{
		UE_LOGF(LogNet, Warning, "UIpNetDriver::LowLevelSend: Invalid send address '%ls'", *Address->ToString(true));
	}
}



FString UIpNetDriver::LowLevelGetNetworkNumber()
{
	return LocalAddr.IsValid() ? LocalAddr->ToString(true) : FString(TEXT(""));
}

void UIpNetDriver::LowLevelDestroy()
{
	using namespace UE::Net::Private;

	Super::LowLevelDestroy();

	// Close the socket.
	FSocket* CurrentSocket = GetSocket();
	if(CurrentSocket != nullptr && !HasAnyFlags(RF_ClassDefaultObject))
	{
		// Wait for send tasks if needed before closing the socket,
		// since at this point CleanUp() may not have been called on the server connection.
		UIpConnection* const IpServerConnection = GetServerConnection();
		if (IpServerConnection)
		{
			IpServerConnection->WaitForSendTasks();
		}

		ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();

		// If using a recieve thread, shut down the socket, which will signal the thread to exit gracefully, then wait on the thread.
		if (SocketReceiveThread.IsValid() && SocketReceiveThreadRunnable.IsValid())
		{
			UE_LOGF(LogNet, Log, "Shutting down and waiting for socket receive thread for %ls", *GetDescription());

			SocketReceiveThreadRunnable->bIsRunning = false;
			
			if (!CurrentSocket->Shutdown(ESocketShutdownMode::Read))
			{
				const ESocketErrors ShutdownError = SocketSubsystem->GetLastErrorCode();
				UE_LOGF(LogNet, Log, "UIpNetDriver::LowLevelDestroy Socket->Shutdown returned error %ls (%d) for %ls", SocketSubsystem->GetSocketError(ShutdownError), static_cast<int>(ShutdownError), *GetDescription());
			}

			SCOPE_CYCLE_COUNTER(STAT_IpNetDriver_Destroy_WaitForReceiveThread);
			SocketReceiveThread->WaitForCompletion();
		}

		if(!CurrentSocket->Close())
		{
			UE_LOGF(LogExit, Log, "closesocket error (%i)", (int32)SocketSubsystem->GetLastErrorCode() );
		}

		FNetConnectionAddressResolution* ConnResolver = FNetDriverAddressResolution::GetConnectionResolver(IpServerConnection);

		if (ConnResolver != nullptr && ConnResolver->IsAddressResolutionEnabled())
		{
			ConnResolver->CleanupResolutionSockets();
		}

		ClearSockets();

		UE_LOGF(LogExit, Log, "%ls shut down",*GetDescription() );
	}

}


bool UIpNetDriver::HandleSocketsCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	Ar.Logf(TEXT(""));
	FSocket* CmdSocket = GetSocket();
	if (CmdSocket != nullptr)
	{
		TSharedRef<FInternetAddr> LocalInternetAddr = GetSocketSubsystem()->CreateInternetAddr();
		CmdSocket->GetAddress(*LocalInternetAddr);
		Ar.Logf(TEXT("%s Socket: %s"), *GetDescription(), *LocalInternetAddr->ToString(true));
	}		
	else
	{
		Ar.Logf(TEXT("%s Socket: null"), *GetDescription());
	}
	return UNetDriver::Exec( InWorld, TEXT("SOCKETS"),Ar);
}

bool UIpNetDriver::HandlePauseReceiveCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	FString PauseTimeStr;
	uint32 PauseTime;

	if (FParse::Token(Cmd, PauseTimeStr, false) && (PauseTime = FCString::Atoi(*PauseTimeStr)) > 0)
	{
		Ar.Logf(TEXT("Pausing Socket Receives for '%i' seconds."), PauseTime);

		PauseReceiveEnd = float(FPlatformTime::Seconds() + (double)PauseTime);
	}
	else
	{
		Ar.Logf(TEXT("Must specify a pause time, in seconds."));
	}

	return true;
}

// Doubles as an inbuilt restart handshake test
bool UIpNetDriver::HandleRecreateSocketCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	using namespace UE::Net;

	FString PortStr;
	int32 OverridePort = INDEX_NONE;

	if (FParse::Token(Cmd, PortStr, false))
	{
		OverridePort = FCString::Atoi(*PortStr);
	}

	ERecreateSocketResult Result = RecreateSocket(OverridePort);

	if (Result == ERecreateSocketResult::BeganRecreate)
	{
		Ar.Logf(TEXT("Began socket recreation."));
	}
	else
	{
		Ar.Logf(TEXT("Failed to trigger socket recreation. Result: %s"), ToCStr(LexToString(Result)));
	}

	return true;
}

#if !UE_BUILD_SHIPPING
void UIpNetDriver::TestSuddenPortChange(uint32 NumConnections)
{
	if (ConnectionlessHandler.IsValid() && StatelessConnectComponent.IsValid())
	{
		TSharedPtr<StatelessConnectHandlerComponent> StatelessConnect = StatelessConnectComponent.Pin();

		for (int32 i = 0; i < ClientConnections.Num() && NumConnections-- > 0; i++)
		{
			// Reset the connection's port to pretend that we used to be sending traffic on an old connection. This is
			// done because once the test is complete, we need to be back onto the port we started with. This
			// fakes what happens in live with clients randomly sending traffic on a new port.
			if (UIpConnection* const TestConnection = Cast<UIpConnection>(ClientConnections[i]))
			{
				TSharedRef<FInternetAddr> RemoteAddrRef = TestConnection->RemoteAddr.ToSharedRef();

				MappedClientConnections.Remove(RemoteAddrRef);

				TestConnection->RemoteAddr->SetPort(i + 9876);

				MappedClientConnections.Add(RemoteAddrRef, TestConnection);

				// We need to set AllowPlayerPortUnreach to true because the net driver will try sending traffic
				// to the IP/Port we just set which is invalid. On Windows, this causes an error to be returned in
				// RecvFrom (WSAECONNRESET). When AllowPlayerPortUnreach is true, these errors are ignored.
				AllowPlayerPortUnreach = true;
				UE_LOGF(LogNet, Log, "TestSuddenPortChange - Changed this connection: %ls.", *TestConnection->Describe());
			}
		}
	}
}
#endif

#if UE_ALLOW_EXEC_COMMANDS
bool UIpNetDriver::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (FParse::Command(&Cmd,TEXT("SOCKETS")))
	{
		return HandleSocketsCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("PauseReceive")))
	{
		return HandlePauseReceiveCommand(Cmd, Ar, InWorld);
	}
	else if (FParse::Command(&Cmd, TEXT("RecreateSocket")))
	{
		return HandleRecreateSocketCommand(Cmd, Ar, InWorld);
	}

	return UNetDriver::Exec( InWorld, Cmd,Ar);
}
#endif // UE_ALLOW_EXEC_COMMANDS

UIpConnection* UIpNetDriver::GetServerConnection() 
{
	return Cast<UIpConnection>(ServerConnection);
}

void UIpNetDriver::TrackAndLogNewIP(const FInternetAddr& InAddr)
{
	using namespace IPNetDriverInternal;

	const uint32 IPHash = GetTypeHash(InAddr);
	int32 HashIdx = NewIPHashes.Find(IPHash);

	if (HashIdx == INDEX_NONE)
	{
		bExceededIPAggregationLimit = NewIPHashes.Num() >= MaxIPHashes;

		if (LIKELY(!bExceededIPAggregationLimit))
		{
			HashIdx = NewIPHashes.Num();

			NewIPHashes.Add(IPHash);
			NewIPHitCount.Add(0);
		}
	}

	if (LIKELY(!bExceededIPAggregationLimit))
	{
		const uint32 IPHitCount = ++NewIPHitCount[HashIdx];
		const bool bLogIPHit = IPHitCount <= static_cast<uint32>(MaxIPHitLogs);

		if (bLogIPHit)
		{
			UE_LOGF(LogNet, Log, "NotifyAcceptingConnection accepted from: %ls", ToCStr(InAddr.ToString(true)));
		}
		else if (IPHitCount == static_cast<uint32>(MaxIPHitLogs) + 1)
		{
			bExceededIPAggregationLimit = AggregatedIPsToLog.Num() >= MaxAggregateIPLogs;

			if (!bExceededIPAggregationLimit)
			{
				AggregatedIPsToLog.Add(UIpNetDriver::FAggregatedIP{IPHash, InAddr.ToString(true)});
			}
		}
	}
}

void UIpNetDriver::RemoveFromNewIPTracking(const FInternetAddr& InAddr)
{
	using namespace IPNetDriverInternal;

	const uint32 IPHash = GetTypeHash(InAddr);
	const int32 HashIdx = NewIPHashes.Find(IPHash);

	if (HashIdx != INDEX_NONE)
	{
		const bool bInAggregateLogList = NewIPHitCount[HashIdx] > static_cast<uint32>(MaxIPHitLogs);

		if (bInAggregateLogList)
		{
			const int32 AggIdx = AggregatedIPsToLog.IndexOfByPredicate([IPHash](const FAggregatedIP& A) { return A.IPHash == IPHash; });

			if (AggIdx != INDEX_NONE)
			{
				AggregatedIPsToLog.RemoveAtSwap(AggIdx, EAllowShrinking::No);
			}
		}

		// Don't remove from other tracking arrays unless it's the last entry
		if (NewIPHashes.Num() == 1)
		{
			ResetNewIPTracking();
		}
		else if (HashIdx == NewIPHashes.Num()-1)
		{
			NewIPHashes.RemoveAt(HashIdx, EAllowShrinking::No);
			NewIPHitCount.RemoveAt(HashIdx, EAllowShrinking::No);
		}
		else
		{
			NewIPHitCount[HashIdx] = 0;
		}
	}
}

void UIpNetDriver::ResetNewIPTracking()
{
	using namespace IPNetDriverInternal;

	NewIPHashes.Empty(MaxIPHashes);
	NewIPHitCount.Empty(MaxIPHashes);
	AggregatedIPsToLog.Empty(MaxAggregateIPLogs);
	bExceededIPAggregationLimit = false;
	NextAggregateIPLogCountdown = 0.0;
}

void UIpNetDriver::TickNewIPTracking(float DeltaTime)
{
	using namespace IPNetDriverInternal;

	bool bCheckAggregatedIPs = false;

	if (NextAggregateIPLogCountdown == 0.0)
	{
		NextAggregateIPLogCountdown = AggregateIPLogInterval + FMath::RandRange(0.0, AggregateIPLogIntervalVariance);
	}
	else
	{
		NextAggregateIPLogCountdown -= DeltaTime;

		if (NextAggregateIPLogCountdown <= 0.0)
		{
			NextAggregateIPLogCountdown = 0.0;
			bCheckAggregatedIPs = true;
		}
	}

	if (bCheckAggregatedIPs)
	{
		if (bExceededIPAggregationLimit)
		{
			UE_LOGF(LogNet, Log, "NotifyAcceptingConnection accepted aggregation: Last tracking period exceeded limit.");
		}
		else if (AggregatedIPsToLog.Num() > 0)
		{
			TStringBuilder<8192> AggregateIPLogStr;

			AggregateIPLogStr.Append(TEXT("NotifyAcceptingConnection accepted aggregation: "));

			for (int32 AggIdx=0; AggIdx<AggregatedIPsToLog.Num(); AggIdx++)
			{
				const int32 HashIdx = NewIPHashes.Find(AggregatedIPsToLog[AggIdx].IPHash);

				if (AggIdx > 0)
				{
					AggregateIPLogStr.Append(TEXT(", "));
				}

				AggregateIPLogStr.Append(ToCStr(AggregatedIPsToLog[AggIdx].IPStr));
				AggregateIPLogStr.Appendf(TEXT(" (%i)"), NewIPHitCount[HashIdx]);
			}

			UE_LOGF(LogNet, Log, "%ls", ToCStr(AggregateIPLogStr.ToString()));
		}

		ResetNewIPTracking();
	}
}

UE::Net::ERecreateSocketResult UIpNetDriver::RecreateSocket(int32 OverridePort)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	ERecreateSocketResult Result = ERecreateSocketResult::NoAction;

	LastRecreateSocketTime = GetElapsedTime();

	// Only clients can recreate the socket on-the-fly
	if (bSupportsRecreateSocket && ServerConnection != nullptr && !ServerConnection->IsReplay())
	{
		const ESocketState CurState = GetSocketState();

		if (CurState == ESocketState::Ready)
		{
			const int32 BindPort = OverridePort != INDEX_NONE ? OverridePort : GetClientPort();

			FString Error;
			FUniqueSocket NewSocket = CreateAndBindSocket(LocalAddr.ToSharedRef(), BindPort, false, ClientDesiredSocketReceiveBufferBytes,
															ClientDesiredSocketSendBufferBytes, Error);

			if (NewSocket.IsValid())
			{
				UE_LOGF(LogNet, Log, "Recreated socket for bind address: %ls", ToCStr(LocalAddr->ToString(true)));

				FSetSocketComplete SetCallback =
					[ThisPtr = TWeakObjectPtr<UIpNetDriver>(this)]()
					{
						if (ThisPtr.IsValid())
						{
							ThisPtr->OnRecreateSocketComplete();
						}
					};

				SocketState = ESocketState::Recreating;
				Result = ERecreateSocketResult::BeganRecreate;

				SetSocket_Internal(TSharedPtr<FSocket>(NewSocket.Release(), FSocketDeleter(NewSocket.GetDeleter())), MoveTemp(SetCallback));
			}
			else
			{
				UE_LOGF(LogNet, Warning, "Could not recreate socket for bind address %ls, got error %ls", ToCStr(LocalAddr->ToString(false)),
						ToCStr(Error));

				Result = ERecreateSocketResult::Error;
			}
		}
		else if (CurState == ESocketState::Resolving)
		{
			Result = ERecreateSocketResult::NotReady;
		}
		else if (CurState == ESocketState::Recreating)
		{
			Result = ERecreateSocketResult::AlreadyInProgress;
		}
		else // if (CurState == ESocketState::Error)
		{
			Result = ERecreateSocketResult::Error;
		}
	}

	return Result;
}

void UIpNetDriver::OnRecreateSocketComplete()
{
	SocketState = ESocketState::Ready;
	LastRecreateSocketTime = GetElapsedTime();
}

void UIpNetDriver::SetSocket_Internal(const TSharedPtr<FSocket>& InSocket, UE::Net::Private::FSetSocketComplete InSetCallback/*=nullptr*/)
{
	if (SocketState != ESocketState::Recreating)
	{
		SocketState = ESocketState::Ready;
	}

	bool bHandledCallback = false;;

	if (InSocket != SocketPrivate)
	{
		// Keep old socket around until we know we are done with it (don't want to trigger ICMP unreachable on the remote side by closing early)
		TSharedPtr<FSocket> SavedSocket = MoveTemp(SocketPrivate);

		SocketPrivate = InSocket;

		const bool bUsingReceiveThread = SocketReceiveThreadRunnable.IsValid();

		if (UIpConnection* IpServerConnection = GetServerConnection())
		{
			FIpConnectionHelper::SetSocket_Local(IpServerConnection, InSocket);
		}
		else
		{
			for (UNetConnection* CurConn : ClientConnections)
			{
				if (UIpConnection* IpConn = Cast<UIpConnection>(CurConn))
				{
					FIpConnectionHelper::SetSocket_Local(IpConn, InSocket);
				}
			}
		}

		if (bUsingReceiveThread)
		{
			SocketReceiveThreadRunnable->SetSocket(SocketPrivate, MoveTemp(InSetCallback));

			bHandledCallback = true;
		}
	}

	if (!bHandledCallback && InSetCallback)
	{
		InSetCallback();
	}
}


UIpNetDriver::FReceiveThreadRunnable::FReceiveThreadRunnable(UIpNetDriver* InOwningNetDriver)
	: ReceiveQueue(CVarNetIpNetDriverReceiveThreadQueueMaxPackets.GetValueOnAnyThread())
	, bIsRunning(true)
	, OwningNetDriver(InOwningNetDriver)
{
	SocketSubsystem = OwningNetDriver->GetSocketSubsystem();
}

bool UIpNetDriver::FReceiveThreadRunnable::DispatchPacket(FReceivedPacket&& IncomingPacket, int32 NbBytesRead)
{
	IncomingPacket.PacketBytes.SetNum(FMath::Max(NbBytesRead, 0), EAllowShrinking::No);
	IncomingPacket.PlatformTimeSeconds = FPlatformTime::Seconds();

	// Add packet to queue. Since ReceiveQueue is a TCircularQueue, if the queue is full, this will simply return false without adding anything.
	return ReceiveQueue.Enqueue(MoveTemp(IncomingPacket));
}

uint32 UIpNetDriver::FReceiveThreadRunnable::Run()
{
	using namespace UE::Net::Private;

	const int32 PollTimeMS = CVarNetIpNetDriverReceiveThreadPollTimeMS.GetValueOnAnyThread();
	const FTimespan Timeout = FTimespan::FromMilliseconds(PollTimeMS);
	const float SleepTimeForWaitableErrorsInSec = CVarRcvThreadSleepTimeForWaitableErrorsInSeconds.GetValueOnAnyThread();
	const int32 ActionForLongRecvErrors = CVarRcvThreadShouldSleepForLongRecvErrors.GetValueOnAnyThread();

	UE_LOGF(LogNet, Log, "UIpNetDriver::FReceiveThreadRunnable::Run starting up.");

	// Wait for the Socket to be set
	while (bIsRunning && !Socket.IsValid())
	{
		// Process commands from the Game Thread
		PumpCommandQueue();

		const float NoSocketSetSleep = .03f;
		FPlatformProcess::SleepNoStats(NoSocketSetSleep);
	}


	while (bIsRunning && Socket.IsValid())
	{
		// If we've encountered any errors during address resolution (this flag will not have the error state on it if resolution is disabled)
		// Then stop running this thread. This stomps out any potential infinite loops caused by undefined behavior.
		UIpConnection* ServerConn = OwningNetDriver->GetServerConnection();

		if (ServerConn != nullptr && FNetDriverAddressResolution::GetConnectionResolver(ServerConn)->HasAddressResolutionFailed())
		{
			ClearSocket();

			break;
		}

		FReceivedPacket IncomingPacket;

		bool bReceiveQueueFull = false;
		const bool bWaitResult = Socket->Wait(ESocketWaitConditions::WaitForRead, Timeout);

		PumpCommandQueue();

		if (bWaitResult)
		{
			bool bOk = false;
			int32 BytesRead = 0;

			IncomingPacket.FromAddress = SocketSubsystem->CreateInternetAddr();

			IncomingPacket.PacketBytes.AddUninitialized(MAX_PACKET_SIZE);

			{
				SCOPE_CYCLE_COUNTER(STAT_IpNetDriver_RecvFromSocket);
				bOk = Socket->RecvFrom(IncomingPacket.PacketBytes.GetData(), IncomingPacket.PacketBytes.Num(), BytesRead, *IncomingPacket.FromAddress);
			}

			if (bOk)
			{
				// Don't even queue empty packets, they can be ignored.
				if (BytesRead != 0)
				{
					const bool bSuccess = DispatchPacket(MoveTemp(IncomingPacket), BytesRead);
					bReceiveQueueFull = !bSuccess;
				}
			}
			else
			{
				// This relies on the platform's implementation using thread-local storage for the last socket error code.
				ESocketErrors RecvFromError = SocketSubsystem->GetLastErrorCode();

				if (IsRecvFailBlocking(RecvFromError) == false)
				{
					// Only non-blocking errors are dispatched to the Game Thread
					IncomingPacket.Error = RecvFromError;
					const bool bSuccess = DispatchPacket(MoveTemp(IncomingPacket), BytesRead);
					bReceiveQueueFull = !bSuccess;
				}

				if (IPNetDriverInternal::IsLongRecvError(RecvFromError))
				{
					if (ActionForLongRecvErrors == 1)
					{
						if (SleepTimeForWaitableErrorsInSec >= 0.f)
						{
							FPlatformProcess::SleepNoStats(SleepTimeForWaitableErrorsInSec);
						}
					}
					else if (ActionForLongRecvErrors == 2)
					{
						break;
					}
				}
			}
		}
		else
		{
			ESocketErrors WaitError = SocketSubsystem->GetLastErrorCode();

			if (IPNetDriverInternal::ShouldSleepOnWaitError(WaitError))
			{
				if (SleepTimeForWaitableErrorsInSec >= 0.0)
				{
					FPlatformProcess::SleepNoStats(SleepTimeForWaitableErrorsInSec);
				}
			}
			else if (IsRecvFailBlocking(WaitError) == false)
			{
				// Only non-blocking errors are dispatched to the Game Thread
				IncomingPacket.Error = WaitError;
				const bool bSuccess = DispatchPacket(MoveTemp(IncomingPacket), 0);
				bReceiveQueueFull = !bSuccess;
			}
		}

		if (bReceiveQueueFull)
		{
			if (SleepTimeForWaitableErrorsInSec >= 0.0)
			{
				FPlatformProcess::SleepNoStats(SleepTimeForWaitableErrorsInSec);
			}
		}
	}

	// In case of a non-standard exit from the main loop, wait for a proper Game Thread triggered shutdown and cleanup
	while (bIsRunning && Socket.IsValid())
	{
		PumpCommandQueue();
		FPlatformProcess::SleepNoStats(PollTimeMS / 1000.f);
	}

	UE_LOGF(LogNet, Log, "UIpNetDriver::FReceiveThreadRunnable::Run returning.");

	return 0;
}

void UIpNetDriver::FReceiveThreadRunnable::PumpOwnerEventQueue()
{
	while (TOptional<TUniqueFunction<void()>> CurCommand = OwnerEventQueue.Dequeue())
	{
		(*CurCommand)();
	}
}

void UIpNetDriver::FReceiveThreadRunnable::PumpCommandQueue()
{
	while (TOptional<TUniqueFunction<void()>> CurCommand = CommandQueue.Dequeue())
	{
		(*CurCommand)();
	}
}

void UIpNetDriver::FReceiveThreadRunnable::SetSocket(const TSharedPtr<FSocket>& InGameThreadSocket,
														UE::Net::Private::FSetSocketComplete InGameThreadSetCallback/*=nullptr*/)
{
	CommandQueue.Enqueue([this, GameThreadSocket = InGameThreadSocket, GameThreadSetCallback = MoveTemp(InGameThreadSetCallback)]() mutable
	{
		this->SetSocketImpl(GameThreadSocket, MoveTemp(GameThreadSetCallback));
	});
}

void UIpNetDriver::FReceiveThreadRunnable::SetSocketImpl(const TSharedPtr<FSocket>& InGameThreadSocket,
														UE::Net::Private::FSetSocketComplete InGameThreadSetCallback/*=nullptr*/)
{
	ClearSocket();

	Socket = InGameThreadSocket;

	if (InGameThreadSetCallback)
	{
		OwnerEventQueue.Enqueue([GameThreadSetCallback = MoveTemp(InGameThreadSetCallback)]() mutable
		{
			GameThreadSetCallback();
		});
	}
}

void UIpNetDriver::FReceiveThreadRunnable::ClearSocket()
{
	if (Socket.IsValid())
	{
		TSharedPtr<FSocket> GameThreadSocket = MoveTemp(Socket);

		Socket.Reset();

		// Clear the socket shared pointer from the Game Thread
		if (GameThreadSocket.IsValid())
		{
			OwnerEventQueue.Enqueue([GameThreadSocket = MoveTemp(GameThreadSocket)]() mutable
			{
				GameThreadSocket.Reset();
			});
		}
	}
}
