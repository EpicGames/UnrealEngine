// Copyright Epic Games, Inc. All Rights Reserved.

#include "JoinSessionCallbackProxy.h"
#include "FindSessionsCallbackProxy.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(JoinSessionCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// UJoinSessionCallbackProxy

UJoinSessionCallbackProxy::UJoinSessionCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Delegate(FOnJoinSessionCompleteDelegate::CreateUObject(this, &ThisClass::OnCompleted))
{
}

UJoinSessionCallbackProxy* UJoinSessionCallbackProxy::JoinSession(UObject* WorldContextObject, class APlayerController* PlayerController, const FBlueprintSessionResult& SearchResult)
{
	UJoinSessionCallbackProxy* Proxy = NewObject<UJoinSessionCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->OnlineSearchResult = SearchResult.OnlineResult;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UJoinSessionCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("JoinSession"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			DelegateHandle = Sessions->AddOnJoinSessionCompleteDelegate_Handle(Delegate);
			Sessions->JoinSession(*Helper.UserID, NAME_GameSession, OnlineSearchResult);

			// OnCompleted will get called, nothing more to do now
			return;
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Sessions not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UJoinSessionCallbackProxy::OnCompleted(FName SessionName, EOnJoinSessionCompleteResult::Type Result)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("JoinSessionCallback"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		auto Sessions = Helper.OnlineSub->GetSessionInterface();
		if (Sessions.IsValid())
		{
			Sessions->ClearOnJoinSessionCompleteDelegate_Handle(DelegateHandle);

			if (Result == EOnJoinSessionCompleteResult::Success)
			{
				// Client travel to the server
				FString ConnectString;
				if (Sessions->GetResolvedConnectString(NAME_GameSession, ConnectString) && PlayerControllerWeakPtr.IsValid())
				{
					UE_LOG_ONLINE_SESSION(Log, TEXT("Join session: traveling to %s"), *ConnectString);
					PlayerControllerWeakPtr->ClientTravel(ConnectString, TRAVEL_Absolute);
					OnSuccess.Broadcast();
					return;
				}
			}
		}
	}

	OnFailure.Broadcast();
}

