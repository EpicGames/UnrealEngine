// Copyright Epic Games, Inc. All Rights Reserved.

#include "LeaderboardBlueprintLibrary.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineLeaderboardInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LeaderboardBlueprintLibrary)

//////////////////////////////////////////////////////////////////////////
// ULeaderboardBlueprintLibrary

ULeaderboardBlueprintLibrary::ULeaderboardBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool ULeaderboardBlueprintLibrary::WriteLeaderboardObject(APlayerController* PlayerController, class FOnlineLeaderboardWrite& WriteObject)
{
	if (APlayerState* PlayerState = (PlayerController != NULL) ? ToRawPtr(PlayerController->PlayerState) : NULL)
	{
		FUniqueNetIdPtr UserId = PlayerState->GetUniqueId().GetUniqueNetId();
		if (UserId.IsValid())
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				IOnlineLeaderboardsPtr Leaderboards = OnlineSub->GetLeaderboardsInterface();
				if (Leaderboards.IsValid())
				{
					// the call will copy the user id and write object to its own memory
					bool bResult = Leaderboards->WriteLeaderboards(PlayerState->SessionName, *UserId, WriteObject);

					// Flush the leaderboard immediately for now
					bool bFlushResult = Leaderboards->FlushLeaderboards(PlayerState->SessionName);

					return bResult && bFlushResult;
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("WriteLeaderboardObject - Leaderboards not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("WriteLeaderboardObject - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("WriteLeaderboardObject - Cannot map local player to unique net ID"), ELogVerbosity::Warning);
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("WriteLeaderboardObject - Invalid player state"), ELogVerbosity::Warning);
	}

	return false;
}

bool ULeaderboardBlueprintLibrary::WriteLeaderboardInteger(APlayerController* PlayerController, FName StatName, int32 StatValue)
{
	FOnlineLeaderboardWrite WriteObject;
	const FString StatNameString = StatName.ToString();
	WriteObject.LeaderboardNames.Add(StatNameString);
	WriteObject.RatedStat = StatNameString;
	WriteObject.DisplayFormat = ELeaderboardFormat::Number;
	WriteObject.SortMethod = ELeaderboardSort::Descending;
	WriteObject.UpdateMethod = ELeaderboardUpdateMethod::KeepBest;
	WriteObject.SetIntStat(StatNameString, StatValue);

	return WriteLeaderboardObject(PlayerController, WriteObject);
}

