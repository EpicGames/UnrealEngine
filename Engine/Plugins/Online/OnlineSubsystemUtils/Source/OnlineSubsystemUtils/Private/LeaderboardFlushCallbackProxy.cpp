// Copyright Epic Games, Inc. All Rights Reserved.

#include "LeaderboardFlushCallbackProxy.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"
#include "OnlineSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LeaderboardFlushCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// ULeaderboardFlushCallbackProxy

ULeaderboardFlushCallbackProxy::ULeaderboardFlushCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULeaderboardFlushCallbackProxy::TriggerFlush(APlayerController* PlayerController, FName InSessionName)
{
	bFailedToEvenSubmit = true;

	if (APlayerState* PlayerState = (PlayerController != NULL) ? ToRawPtr(PlayerController->PlayerState) : NULL)
	{
		FUniqueNetIdPtr UserID = PlayerState->GetUniqueId().GetUniqueNetId();
		if (UserID.IsValid())
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				IOnlineLeaderboardsPtr Leaderboards = OnlineSub->GetLeaderboardsInterface();
				if (Leaderboards.IsValid())
				{
					// Register the completion callback
					LeaderboardFlushCompleteDelegate       = FOnLeaderboardFlushCompleteDelegate::CreateUObject(this, &ULeaderboardFlushCallbackProxy::OnFlushCompleted);
					LeaderboardFlushCompleteDelegateHandle = Leaderboards->AddOnLeaderboardFlushCompleteDelegate_Handle(LeaderboardFlushCompleteDelegate);

					// Flush the leaderboard
					Leaderboards->FlushLeaderboards(InSessionName);
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("ULeaderboardFlushCallbackProxy::TriggerFlush - Leaderboards not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("ULeaderboardFlushCallbackProxy::TriggerFlush - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("ULeaderboardFlushCallbackProxy::TriggerFlush - Cannot map local player to unique net ID"), ELogVerbosity::Warning);
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("ULeaderboardQueryCallbackProxy::TriggerFlush - Invalid player state"), ELogVerbosity::Warning);
	}

	if (bFailedToEvenSubmit)
	{
		OnFlushCompleted(InSessionName, false);
	}
}


void ULeaderboardFlushCallbackProxy::OnFlushCompleted(FName SessionName, bool bWasSuccessful)
{
	RemoveDelegate();

	if (bWasSuccessful)
	{
		OnSuccess.Broadcast(SessionName);
	}
	else
	{
		OnFailure.Broadcast(SessionName);
	}
}

void ULeaderboardFlushCallbackProxy::RemoveDelegate()
{
	if (!bFailedToEvenSubmit)
	{
		if (IOnlineSubsystem* OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
		{
			IOnlineLeaderboardsPtr Leaderboards = OnlineSub->GetLeaderboardsInterface();
			if (Leaderboards.IsValid())
			{
				Leaderboards->ClearOnLeaderboardFlushCompleteDelegate_Handle(LeaderboardFlushCompleteDelegateHandle);
			}
		}
	}
}

void ULeaderboardFlushCallbackProxy::BeginDestroy()
{
	RemoveDelegate();

	Super::BeginDestroy();
}

ULeaderboardFlushCallbackProxy* ULeaderboardFlushCallbackProxy::CreateProxyObjectForFlush(class APlayerController* PlayerController, FName SessionName)
{
	ULeaderboardFlushCallbackProxy* Proxy = NewObject<ULeaderboardFlushCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerFlush(PlayerController, SessionName);
	return Proxy;
}

