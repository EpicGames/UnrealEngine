// Copyright Epic Games, Inc. All Rights Reserved.

#include "LeaderboardQueryCallbackProxy.h"
#include "TimerManager.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"
#include "OnlineSubsystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LeaderboardQueryCallbackProxy)

//////////////////////////////////////////////////////////////////////////
// ULeaderboardQueryCallbackProxy

ULeaderboardQueryCallbackProxy::ULeaderboardQueryCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULeaderboardQueryCallbackProxy::TriggerQuery(APlayerController* PlayerController, const FString& InStatName, EOnlineKeyValuePairDataType::Type StatType)
{
	bFailedToEvenSubmit = true;

	WorldPtr = (PlayerController != NULL) ? PlayerController->GetWorld() : NULL;
	if (APlayerState* PlayerState = (PlayerController != NULL) ? ToRawPtr(PlayerController->PlayerState) : NULL)
	{
		FUniqueNetIdPtr UserID = PlayerState->GetUniqueId().GetUniqueNetId();
		if (UserID.IsValid())
		{
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
			{
				IOnlineLeaderboardsPtr Leaderboards = OnlineSub->GetLeaderboardsInterface();
				if (Leaderboards.IsValid())
				{
					bFailedToEvenSubmit = false;

					StatName = InStatName;
					ReadObject = MakeShareable(new FOnlineLeaderboardRead());
					ReadObject->LeaderboardName = InStatName;
					ReadObject->SortedColumn = InStatName;
					new (ReadObject->ColumnMetadata) FColumnMetaData(InStatName, StatType);

					// Register the completion callback
					LeaderboardReadCompleteDelegate       = FOnLeaderboardReadCompleteDelegate::CreateUObject(this, &ULeaderboardQueryCallbackProxy::OnStatsRead);
					LeaderboardReadCompleteDelegateHandle = Leaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadCompleteDelegate);

					TArray< FUniqueNetIdRef > ListOfIDs;
					ListOfIDs.Add(UserID.ToSharedRef());

					FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();
					Leaderboards->ReadLeaderboards(ListOfIDs, ReadObjectRef);
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("ULeaderboardQueryCallbackProxy::TriggerQuery - Leaderboards not supported by Online Subsystem"), ELogVerbosity::Warning);
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("ULeaderboardQueryCallbackProxy::TriggerQuery - Invalid or uninitialized OnlineSubsystem"), ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("ULeaderboardQueryCallbackProxy::TriggerQuery - Cannot map local player to unique net ID"), ELogVerbosity::Warning);
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("ULeaderboardQueryCallbackProxy::TriggerQuery - Invalid player state"), ELogVerbosity::Warning);
	}

	if (bFailedToEvenSubmit && (PlayerController != NULL))
	{
		OnStatsRead(false);
	}
}

void ULeaderboardQueryCallbackProxy::OnStatsRead(bool bWasSuccessful)
{
	RemoveDelegate();

	bool bFoundValue = false;
	int32 Value = 0;

	if (bWasSuccessful)
	{
		if (ReadObject.IsValid())
		{
			for (int Idx = 0; Idx < ReadObject->Rows.Num(); ++Idx)
			{
				FVariantData* Variant = ReadObject->Rows[Idx].Columns.Find(StatName);

				if (Variant != nullptr)
				{
					bFoundValue = true;
					Variant->GetValue(Value);
				}
			}
		}
	}

	if (bFoundValue)
	{
		bSavedWasSuccessful = true;
		SavedValue = Value;
	}
	else
	{
		bSavedWasSuccessful = false;
		SavedValue = 0;
	}

	if (UWorld* World = WorldPtr.Get())
	{
		// Use a dummy timer handle as we don't need to store it for later but we don't need to look for something to clear
		FTimerHandle TimerHandle;
		World->GetTimerManager().SetTimer(OnStatsRead_DelayedTimerHandle, this, &ULeaderboardQueryCallbackProxy::OnStatsRead_Delayed, 0.001f, false);
	}
	ReadObject = NULL;
}

void ULeaderboardQueryCallbackProxy::OnStatsRead_Delayed()
{
	if (bSavedWasSuccessful)
	{
		OnSuccess.Broadcast(SavedValue);
	}
	else
	{
		OnFailure.Broadcast(0);
	}
}

void ULeaderboardQueryCallbackProxy::RemoveDelegate()
{
	if (!bFailedToEvenSubmit)
	{
		if (IOnlineSubsystem* OnlineSub = IOnlineSubsystem::IsLoaded() ? IOnlineSubsystem::Get() : nullptr)
		{
			IOnlineLeaderboardsPtr Leaderboards = OnlineSub->GetLeaderboardsInterface();
			if (Leaderboards.IsValid())
			{
				Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadCompleteDelegateHandle);
			}
		}
	}
}

void ULeaderboardQueryCallbackProxy::BeginDestroy()
{
	ReadObject = NULL;
	RemoveDelegate();

	Super::BeginDestroy();
}

ULeaderboardQueryCallbackProxy* ULeaderboardQueryCallbackProxy::CreateProxyObjectForIntQuery(class APlayerController* PlayerController, const FString& StatName)
{
	ULeaderboardQueryCallbackProxy* Proxy = NewObject<ULeaderboardQueryCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->TriggerQuery(PlayerController, StatName, EOnlineKeyValuePairDataType::Int32);
	return Proxy;
}

