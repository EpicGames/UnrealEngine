// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogoutCallbackProxy.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemBPCallHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LogoutCallbackProxy)

ULogoutCallbackProxy::ULogoutCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WorldContextObject(nullptr)
{
}

ULogoutCallbackProxy* ULogoutCallbackProxy::Logout(UObject* WorldContextObject, class APlayerController* PlayerController)
{
	ULogoutCallbackProxy* Proxy = NewObject<ULogoutCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void ULogoutCallbackProxy::Activate()
{
	if (!PlayerControllerWeakPtr.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("A player controller must be provided in order to show the external login UI."), ELogVerbosity::Warning);
		OnFailure.Broadcast(PlayerControllerWeakPtr.Get());
		return;
	}

	FOnlineSubsystemBPCallHelper Helper(TEXT("Logout"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.OnlineSub == nullptr)
	{
		OnFailure.Broadcast(PlayerControllerWeakPtr.Get());
		return;
	}
		
	IOnlineIdentityPtr OnlineIdentity = Helper.OnlineSub->GetIdentityInterface();
	if (!OnlineIdentity.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("Logout: identity functionality not supported by the current online subsystem"), ELogVerbosity::Warning);
		OnFailure.Broadcast(PlayerControllerWeakPtr.Get());
		return;
	}

	const ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerControllerWeakPtr->Player);

	if (LocalPlayer == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Only local players can log out"), ELogVerbosity::Warning);
		OnFailure.Broadcast(PlayerControllerWeakPtr.Get());
		return;
	}
	
	const int32 ControllerId = LocalPlayer->GetControllerId();

	if (!OnlineIdentity->OnLogoutCompleteDelegates[ControllerId].IsBoundToObject(this))
	{
		OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(ControllerId,
			FOnLogoutCompleteDelegate::CreateUObject(this, &ULogoutCallbackProxy::OnLogoutCompleted));
		OnlineIdentity->Logout(ControllerId);
	}
	else
	{
		// already trying to logout!
	}
}

void ULogoutCallbackProxy::OnLogoutCompleted(int LocalPlayerNum, bool bWasSuccessful)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("Logout"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineIdentityPtr OnlineIdentity = Helper.OnlineSub->GetIdentityInterface();
		if (OnlineIdentity.IsValid())
		{
			OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(LocalPlayerNum, OnLogoutCompleteDelegateHandle);
		}
	}

	if (bWasSuccessful)
	{
		OnSuccess.Broadcast(PlayerControllerWeakPtr.Get());
	}
	else
	{
		OnFailure.Broadcast(PlayerControllerWeakPtr.Get());
	}
}

