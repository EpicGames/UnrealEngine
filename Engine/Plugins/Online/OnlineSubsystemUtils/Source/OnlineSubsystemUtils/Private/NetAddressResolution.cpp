// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "NetAddressResolution.h"
#include "IpConnection.h"
#include "Sockets.h"
#include "String/Numeric.h"
#include "Trace/Trace.inl"

// Stats
DECLARE_CYCLE_STAT(TEXT("IpConnection Address Synthesis"), STAT_IpConnection_AddressSynthesis, STATGROUP_Net);


// CVars
#if !UE_BUILD_SHIPPING
TAutoConsoleVariable<FString> CVarNetDebugAddResolverAddress(
	TEXT("net.DebugAppendResolverAddress"),
	TEXT(""),
	TEXT("If this is set, all IP address resolution methods will add the value of this CVAR to the list of results.")
		TEXT("This allows for testing resolution functionality across all multiple addresses with the end goal of having a successful result")
		TEXT("(being the value of this CVAR)"),
	ECVF_Default | ECVF_Cheat);
#endif

static bool GNumericIPSyncResolve = true;

static FAutoConsoleVariableRef CVarNetNumericIPSyncResolve(
	TEXT("net.NumericIPSyncResolve"),
	GNumericIPSyncResolve,
	TEXT("When true (default), numeric IPv4 addresses are resolved synchronously in InitConnect, ")
	TEXT("avoiding the 1-frame async delay that can compound with frame hitches. ")
	TEXT("When false, all addresses go through the async GetAddressInfoAsync path."));

namespace UE::Net::Private
{

/** Lightweight check for standard dotted-decimal IPv4 addresses (e.g., "10.0.0.1").
 *  Only matches the canonical 4-octet form. Shorthand forms like "127.1" are
 *  intentionally not matched and fall through to the async resolution path. */
static bool IsNumericIPv4Address(const FString& Host)
{
	TArray<FString> Tokens;

	if (Host.ParseIntoArray(Tokens, TEXT("."), false) == 4)
	{
		for (const FString& Token : Tokens)
		{
			if (!UE::String::IsNumericOnlyDigits(Token))
			{
				return false;
			}

			const int32 Octet = FCString::Atoi(*Token);
			if (Octet < 0 || Octet > MAX_uint8)
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

/**
 * FNetDriverAddressResolution
 */

bool FNetDriverAddressResolution::InitBindSockets(FCreateAndBindSocketFunc CreateAndBindSocketFunc, EInitBindSocketsFlags Flags,
													ISocketSubsystem* SocketSubsystem, FString& Error)
{
	TArray<TSharedRef<FInternetAddr>> BindAddresses = SocketSubsystem->GetLocalBindAddresses();

	// Handle potentially empty arrays
	if (BindAddresses.Num() == 0)
	{
		Error = TEXT("No binding addresses could be found or grabbed for this platform! Sockets could not be created!");
		return false;
	}

	// Create sockets for every bind address
	for (TSharedRef<FInternetAddr>& BindAddr : BindAddresses)
	{
		FUniqueSocket NewSocket = CreateAndBindSocketFunc(BindAddr, Error);

		if (NewSocket.IsValid())
		{
			UE_LOGF(LogNet, Log, "Created socket for bind address: %ls", ToCStr(BindAddr->ToString(true)));

			BoundSockets.Emplace(NewSocket.Release(), FSocketDeleter(NewSocket.GetDeleter()));
		}
		else
		{
			UE_LOGF(LogNet, Warning, "Could not create socket for bind address %ls, got error %ls", ToCStr(BindAddr->ToString(false)),
					ToCStr(Error));

			Error = TEXT("");

			continue;
		}

		// Servers should only have one socket that they bind on in our code.
		if (EnumHasAnyFlags(Flags, EInitBindSocketsFlags::Server))
		{
			break;
		}
	}

	if (!Error.IsEmpty() || BoundSockets.Num() == 0)
	{
		UE_LOGF(LogNet, Warning, "Encountered an error while creating sockets for the bind addresses. %ls", *Error);
		
		// Make sure to destroy all sockets that we don't end up using.
		BoundSockets.Reset();

		return false;
	}

	return true;
}

bool FNetDriverAddressResolution::InitConnect(UNetConnection* ServerConnection, ISocketSubsystem* SocketSubsystem, const FSocket* ActiveSocket,
												const FURL& ConnectURL)
{
	UIpConnection* IPConnection = CastChecked<UIpConnection>(ServerConnection);
	int32 DestinationPort = ConnectURL.Port;

	if (IPConnection->Resolver->IsAddressResolutionEnabled())
	{
		IPConnection->Resolver->BindSockets = MoveTemp(BoundSockets);

		// Create a weakobj so that we can pass the Connection safely to the lambda for later
		TWeakObjectPtr<UIpConnection> SafeConnectionPtr(IPConnection);

		auto AsyncResolverHandler = [SafeConnectionPtr, ConnectURL](FAddressInfoResult Results)
		{
			FGCScopeGuard Guard;

			// Check if we still have a valid pointer
			if (!SafeConnectionPtr.IsValid())
			{
				// If we got in here, we are already in some sort of exiting state typically.
				// We shouldn't have to do any more other than not do any sort of operations on the connection
				UE_LOGF(LogNet, Warning, "GAI Resolver Lambda: The NetConnection class has become invalid after results for %ls were grabbed.", *Results.QueryHostName);
				return;
			}

			UIpConnection* Connection = SafeConnectionPtr.Get();

			Connection->Resolver->ApplyResolutionResults(Results, Connection, ConnectURL);
		};

		const bool bIsNumericIPv4 = IsNumericIPv4Address(ConnectURL.Host);
		const bool bLooksLikeIPv6 = ConnectURL.Host.Contains(TEXT(":"));
		const TCHAR* AddressType = bIsNumericIPv4 ? TEXT("IPv4") : (bLooksLikeIPv6 ? TEXT("IPv6") : TEXT("Hostname"));

		UE_LOGF(LogNet, Log, "AddressResolution: Begin [Host=%ls] [Port=%d] [Type=%ls]",
			*ConnectURL.Host, DestinationPort, AddressType);

		if (bIsNumericIPv4 && GNumericIPSyncResolve)
		{
			// getaddrinfo() is instant for dotted-decimal IPs (no DNS lookup).
			// Resolving synchronously avoids the 1-frame async delay that can compound
			// with frame hitches (GC, asset loading) on the DeferredRetry frame.
			UE_LOGF(LogNet, Log, "AddressResolution: Numeric IPv4 detected, resolving synchronously [Host=%ls]", *ConnectURL.Host);
			FAddressInfoResult SyncResult = SocketSubsystem->GetAddressInfo(
				*ConnectURL.Host, *FString::Printf(TEXT("%d"), DestinationPort),
				EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses,
				NAME_None, ESocketType::SOCKTYPE_Datagram);

			if (!IPConnection->Resolver->ApplyResolutionResults(SyncResult, IPConnection, ConnectURL, TEXT(" [Sync]")))
			{
				// Connection is already closed by ApplyResolutionResults.
				return false;
			}

			// Immediately consume: assign socket and RemoteAddr so BeginHandshaking()
			// (called later in the same call stack) finds a ready connection.
			if (!IPConnection->TryConsumeAddressResolution(TEXT(" [Sync]")))
			{
				// Socket protocol mismatch or exhausted results.
				// Close the connection to satisfy the documented postcondition
				// (mirrors what Tick() does when CheckAddressResolution returns Error).
				IPConnection->Close(ENetCloseResult::AddressResolutionFailed);
				return false;
			}
		}
		else
		{
			// Async path for hostnames and IPv6
			SocketSubsystem->GetAddressInfoAsync(AsyncResolverHandler, *ConnectURL.Host, *FString::Printf(TEXT("%d"), DestinationPort),
				EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses, NAME_None, ESocketType::SOCKTYPE_Datagram);
		}
	}
	else
	{
		// Clean up any potential multiple sockets we have created when resolution was disabled.
		// InitBase could have created multiple sockets and if so, we'll want to clean them up.
		for (int32 SockIdx=BoundSockets.Num()-1; SockIdx >= 0; SockIdx--)
		{
			if (BoundSockets[SockIdx].Get() != ActiveSocket)
			{
				BoundSockets.RemoveAt(SockIdx);
			}
		}
	}

	return true;
}

void FNetDriverAddressResolution::SetRetrieveTimestamp(bool bRetrieveTimestamp)
{
	for (TSharedPtr<FSocket>& CurSocket : BoundSockets)
	{
		CurSocket->SetRetrieveTimestamp(bRetrieveTimestamp);
	}
}

FNetConnectionAddressResolution* FNetDriverAddressResolution::GetConnectionResolver(UIpConnection* Connection)
{
	return Connection != nullptr ? Connection->Resolver.Get() : nullptr;
}


/**
 * FNetConnectionAddressResolution
 */

bool FNetConnectionAddressResolution::ApplyResolutionResults(const FAddressInfoResult& Results, UIpConnection* Connection, const FURL& ConnectURL, const TCHAR* LogSuffix)
{
	// Fail-fast: some platforms return SE_NO_ERROR with zero results.
	// Catch it here rather than deferring to CheckAddressResolution() with an empty array.
	if (Results.ReturnCode == SE_NO_ERROR && Results.Results.Num() > 0)
	{
		TArray<TSharedRef<FInternetAddr>> AddressResults;
		for (const FAddressInfoResultData& Result : Results.Results)
		{
			AddressResults.Add(Result.Address);
		}

#if !UE_BUILD_SHIPPING
		// Inject debug address if configured
		{
			const FString DebugAddressAddition = CVarNetDebugAddResolverAddress.GetValueOnAnyThread();
			if (!DebugAddressAddition.IsEmpty())
			{
				ISocketSubsystem* SocketSubsystem = Connection->Driver ? Connection->Driver->GetSocketSubsystem() : nullptr;
				if (SocketSubsystem)
				{
					TSharedPtr<FInternetAddr> SpecialResultAddr = SocketSubsystem->GetAddressFromString(DebugAddressAddition);
					if (SpecialResultAddr.IsValid())
					{
						SpecialResultAddr->SetPort(ConnectURL.Port);
						AddressResults.Add(SpecialResultAddr.ToSharedRef());
						UE_LOGF(LogNet, Log, "Added additional result address %ls to resolver list", *SpecialResultAddr->ToString(false));
					}
				}
			}
		}
#endif // !UE_BUILD_SHIPPING

		ResolverResults = MoveTemp(AddressResults);
		ResolutionState = EAddressResolutionState::TryNextAddress;

		return true;
	}
	else
	{
		UE_LOGF(LogNet, Warning, "AddressResolution: Result [Host=%ls] [FAILED] [Error=%d]%ls",
			*Results.QueryHostName, static_cast<int32>(Results.ReturnCode), LogSuffix);

		ResolutionState = EAddressResolutionState::Error;
		Connection->Close(ENetCloseResult::AddressResolutionFailed);

		return false;
	}
}

bool FNetConnectionAddressResolution::InitLocalConnection(ISocketSubsystem* SocketSubsystem, FSocket* InSocket, const FURL& InURL)
{
	bool bValidInit = true;

	// If resolution is disabled, fall back to address synthesis
	if (!IsAddressResolutionEnabled())
	{
		// Figure out IP address from the host URL
		bValidInit = false;

		// Get numerical address directly.
		RemoteAddr = SocketSubsystem->CreateInternetAddr();
		RemoteAddr->SetIp(*InURL.Host, bValidInit);

		// If the protocols do not match, attempt to synthesize the address so they do.
		FName SocketProtocol = InSocket->GetProtocol();

		if ((bValidInit && SocketProtocol != RemoteAddr->GetProtocolType()) || !bValidInit)
		{
			SCOPE_CYCLE_COUNTER(STAT_IpConnection_AddressSynthesis);

			// We want to use GAI to create the address with the correct protocol.
			const FAddressInfoResult MapRequest = SocketSubsystem->GetAddressInfo(*InURL.Host, nullptr,
				EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses, SocketProtocol);

			// Set the remote addr provided we have information.
			if (MapRequest.ReturnCode == SE_NO_ERROR && MapRequest.Results.Num() > 0)
			{
				RemoteAddr = MapRequest.Results[0].Address->Clone();
				bValidInit = true;
			}
			else
			{
				UE_LOGF(LogNet, Warning, "IpConnection::InitConnection: Address protocols do not match and cannot be synthesized to a "
					"similar address, this will likely lead to issues!");
			}
		}
		if (RemoteAddr->IsPortValid(InURL.Port))
		{
			RemoteAddr->SetPort(InURL.Port);
		}

		if (!bValidInit)
		{
			UE_LOGF(LogNet, Verbose, "IpConnection::InitConnection: Unable to resolve %ls", ToCStr(InURL.Host));
		}
	}
	else
	{
		ResolutionState = EAddressResolutionState::WaitingForResolves;
	}

	return bValidInit;
}

EEAddressResolutionHandleResult FNetConnectionAddressResolution::NotifyTimeout()
{
	EEAddressResolutionHandleResult Result = EEAddressResolutionHandleResult::CallerShouldHandle;

	if (CanContinueResolution())
	{
		ResolutionState = EAddressResolutionState::TryNextAddress;
		Result = EEAddressResolutionHandleResult::HandledInternally;
	}

	return Result;
}

EEAddressResolutionHandleResult FNetConnectionAddressResolution::NotifyReceiveError()
{
	EEAddressResolutionHandleResult Result = EEAddressResolutionHandleResult::CallerShouldHandle;

	if (CanContinueResolution())
	{
		ResolutionState = EAddressResolutionState::TryNextAddress;
		Result = EEAddressResolutionHandleResult::HandledInternally;
	}
	else
	{
		ResolutionState = EAddressResolutionState::Error;
	}

	return Result;
}

EEAddressResolutionHandleResult FNetConnectionAddressResolution::NotifySendError()
{
	EEAddressResolutionHandleResult Result = EEAddressResolutionHandleResult::CallerShouldHandle;

	if (CanContinueResolution())
	{
		ResolutionState = EAddressResolutionState::TryNextAddress;
		Result = EEAddressResolutionHandleResult::HandledInternally;
	}
	else
	{
		ResolutionState = EAddressResolutionState::Error;
	}

	return Result;
}

ECheckAddressResolutionResult FNetConnectionAddressResolution::CheckAddressResolution()
{
	ECheckAddressResolutionResult Result = ECheckAddressResolutionResult::None;

	if (ResolutionState == EAddressResolutionState::TryNextAddress)
	{
		if (CurrentAddressIndex >= ResolverResults.Num())
		{
			UE_LOGF(LogNet, Warning, "Exhausted the number of resolver results, closing the connection now.");
			ResolutionState = EAddressResolutionState::Error;
			return ECheckAddressResolutionResult::None;
		}

		RemoteAddr = ResolverResults[CurrentAddressIndex];

		ResolutionSocket.Reset();

		for (const TSharedPtr<FSocket>& BindSocket : BindSockets)
		{
			if (BindSocket->GetProtocol() == RemoteAddr->GetProtocolType())
			{
				ResolutionSocket = BindSocket;
				break;
			}
		}

		if (ResolutionSocket.IsValid())
		{
			ResolutionState = EAddressResolutionState::Connecting;

			if (CurrentAddressIndex == 0)
			{
				Result = ECheckAddressResolutionResult::TryFirstAddress;
			}
			else
			{
				Result = ECheckAddressResolutionResult::TryNextAddress;
			}

			++CurrentAddressIndex;
		}
		else
		{
			UE_LOGF(LogNet, Error, "Unable to find a binding socket for the resolve address result %ls", ToCStr(RemoteAddr->ToString(true)));

			ResolutionState = EAddressResolutionState::Error;
			Result = ECheckAddressResolutionResult::FindSocketError;
		}
	}
	else if (ResolutionState == EAddressResolutionState::Connected)
	{
		ResolutionState = EAddressResolutionState::Done;
		Result = ECheckAddressResolutionResult::Connected;

		CleanupResolutionSockets(ECleanupResolutionSocketsFlags::CleanInactive);
	}
	else if (ResolutionState == EAddressResolutionState::Error)
	{
		UE_LOGF(LogNet, Warning, "Encountered an error, cleaning up this connection now");

		ResolutionState = EAddressResolutionState::Done;
		Result = ECheckAddressResolutionResult::Error;
	}

	return Result;
}

void FNetConnectionAddressResolution::NotifyAddressResolutionConnected()
{
	if (ResolutionState == EAddressResolutionState::Connecting)
	{
		ResolutionState = EAddressResolutionState::Connected;
	}
}

void FNetConnectionAddressResolution::CleanupResolutionSockets(ECleanupResolutionSocketsFlags CleanupFlags
																/*=ECleanupResolutionSocketsFlags::CleanAll*/)
{
	if (IsAddressResolutionEnabled())
	{
		if (EnumHasAnyFlags(CleanupFlags, ECleanupResolutionSocketsFlags::CleanAll))
		{
			ResolutionSocket.Reset();
		}
	
		BindSockets.Reset();
		ResolverResults.Empty();
	}
}

}
