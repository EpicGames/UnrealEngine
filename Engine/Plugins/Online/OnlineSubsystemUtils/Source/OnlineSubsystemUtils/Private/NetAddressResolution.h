// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// Includes
#include "SocketSubsystem.h"

class UIpConnection;
struct FURL;


// Forward declarations
class ISocketSubsystem;
class FSocket;
class FInternetAddr;
class UNetConnection;
class UIpNetDriver;

namespace UE::Net::Private
{

class FNetConnectionAddressResolution;


/**
 * NetDriver callback creating and binding the NetDriver-specific socket
 *
 * @param BindAddr	The bind address for the socket
 * @param Error		If there was an error, specifies the error message
 */
using FCreateAndBindSocketFunc = TUniqueFunction<FUniqueSocket(TSharedRef<FInternetAddr> /*BindAddr*/, FString& /*Error*/)>;

/**
 * Flags specifying the type of call to InitBindSockets
 */
enum class EInitBindSocketsFlags : uint16
{
	Client		= 0x0001,
	Server		= 0x0002
};

ENUM_CLASS_FLAGS(EInitBindSocketsFlags);


/**
 * NetDriver level Address Resolution
 */
class FNetDriverAddressResolution
{
	friend UIpNetDriver;

private:
	/**
	 * Creates and binds sockets for each local bind address
	 *
	 * @param CreateAndBindSocketFunc	The NetDriver callback for creating the actual socket
	 * @param Flags						Flags specifying the type of initialization this is (typically Client vs Server)
	 * @param SocketSubsystem			The NetDriver socket subsystem
	 * @param Error						If there was an error, specifies the error message
	 * @return							Whether or not the call was successful
	 */
	bool InitBindSockets(FCreateAndBindSocketFunc CreateAndBindSocketFunc, EInitBindSocketsFlags Flags, ISocketSubsystem* SocketSubsystem,
							FString& Error);

	/**
	 * Kicks off NetDriver level address resolution for clients connecting to servers (with resolution passing on to FNetConnectionAddressResolution).
	 *
	 * @param ServerConnection		The connection to the server
	 * @param SocketSubsystem		The NetDriver socket subsystem
	 * @param ActiveSocket			Read-only pointer to the active NetDriver socket - in case it is not controlled by address resolution
	 * @param ConnectURL			The URL used to connect to the server
	 * @return						False if synchronous resolution failed immediately (connection is already closed); true otherwise
	 */
	bool InitConnect(UNetConnection* ServerConnection, ISocketSubsystem* SocketSubsystem, const FSocket* ActiveSocket, const FURL& ConnectURL);


	/** Clears all references to sockets held by this resolver. */
	void ClearSockets()
	{
		BoundSockets.Reset();
	}

	/**
	 * Gets the first socket bound by address resolution.
	 *
	 * @return	The first socket bound by address resolution
	 */
	TSharedPtr<FSocket> GetFirstSocket() const
	{
		return BoundSockets.Num() > 0 ? BoundSockets[0] : TSharedPtr<FSocket>();
	}

	/**
	 * Sets the internal socket flag for whether or not timestamps from received packets should be retrieved.
	 *
	 * @param bRetrieveTimestamp	Whether or not timestamps should be retrieved with received packets
	 */
	void SetRetrieveTimestamp(bool bRetrieveTimestamp);

	/**
	 * Gets the NetConnection specific address resolver, for a connection
	 *
	 * @param Connection			The connection whose Resolver we want access to
	 * @return						Returns a pointer to the connection's Resolver
	 */
	static FNetConnectionAddressResolution* GetConnectionResolver(UIpConnection* Connection);

private:
	/**
	 * An array sockets created for every binding address a machine has in use for performing address resolution.
	 * This array empties after connections have been spun up.
	 */
	TArray<TSharedPtr<FSocket>> BoundSockets;
};


/** A state system of the address resolution functionality. */
enum class EAddressResolutionState : uint8
{
	None,					// Undefined
	Disabled,				// Address resolution is explicitly disabled
	WaitingForResolves,		// Address resolution is underway and we're awaiting results
	Connecting,				// Address resolution for a single address is complete and we're attempting to connect (not complete yet)
	TryNextAddress,			// Address resolution for the current address failed, and we're trying the next address on the list
	Connected,				// We have successfully connected to the current address - awaiting completion on next state check
	Done,					// Address resolution has completed
	Error					// There was an error during address resolution
};

/**
 * Result for CheckAddressResolution - partially mirrors EAddressResolutionState
 */
enum class ECheckAddressResolutionResult : uint8
{
	None,				// No action needed
	TryFirstAddress,	// Trying the first address for address resolution
	TryNextAddress,		// Trying a subsequent address for address resolution
	Connected,			// Address resolution was just marked as connected to the current address
	Error,				// There was an error during address resolution
	FindSocketError		// Unable to find a socket that can be used with the current resolve address
};

/**
 * Specifies whether an Address Resolution 'Handle*' function handled the event internally, or whether it should be handled by the caller instead
 */
enum class EEAddressResolutionHandleResult : uint8
{
	HandledInternally,		// The event was handled within the internal address resolution code, and needs no handling by the caller
	CallerShouldHandle		// The event was unhandled, the caller of the 'Handle*' function should handle the event
};

/**
 * CleanupResolutionSockets parameter flags (implemented as flags for extensibility/multiple flags)
 */
enum class ECleanupResolutionSocketsFlags : uint8
{
	CleanAll		= 0x01,	// Clean all sockets, including the active one
	CleanInactive	= 0x02	// Clean only inactive sockets
};

ENUM_CLASS_FLAGS(ECleanupResolutionSocketsFlags);


/**
 * NetConnection level Address Resolution
 *
 * NOTE: Private interface is accessible to the NetConnection, the public interface is only for the NetDriver
 */
class FNetConnectionAddressResolution
{
	friend UIpConnection;
	friend FNetDriverAddressResolution;

public:
	/**
	 * Checks to see if the owner NetConnection class can use address resolution
	 *
	 * @return if address resolution is allowed to continue processing.
	 */
	bool IsAddressResolutionEnabled() const
	{
		return ResolutionState != EAddressResolutionState::Disabled;
	}

	/**
	 * Whether or not address resolution has completed.
	 *
	 * @return		Whether or not address resolution has completed
	 */
	bool IsAddressResolutionComplete() const
	{
		return ResolutionState == EAddressResolutionState::Done;
	}

	/**
	 * Whether or not address resolution has failed
	 *
	 * @return	Returns whether or not address resolution has failed
	 */
	bool HasAddressResolutionFailed() const
	{
		return ResolutionState == EAddressResolutionState::Error;
	}

	/**
	 * Whether or not address resolution has pending results ready to be attempted
	 * (i.e., the state machine is in TryNextAddress). CheckAddressResolution() may still
	 * fail to yield an address (e.g., FindSocketError if no socket matches the protocol).
	 *
	 * @return		Whether or not there are pending address results to attempt
	 */
	bool HasPendingAddressResults() const
	{
		return ResolutionState == EAddressResolutionState::TryNextAddress;
	}

	/**
	 * Cleans up the socket information in use with resolution. This can get called numerous times.
	 *
	 * @param CleanupFlags		Flags specifying what should be cleaned up
	 */
	void CleanupResolutionSockets(ECleanupResolutionSocketsFlags CleanupFlags=ECleanupResolutionSocketsFlags::CleanAll);

private:
	/**
	 * Initializes Address resolution for clients connecting to a server.
	 *
	 * @param SocketSubsystem	The NetConnection socket subsystem
	 * @param InSocket			When address resolution is disabled, the socket being used for connecting.
	 * @param InURL				The URL the client is connecting to
	 * @return					Whether or not address resolution kicked off successfully
	 */
	bool InitLocalConnection(ISocketSubsystem* SocketSubsystem, FSocket* InSocket, const FURL& InURL);

	/**
	 * Gives address resolution a chance to override a NetConnection timeout.
	 *
	 * @return		Whether or not the timeout was handled internally by address resolution, or whether it should be handled by the NetConnection
	 */
	EEAddressResolutionHandleResult NotifyTimeout();

	/**
	 * Gives address resolution a chance to override NetConnection handling of packet/socket receive errors.
	 *
	 * @return		Whether or not the error was handled internally by address resolution, or whether it should be handled by the NetConnection
	 */
	EEAddressResolutionHandleResult NotifyReceiveError();

	/**
	 * Gives address resolution a chance to override NetConnection handling of socket send errors.
	 *
	 * @return		Whether or not the error was handled internally by address resolution, or whether it should be handled by the NetConnection
	 */
	EEAddressResolutionHandleResult NotifySendError();

	/**
	 * Checks the state of address resolution, and kicks off next stage if applicable - returning the result/action taken or current status update.
	 *
	 * @return		Returns the result/action or the current status update.
	 */
	ECheckAddressResolutionResult CheckAddressResolution();

	/**
	 * Notification from the NetConnection that address resolution has connected (that a packet was sucessfully received)
	 */
	void NotifyAddressResolutionConnected();

	/**
	 * Determines if we can continue processing resolution results or not based on flags and current flow.
	 *
	 * @return if resolution is allowed to continue processing.
	 */
	bool CanContinueResolution() const {
		return CurrentAddressIndex < ResolverResults.Num() && IsAddressResolutionEnabled() &&
			ResolutionState != EAddressResolutionState::Error && ResolutionState != EAddressResolutionState::Done;
	}

	/**
	 * Applies resolved addresses to the connection's resolver. On failure, closes the connection.
	 * In non-shipping builds, also injects any debug address from net.DebugAppendResolverAddress.
	 *
	 * @param Results		The address info result from GetAddressInfo/GetAddressInfoAsync
	 * @param Connection	The owning connection (used for logging, error handling, and socket subsystem access)
	 * @param ConnectURL	The original connection URL
	 * @param LogSuffix		Optional suffix for log lines (e.g., " [Sync]")
	 * @return				True if results were applied successfully, false on error
	 */
	bool ApplyResolutionResults(const FAddressInfoResult& Results, UIpConnection* Connection, const FURL& ConnectURL, const TCHAR* LogSuffix = TEXT(""));

	/**
	 * Disables address resolution by pushing the disabled flag into the status field.
	 */
	void DisableAddressResolution()
	{
		ResolutionState = EAddressResolutionState::Disabled;
	}

	/**
	 * Whether or not address resolution is in the process of attempting to connect to the server using the current address
	 *
	 * @return		Whether or not a connection attempt is underway
	 */
	bool IsAddressResolutionConnecting() const
	{
		return ResolutionState == EAddressResolutionState::Connecting;
	}

	/**
	 * Whether or not address resolution is requesting that packet sends should be blocked.
	 *
	 * @return		Whether or not to block packet sends
	 */
	bool ShouldBlockSend() const
	{
		return IsAddressResolutionEnabled() &&
			(ResolutionState == EAddressResolutionState::WaitingForResolves || ResolutionState == EAddressResolutionState::TryNextAddress);
	}

	/**
	 * Gets the currently resolved socket (used as the primary NetConnection socket).
	 *
	 * @return		Returns the currently resolved socket
	 */
	TSharedPtr<FSocket> GetResolutionSocket() const
	{
		return ResolutionSocket;
	}

	/**
	 * Gets the remote address of the currently resolved socket.
	 *
	 * @return		The remote address of the currently resolved socket.
	 */
	TSharedPtr<FInternetAddr> GetRemoteAddr() const
	{
		return RemoteAddr;
	}


private:
	/** An array of sockets tied to every binding address. */
	TArray<TSharedPtr<FSocket>> BindSockets;

	/** Holds a refcount to the actual socket to be used from BindSockets. */
	TSharedPtr<FSocket> ResolutionSocket;

	/** Stores the remote address from socket resolution */
	TSharedPtr<FInternetAddr> RemoteAddr;

	/** An array containing the address results GAI returns for the current host value. Given to us from the netdriver. */
	TArray<TSharedRef<FInternetAddr>> ResolverResults;

	/** The index into the ResolverResults that we're currently attempting */
	int32 CurrentAddressIndex = 0;

	/** 
	 *  The connection's current status of where it is in the resolution state machine.
	 *  If a platform should not use resolution, call DisableAddressResolution() in your constructor
	 */
	EAddressResolutionState ResolutionState = EAddressResolutionState::None;
};
}
