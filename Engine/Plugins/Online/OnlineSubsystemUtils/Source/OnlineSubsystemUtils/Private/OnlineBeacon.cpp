// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeacon.h"
#include "Engine/Channel.h"
#include "Engine/NetConnection.h"
#include "Engine/Engine.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeacon)

DEFINE_LOG_CATEGORY(LogBeacon);

AOnlineBeacon::AOnlineBeacon(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	NetDriver(nullptr),
	BeaconState(EBeaconState::DenyRequests)
{
	NetDriverName = FName(TEXT("BeaconDriver"));
	NetDriverDefinitionName = NAME_BeaconNetDriver;
	bRelevantForNetworkReplays = false;
}

bool AOnlineBeacon::InitBase()
{
	NetDriver = GEngine->CreateNetDriver(GetWorld(), NetDriverDefinitionName);
	if (NetDriver != nullptr)
	{
		HandleNetworkFailureDelegateHandle = GEngine->OnNetworkFailure().AddUObject(this, &AOnlineBeacon::HandleNetworkFailure);
		SetNetDriverName(NetDriver->NetDriverName);
		return true;
	}

	return false;
}

void AOnlineBeacon::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	CleanupNetDriver();
	Super::EndPlay(EndPlayReason);
}

bool AOnlineBeacon::HasNetOwner() const
{
    // Beacons are their own net owners
	return true;
}

void AOnlineBeacon::DestroyBeacon()
{
	UE_LOGF(LogBeacon, Verbose, "Destroying beacon %ls, netdriver %ls", *GetName(), NetDriver ? *NetDriver->GetDescription() : TEXT("NULL"));
	GEngine->OnNetworkFailure().Remove(HandleNetworkFailureDelegateHandle);

	CleanupNetDriver();
	Destroy();
}

void AOnlineBeacon::CleanupNetDriver()
{
	if (NetDriver)
	{
		if (NetDriver->GetWorld())
		{
			UE_LOGF(LogBeacon, Log, "OnlineBeacon %ls requesting net driver destruction: %ls", *GetName(), *NetDriver->GetDescription());
			NetDriver->RequestNetDriverDestruction();
		}
		else
		{
			UE_LOGF(LogBeacon, Log, "OnlineBeacon %ls immediately destroying netdriver: %ls",*GetName(), *NetDriver->GetDescription());
			GEngine->DestroyNamedNetDriver(GetWorld(), NetDriverName);
		}

		// If the net connection is currently in the middle of processing messages it is
		// possible for the notifier to be fired again if not cleared.
		NetDriver->Notify = nullptr;
		NetDriver = nullptr;
	}
}

void AOnlineBeacon::HandleNetworkFailure(UWorld *World, UNetDriver *InNetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString)
{
	if (InNetDriver && InNetDriver->NetDriverName == NetDriverName)
	{
		UE_LOGF(LogBeacon, Verbose, "NetworkFailure %ls: %ls", *GetName(), ENetworkFailure::ToString(FailureType));
		OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_network_failure"));
	}
}

void AOnlineBeacon::OnFailure()
{
	GEngine->OnNetworkFailure().Remove(HandleNetworkFailureDelegateHandle);
	CleanupNetDriver();
}

void AOnlineBeacon::OnFailure(EBeaconFailureReason Reason, FStringView ErrorMessage)
{
	UE_LOGF(LogBeacon, Verbose, "Online Beacon Failure: %ls. %ls", LexToString(Reason), ErrorMessage.IsEmpty() ? TEXT("") : ErrorMessage.GetData());
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnFailure();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void AOnlineBeacon::OnActorChannelOpen(FInBunch& Bunch, UNetConnection* Connection)
{
	Connection->OwningActor = this;
	Super::OnActorChannelOpen(Bunch, Connection);

	{
		// Enable replication
		if (UReplicationSystem* ReplicationSystem = Connection->Driver->GetReplicationSystem())
		{
			ReplicationSystem->SetReplicationEnabledForConnection(Connection->GetConnectionHandle().GetParentConnectionId(), true);
		}
	}
}

bool AOnlineBeacon::IsRelevancyOwnerFor(const AActor* ReplicatedActor, const AActor* ActorOwner, const AActor* ConnectionActor) const
{
	bool bRelevantOwner = (ConnectionActor == ReplicatedActor);
	return bRelevantOwner;
}

bool AOnlineBeacon::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	// Only replicate to the owner or to connections of the same beacon type (possible that multiple UNetConnections come from the same client)
	bool bIsOwner = GetNetConnection() == ViewTarget->GetNetConnection();
	bool bSameBeaconType = GetClass() == RealViewer->GetClass();
	return bOnlyRelevantToOwner ? bIsOwner : bSameBeaconType;
}

EAcceptConnection::Type AOnlineBeacon::NotifyAcceptingConnection()
{
	check(NetDriver);
	if(NetDriver->ServerConnection)
	{
		// We are a client and we don't welcome incoming connections.
		UE_LOGF(LogNet, Log, "NotifyAcceptingConnection: Client refused");
		return EAcceptConnection::Reject;
	}
	else if(BeaconState == EBeaconState::DenyRequests)
	{
		// Server is down
		UE_LOGF(LogNet, Log, "NotifyAcceptingConnection: Server %ls refused", *GetName());
		return EAcceptConnection::Reject;
	}
	else //if(BeaconState == EBeaconState::AllowRequests)
	{
		// Server is up and running.
		UE_CLOGF(!NetDriver->DDoS.CheckLogRestrictions(), LogNet, Log, "NotifyAcceptingConnection: Server %ls accept", *GetName());
		return EAcceptConnection::Accept;
	}
}

void AOnlineBeacon::NotifyAcceptedConnection(UNetConnection* Connection)
{
	check(NetDriver != nullptr);
	check(NetDriver->ServerConnection == nullptr);
	UE_LOGF(LogNet, Log, "NotifyAcceptedConnection: Name: %ls, TimeStamp: %ls, %ls", *GetName(), FPlatformTime::StrTimestamp(), *Connection->Describe());
}

bool AOnlineBeacon::NotifyAcceptingChannel(UChannel* Channel)
{
	check(Channel);
	check(Channel->Connection);
	check(Channel->Connection->Driver);
	UNetDriver* Driver = Channel->Connection->Driver;
	check(NetDriver == Driver);

	if (Driver->ServerConnection)
	{
		// We are a client and the server has just opened up a new channel.
		UE_LOGF(LogNet, Log, "NotifyAcceptingChannel %i/%ls client %ls", Channel->ChIndex, *Channel->ChName.ToString(), *GetName());
		if (Driver->ChannelDefinitionMap[Channel->ChName].bServerOpen)
		{
			UE_LOGF(LogNet, Log, "Client accepting %ls channel", *Channel->ChName.ToString());
			return true;
		}
		else
		{
			// Unwanted channel type.
			UE_LOGF(LogNet, Log, "Client refusing unwanted channel of type %ls", *Channel->ChName.ToString());
			return false;
		}
	}
	else
	{
		// We are the server.
		if (Driver->ChannelDefinitionMap[Channel->ChName].bClientOpen)
		{
			// The client has opened initial channel.
			UE_LOGF(LogNet, Log, "NotifyAcceptingChannel Control %i server %ls: Accepted", Channel->ChIndex, *GetFullName());
			return true;
		}
		else
		{
			// Client can't open any other kinds of channels.
			UE_LOGF(LogNet, Log, "NotifyAcceptingChannel %ls %i server %ls: Refused", *Channel->ChName.ToString(), Channel->ChIndex, *GetFullName());
			return false;
		}
	}
}

void AOnlineBeacon::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch)
{
}

