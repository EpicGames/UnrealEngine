// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconClient.h"
#include "Engine/GameInstance.h"
#include "PacketHandler.h"
#include "TimerManager.h"
#include "OnlineBeaconHostObject.h"
#include "Engine/PackageMapClient.h"
#include "Engine/LocalPlayer.h"
#include "Net/DataChannel.h"
#include "OnlineSubsystemUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconClient)

/** For backwards compatibility with newer engine encryption API */
#ifndef NETCONNECTION_HAS_SETENCRYPTIONKEY
	#define NETCONNECTION_HAS_SETENCRYPTIONKEY 0
#endif

AOnlineBeaconClient::AOnlineBeaconClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	BeaconOwner(nullptr),
	BeaconConnection(nullptr),
	ConnectionState(EBeaconConnectionState::Invalid)
{
	NetDriverName = FName(TEXT("BeaconDriverClient"));
	bOnlyRelevantToOwner = true;

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bAllowTickOnDedicatedServer = false;
	PrimaryActorTick.bStartWithTickEnabled = true;
}

FString AOnlineBeaconClient::GetBeaconType() const
{
	return GetClass()->GetName();
}

AOnlineBeaconHostObject* AOnlineBeaconClient::GetBeaconOwner() const
{
	return BeaconOwner;
}

void AOnlineBeaconClient::SetBeaconOwner(AOnlineBeaconHostObject* InBeaconOwner)
{
	BeaconOwner = InBeaconOwner;
}

const AActor* AOnlineBeaconClient::GetNetOwner() const
{
	return BeaconOwner;
}

class UNetConnection* AOnlineBeaconClient::GetNetConnection() const
{
	return BeaconConnection;
}

bool AOnlineBeaconClient::DestroyNetworkActorHandled()
{
	// AOnlineBeacon::NetDriver may have already been nulled out in AOnlineBeacon::CleanupNetDriver
	const UNetDriver* LocalNetDriver = BeaconConnection ? BeaconConnection->Driver : nullptr;

	if (BeaconConnection && BeaconConnection->GetConnectionState() != USOCK_Closed &&
		LocalNetDriver && !LocalNetDriver->GetPendingDestruction())
	{
		// This will be cleaned up in ~2 sec by UNetConnection Tick
		BeaconConnection->bPendingDestroy = true;
		return true;
	}

	// The UNetConnection is gone or has been closed (NetDriver destroyed) and needs to go away now
	return false;
}

const FUniqueNetIdRepl& AOnlineBeaconClient::GetUniqueId() const
{
	if (BeaconConnection)
	{
		return BeaconConnection->PlayerId;
	}

	static FUniqueNetIdRepl EmptyId;
	return EmptyId;
}

EBeaconConnectionState AOnlineBeaconClient::GetConnectionState() const
{
	return ConnectionState;
}

void AOnlineBeaconClient::SetConnectionState(EBeaconConnectionState NewConnectionState)
{
	ConnectionState = NewConnectionState;
}

FString AOnlineBeaconClient::GetLoginOptions(const FUniqueNetIdRepl& PlayerId)
{
	const FString& AuthTicket = GetAuthTicketInternal(*PlayerId);

	FURL URL(nullptr, TEXT(""), TRAVEL_Absolute);
	if (!AuthTicket.IsEmpty())
	{
		URL.AddOption(*FString::Printf(TEXT("AuthTicket=%s"), *AuthTicket));
	}
	return URL.ToString();
}

namespace
{
FUniqueNetIdRepl GetDedicatedServerAccountId(UWorld* World)
{
	FUniqueNetIdRepl AccountId;
	IOnlineIdentityPtr IdentityPtr = Online::GetIdentityInterface(World);
	if (IdentityPtr.IsValid())
	{
		AccountId = IdentityPtr->GetUniquePlayerId(DEDICATED_SERVER_USER_INDEX);
	}
	return AccountId;
}

FUniqueNetIdRepl GetFirstPlayerAccountId(UWorld* World)
{
	FUniqueNetIdRepl AccountId;
	// Prefer local player's preferred id
	if (ULocalPlayer* LocalPlayer = GEngine->GetFirstGamePlayer(World))
	{
		AccountId = LocalPlayer->GetPreferredUniqueNetId();
	}
	else
	{
		// Fall back to querying the identity interface.
		if (IOnlineIdentityPtr IdentityPtr = Online::GetIdentityInterface(World))
		{
			TArray<TSharedPtr<FUserOnlineAccount> > UserAccounts = IdentityPtr->GetAllUserAccounts();
			if (!UserAccounts.IsEmpty())
			{
				AccountId = UserAccounts[0]->GetUserId();
			}
		}
	}
	return AccountId;
}
}

bool AOnlineBeaconClient::InitClient(const FOnlineBeaconClientInitParams& Params)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineBeaconClient_InitClient);

	bool bSuccess = false;

	if(Params.URL.Valid)
	{
		if (InitBase() && NetDriver)
		{
			NetDriver->SetWorld(GetWorld());
			NetDriver->SetReplicateTransactionally(Params.bReplicateTransactionally);

			FString Error;
			if (NetDriver->InitConnect(this, Params.URL, Error))
			{
				check(NetDriver->ServerConnection);
				UWorld* World = GetWorld();
				if (World)
				{
					BeaconConnection = NetDriver->ServerConnection;

					if (IsRunningDedicatedServer())
					{
						BeaconConnection->PlayerId = GetDedicatedServerAccountId(World);
					}
					else
					{
						// Send the player unique Id at login
						BeaconConnection->PlayerId = GetFirstPlayerAccountId(World);
					}

#if NETCONNECTION_HAS_SETENCRYPTIONKEY
					if (!EncryptionData.Identifier.IsEmpty())
					{
						BeaconConnection->SetEncryptionData(EncryptionData);
					}
#endif

					SetConnectionState(EBeaconConnectionState::Pending);

					// Kick off the connection handshake
					bool bSentHandshake = false;

					if (BeaconConnection->Handler.IsValid())
					{
						BeaconConnection->Handler->BeginHandshaking(
							FPacketHandlerHandshakeComplete::CreateUObject(this, &AOnlineBeaconClient::SendInitialJoin));

						bSentHandshake = true;
					}

					if (NetDriver)
					{
						NetDriver->Notify = this;
						NetDriver->InitialConnectTimeout = BeaconConnectionInitialTimeout;
						NetDriver->ConnectionTimeout = BeaconConnectionTimeout;

						if (!bSentHandshake)
						{
							SendInitialJoin();
						}

						bSuccess = true;
					}
					else
					{
						// an error must have occurred during BeginHandshaking
						UE_LOGF(LogBeacon, Warning, "AOnlineBeaconClient::InitClient BeginHandshaking failed");

						// if the connection is still pending, notify of failure
						if (GetConnectionState() == EBeaconConnectionState::Pending)
						{
							SetConnectionState(EBeaconConnectionState::Invalid);
							AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_client_init_handshake_failure"));
						}
					}
				}
				else
				{
					// error initializing the network stack, no world...
					UE_LOGF(LogBeacon, Log, "AOnlineBeaconClient::InitClient failed - no world");
					SetConnectionState(EBeaconConnectionState::Invalid);
					AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_client_init_no_world"));
				}
			}
			else
			{
				// error initializing the network stack...
				UE_LOGF(LogBeacon, Log, "AOnlineBeaconClient::InitClient failed");
				SetConnectionState(EBeaconConnectionState::Invalid);
				AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_client_init_failure"));
			}
		}
	}

	return bSuccess;
}

bool AOnlineBeaconClient::InitClient(FURL& URL)
{
	FOnlineBeaconClientInitParams Params;
	Params.URL = URL;
	return InitClient(Params);
}

void AOnlineBeaconClient::Tick(float DeltaTime)
{
	if (NetDriver != nullptr && NetDriver->ServerConnection != nullptr)
	{
		// Monitor for close bunches sent by the server which close down the connection in UChannel::Cleanup
		// See similar code in UWorld::TickNetClient
		if (((ConnectionState == EBeaconConnectionState::Pending) || (ConnectionState == EBeaconConnectionState::Open)) &&
			(NetDriver->ServerConnection->GetConnectionState() == USOCK_Closed))
		{
			UE_LOGF(LogBeacon, Verbose, "Client beacon (%ls) socket has closed, triggering failure.", *GetName());
			AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_connection_closed_remotely"));
		}
	}
}

void AOnlineBeaconClient::SetEncryptionData(const FEncryptionData& InEncryptionData)
{
	if (CVarNetAllowEncryption.GetValueOnGameThread() != 0)
	{
		EncryptionData = InEncryptionData;
	}
}

void AOnlineBeaconClient::SendInitialJoin()
{
	UNetConnection* ServerConn = NetDriver != nullptr ? NetDriver->ServerConnection : nullptr;

	if (ensure(ServerConn != nullptr))
	{
		uint8 IsLittleEndian = uint8(PLATFORM_LITTLE_ENDIAN);
		check(IsLittleEndian == !!IsLittleEndian); // should only be one or zero

		const int32 AllowEncryption = CVarNetAllowEncryption.GetValueOnGameThread();
		uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();

		if (AllowEncryption == 0)
		{
			EncryptionData.Identifier.Empty();
		}

		bool bEncryptionRequirementsFailure = false;

		if (EncryptionData.Identifier.IsEmpty())
		{
			EEncryptionFailureAction FailureResult = EEncryptionFailureAction::Default;

			if (FNetDelegates::OnReceivedNetworkEncryptionFailure.IsBound())
			{
				FailureResult = FNetDelegates::OnReceivedNetworkEncryptionFailure.Execute(ServerConn);
			}

			const bool bGameplayDisableEncryptionCheck = FailureResult == EEncryptionFailureAction::AllowConnection;

			bEncryptionRequirementsFailure = NetDriver->IsEncryptionRequired() && !bGameplayDisableEncryptionCheck;
		}

		if (!bEncryptionRequirementsFailure)
		{
			EEngineNetworkRuntimeFeatures LocalNetworkFeatures = NetDriver->GetNetworkRuntimeFeatures();
			FNetControlMessage<NMT_Hello>::Send(NetDriver->ServerConnection, IsLittleEndian, LocalNetworkVersion, EncryptionData.Identifier, LocalNetworkFeatures);

			ServerConn->FlushNet();
		}
		else if (GetConnectionState() == EBeaconConnectionState::Pending)
		{
			UE_LOGF(LogNet, Error, "AOnlineBeaconClient::SendInitialJoin: EncryptionToken is empty when 'net.AllowEncryption' requires it.");

			SetConnectionState(EBeaconConnectionState::Invalid);
			Super::OnFailure(EBeaconFailureReason::EncryptionFailure, TEXTVIEW("beacon_client_empty_encryption_token"));
		}
	}
}

void AOnlineBeaconClient::OnFailure()
{
	UE_LOGF(LogBeacon, Verbose, "Client beacon (%ls) connection failure, handling connection timeout.", *GetName());
	SetConnectionState(EBeaconConnectionState::Invalid);
	HostConnectionFailure.ExecuteIfBound();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Super::OnFailure();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


/// @cond DOXYGEN_WARNINGS

void AOnlineBeaconClient::ClientOnConnected_Implementation()
{
	UE_LOGF(LogBeacon, Verbose, "%ls: Received first RPC from server", *GetName());

	SetConnectionState(EBeaconConnectionState::Open);
	BeaconConnection->SetConnectionState(USOCK_Open);

	SetRole(ROLE_Authority);
	SetReplicates(true);
	SetAutonomousProxy(true);

	// Fail safe for connection to server but no client connection RPC
	GetWorldTimerManager().ClearTimer(TimerHandle_OnFailure);

	// Call the overloaded function for this client class
	OnConnected();
}

/// @endcond

bool AOnlineBeaconClient::UseShortConnectTimeout() const
{
	return ConnectionState == EBeaconConnectionState::Open;
}

void AOnlineBeaconClient::DestroyBeacon()
{
	SetConnectionState(EBeaconConnectionState::Closed);
	SetActorTickEnabled(false);

	UWorld* World = GetWorld();
	if (World)
	{
		// Fail safe for connection to server but no client connection RPC
		GetWorldTimerManager().ClearTimer(TimerHandle_OnFailure);
	}

	Super::DestroyBeacon();
}

void AOnlineBeaconClient::OnNetCleanup(UNetConnection* Connection)
{
	// During the garbage collection of UNetConnection OnNetCleanup may be triggered.
	// When this happens BeaconConnection will have already been set to nullptr by GC.
	ensure(NetDriver == nullptr || Connection == BeaconConnection);
	SetConnectionState(EBeaconConnectionState::Closed);

	AOnlineBeaconHostObject* BeaconHostObject = GetBeaconOwner();
	if (BeaconHostObject)
	{
		BeaconHostObject->NotifyClientDisconnected(this);
	}

	BeaconConnection = nullptr;
	Destroy(true);
}

void AOnlineBeaconClient::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch)
{
	if(NetDriver->ServerConnection)
	{
		check(Connection == NetDriver->ServerConnection);

		// We are the client
#if !(UE_BUILD_SHIPPING && WITH_EDITOR)
		UE_LOGF(LogBeacon, Log, "%ls[%ls] Client received: %ls", *GetName(), *Connection->GetName(), FNetControlMessageInfo::GetName(MessageType));
#endif
		switch (MessageType)
		{
		case NMT_EncryptionAck:
			{
				// Enable encryption using the beacons encryption data when set.
				if (!EncryptionData.Identifier.IsEmpty())
				{
					FEncryptionKeyResponse Response;
					Response.Response = EEncryptionResponse::Success;
					Response.EncryptionData = EncryptionData;
					FinalizeEncryptedConnection(Response, Connection);
				}
				else
				{
					if (FNetDelegates::OnReceivedNetworkEncryptionAck.IsBound())
					{
						TWeakObjectPtr<UNetConnection> WeakConnection = Connection;
						FNetDelegates::OnReceivedNetworkEncryptionAck.Execute(FOnEncryptionKeyResponse::CreateUObject(this, &ThisClass::FinalizeEncryptedConnection, WeakConnection));
					}
					else
					{
						// Force close the session
						UE_LOGF(LogBeacon, Warning, "%ls: No delegate available to handle encryption ack, disconnecting.", *Connection->GetName());
						AOnlineBeacon::OnFailure(EBeaconFailureReason::EncryptionFailure, TEXTVIEW("beacon_client_encryption_delegate_missing"));
					}
				}
				break;
			}
		case NMT_Challenge:
		{
			// Challenged by server.
			if (FNetControlMessage<NMT_Challenge>::Receive(Bunch, Connection->Challenge))
			{
				// build a URL
				FURL URL(nullptr, TEXT(""), TRAVEL_Absolute);
				const FString& LoginOptions = GetLoginOptions(Connection->PlayerId);
				if (!LoginOptions.IsEmpty())
				{
					URL.AddOption(*LoginOptions);
				}
				FString URLString = URL.ToString();

				// compute the player's online platform name
				FName OnlinePlatformName = NAME_None;
				if (const FWorldContext* const WorldContext = GEngine->GetWorldContextFromWorld(GetWorld()))
				{
					if (WorldContext->OwningGameInstance)
					{
						OnlinePlatformName = WorldContext->OwningGameInstance->GetOnlinePlatformName();
					}
				}
				FString OnlinePlatformNameString = OnlinePlatformName.ToString();

				// send NMT_Login
				Connection->ClientResponse = TEXT("0");
				FNetControlMessage<NMT_Login>::Send(Connection, Connection->ClientResponse, URLString, Connection->PlayerId, OnlinePlatformNameString);
				Connection->FlushNet();
			}
			else
			{
				// Force close the session
				UE_LOGF(LogBeacon, Warning, "%ls: Unable to parse challenge request message.", *Connection->GetName());
				AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_parse_challenge_error"));
			}
			break;
		}
		case NMT_BeaconWelcome:
			{
				Connection->ClientResponse = TEXT("0");
				FNetControlMessage<NMT_Netspeed>::Send(Connection, Connection->CurrentNetSpeed);

				FString BeaconType = GetBeaconType();
				if (!BeaconType.IsEmpty())
				{
					FNetControlMessage<NMT_BeaconJoin>::Send(Connection, BeaconType, Connection->PlayerId);
					Connection->FlushNet();
				}
				else
				{
					// Force close the session
					UE_LOGF(LogBeacon, Log, "Beacon close from invalid beacon type");
					AOnlineBeacon::OnFailure(EBeaconFailureReason::VersionMismatch, TEXTVIEW("beacon_client_invalid_beacon_type"));
				}
				break;
			}
		case NMT_BeaconAssignGUID:
			{
				UE::Net::FNetIDVariant NetID;

				if (FNetControlMessage<NMT_BeaconAssignGUID>::Receive(Bunch, NetID))
				{
					if (NetID.IsValid())
					{
						Connection->Driver->AssignNetIDToObject(NetID, this);

						FString BeaconType = GetBeaconType();
						FNetControlMessage<NMT_BeaconNetGUIDAck>::Send(Connection, BeaconType);
						Connection->FlushNet();
						// Server will send ClientOnConnected() when it gets this control message

						// Fail safe for connection to server but no client connection RPC
						if (!Connection->Driver->bNoTimeouts)
						{
							float Timeout = Connection->GetTimeoutValue();
							UE_LOGF(LogBeacon, Verbose, "%ls: Waiting for first RPC from server, timeout: %.3f", *GetName(), Timeout);
							FTimerDelegate TimerDelegate;
							TimerDelegate.BindWeakLambda(this, [this]() mutable
								{
									this->AOnlineBeacon::OnFailure(EBeaconFailureReason::TimeOut, TEXTVIEW("beacon_client_rpc_timeout"));
								});
							
							GetWorldTimerManager().SetTimer(TimerHandle_OnFailure, TimerDelegate, Timeout, false);
						}
					}
					else
					{
						// Force close the session
						UE_LOGF(LogBeacon, Log, "Beacon close from invalid NetGUID");
						AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_invalid_netguid"));
					}
				}

				break;
			}
		case NMT_Upgrade:
			{
				// Report mismatch.
				uint32 RemoteNetworkVersion = 0;
				EEngineNetworkRuntimeFeatures RemoteNetworkFeatures = EEngineNetworkRuntimeFeatures::None;

				if (FNetControlMessage<NMT_Upgrade>::Receive(Bunch, RemoteNetworkVersion, RemoteNetworkFeatures))
				{
					const bool bUpgradeSuccess = Connection->HandleReceiveNetUpgrade(RemoteNetworkVersion, RemoteNetworkFeatures);

					if (bUpgradeSuccess)
					{
						ensureMsgf(false, TEXT("Beacons don't support any NetworkFeature trait yet. Upgrades shouldn't happen."));
						//todo: retrigger an handshake via Join message
					}
					else
					{
						// Should the beacon disconnect here ?
					}
				}

				break;
			}
		case NMT_Failure:
			{
				FString ErrorMsg;

				if (FNetControlMessage<NMT_Failure>::Receive(Bunch, ErrorMsg))
				{
					if (ErrorMsg.IsEmpty())
					{
						ErrorMsg = NSLOCTEXT("NetworkErrors", "GenericBeaconConnectionFailed", "Beacon Connection Failed.").ToString();
					}

					// Force close the session
					UE_LOGF(LogBeacon, Log, "Beacon close from NMT_Failure %ls", *ErrorMsg);
					AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_nmt_failure"));
				}

				break;
			}
		case NMT_BeaconJoin:
		case NMT_BeaconNetGUIDAck:
		default:
			{
				// Force close the session
				UE_LOGF(LogBeacon, Log, "Beacon close from unexpected control message");
				AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_unexpected_control_message"));
				break;
			}
		}
	}	
}

FString AOnlineBeaconClient::GetAuthTicketInternal(const FUniqueNetIdRepl& PlayerId)
{
	FString AuthTicket;

	IOnlineIdentityPtr IdentityPtr = Online::GetIdentityInterface(GetWorld());
	if (IdentityPtr.IsValid() && ensure(PlayerId.IsValid()))
	{
		TSharedPtr<FUserOnlineAccount> UserAcct = IdentityPtr->GetUserAccount(*PlayerId);
		if (UserAcct.IsValid())
		{
			AuthTicket = *UserAcct->GetAccessToken();
		}
	}

	return AuthTicket;
}

void AOnlineBeaconClient::FinalizeEncryptedConnection(const FEncryptionKeyResponse& Response, TWeakObjectPtr<UNetConnection> WeakConnection)
{
	UNetConnection* Connection = WeakConnection.Get();
	if (Connection)
	{
		if (Connection->GetConnectionState() != USOCK_Invalid && Connection->GetConnectionState() != USOCK_Closed && Connection->Driver)
		{
			if (Response.Response == EEncryptionResponse::Success)
			{
				Connection->EnableEncryption(Response.EncryptionData);
			}
			else
			{
				FString ResponseStr(LexToString(Response.Response));
				UE_LOGF(LogBeacon, Warning, "AOnlineBeaconClient::FinalizeEncryptedConnection: encryption failure [%ls] %ls", *ResponseStr, *Response.ErrorMsg);
				AOnlineBeacon::OnFailure(EBeaconFailureReason::EncryptionFailure, TEXTVIEW("beacon_client_encryption_failure"));
			}
		}
		else
		{
			UE_LOGF(LogBeacon, Warning, "AOnlineBeaconClient::FinalizeEncryptedConnection: connection in invalid state. %ls", *Connection->Describe());
			AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_invalid_state_connection"));
		}
	}
	else
	{
		UE_LOGF(LogBeacon, Warning, "AOnlineBeaconClient::FinalizeEncryptedConnection: Connection is null.");
		AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_client_null_connection"));
	}
}

