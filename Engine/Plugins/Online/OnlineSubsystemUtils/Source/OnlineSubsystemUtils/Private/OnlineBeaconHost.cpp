// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconHost.h"
#include "Misc/CommandLine.h"
#include "UObject/Package.h"
#include "Engine/World.h"
#include "OnlineBeaconHostObject.h"
#include "OnlineError.h"
#include "Engine/PackageMapClient.h"
#include "Kismet/GameplayStatics.h"
#include "Net/DataChannel.h"
#include "OnlineBeaconClient.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconHost)

#if !UE_BUILD_SHIPPING
namespace BeaconConsoleVariables
{
	TAutoConsoleVariable<float> CVarDelayFinishHandshake(
		TEXT("beacon.DelayFinishHandshake"),
		0.0f,
		TEXT("Delay time before finishing handshake by calling client RPC\n")
		TEXT("Time in seconds. A value of 0 means no delay, and a negative value means never call it."),
		ECVF_Default);

	TAutoConsoleVariable<FString> CVarDelayFinishHandshakeType(
		TEXT("beacon.DelayFinishHandshakeBeaconType"),
		TEXT(""),
		TEXT("The type of beacon to apply the handshake delay to.\n")
		TEXT("Leave blank for all."),
		ECVF_Default);
}
#endif

const FText Error_UnableToParsePacket = NSLOCTEXT("NetworkErrors", "UnableToParsePacket", "Unable to parse expected packet structure: {0}.");
const FText Error_ControlFlow = NSLOCTEXT("NetworkErrors", "ControlFlowError", "Control flow error: {0}.");
const FText Error_Authentication = NSLOCTEXT("NetworkErrors", "AuthenticationFailure", "Failed to verify user authentication.");

AOnlineBeaconHost::AOnlineBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	NetDriverName = FName(TEXT("BeaconDriverHost"));
}

void AOnlineBeaconHost::OnNetCleanup(UNetConnection* Connection)
{
	// minor hack here. AOnlineBeaconHost is considered the connection's OwningActor until we spawn a client actor,
	// so if we get this call it means the connection has disconnected before fully joining 
	UE_LOGF(LogBeacon, Log, "%ls: Cleaning up in-progress connection due to closure.", *GetDebugName(Connection));
	OnConnectionClosed(Connection);
}

void AOnlineBeaconHost::OnConnectionClosed(UNetConnection* Connection)
{
	AOnlineBeaconClient* ClientActor = GetClientActor(Connection);
	if (ClientActor)
	{
		AOnlineBeaconHostObject* BeaconHostObject = GetHost(ClientActor->GetBeaconType());
		UE_LOGF(LogBeacon, Verbose, "%ls: Connection closed. BeaconType: %ls, BeaconActor: %ls, HostObject: %ls",
			*GetDebugName(Connection), *ClientActor->GetBeaconType(), *ClientActor->GetName(), *GetNameSafe(BeaconHostObject));

		if (BeaconHostObject)
		{
			BeaconHostObject->NotifyClientDisconnected(ClientActor);
		}

		RemoveClientActor(ClientActor);
	}

	ConnectionState.Remove(Connection);
}

bool AOnlineBeaconHost::GetConnectionDataForUniqueNetId(const FUniqueNetId& UniqueNetId, UNetConnection*& OutConnection, FConnectionState*& OutConnectionState)
{
	for (TMap<UNetConnection*, FConnectionState>::ElementType& ConnectionStatePair : ConnectionState)
	{
		if (ConnectionStatePair.Key->PlayerId == UniqueNetId)
		{
			OutConnection = ConnectionStatePair.Key;
			OutConnectionState = &ConnectionStatePair.Value;
			return true;
		}
	}

	return false;
}

bool AOnlineBeaconHost::InitHost()
{
	FURL URL(nullptr, TEXT(""), TRAVEL_Absolute);

	// Allow the command line to override the default port
	int32 PortOverride;
	if (FParse::Value(FCommandLine::Get(), TEXT("BeaconPort="), PortOverride) && PortOverride != 0)
	{
		ListenPort = PortOverride;
	}

	URL.Port = ListenPort;
	if(URL.Valid)
	{
		if (InitBase() && NetDriver)
		{
			NetDriver->SetWorld(GetWorld());

			FString Error;
			if (NetDriver->InitListen(this, URL, bReuseAddressAndPort, Error))
			{
				ListenPort = URL.Port;
				NetDriver->Notify = this;
				NetDriver->InitialConnectTimeout = BeaconConnectionInitialTimeout;
				NetDriver->ConnectionTimeout = BeaconConnectionTimeout;
				return true;
			}
			else
			{
				// error initializing the network stack...
				UE_LOGF(LogBeacon, Log, "%ls: AOnlineBeaconHost::InitHost failed", *GetName());
				OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_host_init_failure"));
			}
		}
	}

	return false;
}

void AOnlineBeaconHost::HandleNetworkFailure(UWorld* World, class UNetDriver* InNetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString)
{
	if (InNetDriver && InNetDriver->NetDriverName == NetDriverName)
	{
		// Timeouts from clients are ignored
		if (FailureType != ENetworkFailure::ConnectionTimeout)
		{
			Super::HandleNetworkFailure(World, InNetDriver, FailureType, ErrorString);
		}
	}
}

void AOnlineBeaconHost::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch)
{
	if (NetDriver->ServerConnection == nullptr && Connection)
	{
		// We are the server.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_LOGF(LogBeacon, Verbose, "%ls: Host received: %ls", *GetDebugName(Connection), FNetControlMessageInfo::GetName(MessageType));
#endif
		if (!HandleControlMessage(Connection, MessageType, Bunch))
		{
			CloseHandshakeConnection(Connection);
		}
	}
}

bool AOnlineBeaconHost::HandleControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch)
{
	FConnectionState* ConnState = ConnectionState.Find(Connection);
	if (ConnState == nullptr)
	{
		// only setup state for the hello message
		if (MessageType != NMT_Hello || Connection->OwningActor != nullptr)
		{
			SendFailurePacket(Connection, ENetCloseResult::BeaconControlFlowError,
								FText::Format(Error_ControlFlow, FText::FromString(TEXT("NMT_Hello"))));

			return false;
		}

		// make one
		ConnState = &ConnectionState.Add(Connection, FConnectionState(*this));
		Connection->OwningActor = this; // make sure we get OnNetCleanup if this dies
	}
	check(ConnState != nullptr);

	switch (MessageType)
	{
	case NMT_Hello:
		{
			if (ConnState->bHasSentHello)
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconControlFlowError,
									FText::Format(Error_ControlFlow, FText::FromString(TEXT("NMT_Hello"))));

				return false;
			}
			ConnState->bHasSentHello = true;

			UE_LOGF(LogBeacon, Log, "%ls: Beacon Hello", *GetDebugName(Connection));

			uint8 IsLittleEndian = 0;
			uint32 RemoteNetworkVersion = 0;
			FString EncryptionToken;
			EEngineNetworkRuntimeFeatures LocalNetworkFeatures = NetDriver->GetNetworkRuntimeFeatures();
			EEngineNetworkRuntimeFeatures RemoteNetworkFeatures = EEngineNetworkRuntimeFeatures::None;
	
			if (!FNetControlMessage<NMT_Hello>::Receive(Bunch, IsLittleEndian, RemoteNetworkVersion, EncryptionToken, RemoteNetworkFeatures))
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconUnableToParsePacket,
									FText::Format(Error_UnableToParsePacket, FText::FromString(TEXT("NMT_Hello"))));

				return false;
			}

			// check for net compatibility (in this case sent NMT_Upgrade)
			uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();
			const bool bIsCompatible = FNetworkVersion::IsNetworkCompatible(LocalNetworkVersion, RemoteNetworkVersion) && FNetworkVersion::AreNetworkRuntimeFeaturesCompatible(LocalNetworkFeatures, RemoteNetworkFeatures);

			if (!bIsCompatible)
			{
				TStringBuilder<128> LocalNetFeaturesDescription;
				TStringBuilder<128> RemoteNetFeaturesDescription;

				FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(LocalNetworkFeatures, LocalNetFeaturesDescription);
				FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(RemoteNetworkFeatures, RemoteNetFeaturesDescription);

				UE_LOGF(LogBeacon, Error, "Client not network compatible %ls: LocalNetVersion=%u, RemoteNetVersion=%u, LocalNetFeatures=%ls, RemoteNetFeatures=%ls", 
					*GetDebugName(Connection), 
					LocalNetworkVersion, RemoteNetworkVersion,
					LocalNetFeaturesDescription.ToString(), RemoteNetFeaturesDescription.ToString()
				);
				FNetControlMessage<NMT_Upgrade>::Send(Connection, LocalNetworkVersion, LocalNetworkFeatures);
				return false;
			}
			
			// if the client didn't specify an encryption token we're done with Hello
			if (EncryptionToken.IsEmpty())
			{
				EEncryptionFailureAction FailureResult = EEncryptionFailureAction::Default;
							
				if (FNetDelegates::OnReceivedNetworkEncryptionFailure.IsBound())
				{
					FailureResult = FNetDelegates::OnReceivedNetworkEncryptionFailure.Execute(Connection);
				}

				const bool bGameplayDisableEncryptionCheck = FailureResult == EEncryptionFailureAction::AllowConnection;
				const bool bEncryptionRequired = NetDriver != nullptr && NetDriver->IsEncryptionRequired() && !bGameplayDisableEncryptionCheck;

				if (!bEncryptionRequired)
				{
					OnHelloSequenceComplete(Connection);
				}
				else
				{
					SendFailurePacket(Connection, ENetCloseResult::EncryptionTokenMissing, FText::FromString(TEXT("Encryption token missing")));
					return false;
				}
			}
			else
			{
				// make sure the delegate is bound then route it there
				if (!FNetDelegates::OnReceivedNetworkEncryptionToken.IsBound())
				{
					static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconEncryptionNotBoundError", "Encryption failure");
					SendFailurePacket(Connection, ENetCloseResult::EncryptionFailure, ErrorTxt);
					return false;
				}

				TWeakObjectPtr<UNetConnection> WeakConnection = Connection;
				FNetDelegates::OnReceivedNetworkEncryptionToken.Execute(EncryptionToken, FOnEncryptionKeyResponse::CreateUObject(this, &AOnlineBeaconHost::OnEncryptionResponse, WeakConnection));
			}
		}
		break;
	case NMT_Login:
		{
			if (!ConnState->bHasSentChallenge || ConnState->bHasSentLogin)
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconControlFlowError,
									FText::Format(Error_ControlFlow, FText::FromString(TEXT("NMT_Login"))));

				return false;
			}
			ConnState->bHasSentLogin = true;

			// parse the packet
			FUniqueNetIdRepl UniqueIdRepl;
			FString OnlinePlatformName;

			// Expand the maximum string serialization size to accommodate an authentication token.
			Bunch.ArMaxSerializeSize += MaxAuthTokenSize;
			bool bReceived = FNetControlMessage<NMT_Login>::Receive(Bunch, Connection->ClientResponse, Connection->RequestURL, UniqueIdRepl, OnlinePlatformName);
			Bunch.ArMaxSerializeSize -= MaxAuthTokenSize;

			if (!bReceived)
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconUnableToParsePacket,
									FText::Format(Error_UnableToParsePacket, FText::FromString(TEXT("NMT_Login"))));

				return false;
			}

			if (!UniqueIdRepl.IsValid())
			{
				static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconLoginInvalidIdError", "Login Failure. Invalid ID.");
				SendFailurePacket(Connection, ENetCloseResult::BeaconLoginInvalidIdError, ErrorTxt);
				return false;
			}

			// Only the options/portal for the URL should be used during join
			FString OptionsURL;
			const int32 OptionsStart = Connection->RequestURL.Find(TEXT("?"));
			if (OptionsStart != INDEX_NONE)
			{
				OptionsURL = Connection->RequestURL.RightChop(OptionsStart);
			}

			UE_LOGF(LogBeacon, Log, "%ls: Login request: %ls userId: %ls platform: %ls",
				*GetDebugName(Connection), *OptionsURL, *UniqueIdRepl.ToDebugString(), *OnlinePlatformName);

			// keep track of net id for player associated with remote connection
			Connection->PlayerId = UniqueIdRepl;

			// keep track of the online platform the player associated with this connection is using.
			Connection->SetPlayerOnlinePlatformName(FName(*OnlinePlatformName));

			// Try to kick off verification for this player.
			const FString AuthTicket = UGameplayStatics::ParseOption(OptionsURL, TEXT("AuthTicket"));

			// Create completion delegate.
			FOnAuthenticationVerificationCompleteDelegate OnAuthComplete = FOnAuthenticationVerificationCompleteDelegate::CreateWeakLambda(this, [this, WeakConnection = TWeakObjectPtr<UNetConnection>(Connection)](const FOnlineError& OnlineError)
			{
				OnAuthenticationVerificationComplete(WeakConnection.Get(), OnlineError);
			});

			bool bStartedAuth = StartVerifyAuthentication(*UniqueIdRepl, OptionsURL, AuthTicket, OnAuthComplete);

			if (!bStartedAuth)
			{
				static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconLoginInvalidAuthHandlerError", "Login Failure. Unable to process authentication.");
				SendFailurePacket(Connection, ENetCloseResult::BeaconLoginInvalidAuthHandlerError, ErrorTxt);
				return false;
			}
		}
		break;
	case NMT_Netspeed:
		{
			if (!ConnState->bHasSentWelcome || ConnState->bHasSetNetspeed)
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconControlFlowError,
									FText::Format(Error_ControlFlow, FText::FromString(TEXT("NMT_Netspeed"))));

				return false;
			}
			ConnState->bHasSetNetspeed = true;

			int32 Rate;
			if (!FNetControlMessage<NMT_Netspeed>::Receive(Bunch, Rate))
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconUnableToParsePacket,
									FText::Format(Error_UnableToParsePacket, FText::FromString(TEXT("NMT_Netspeed"))));

				return false;
			}
			Connection->CurrentNetSpeed = FMath::Clamp(Rate, 1800, NetDriver->MaxClientRate);
			UE_LOGF(LogBeacon, Log, "%ls: Client netspeed is %i", *GetDebugName(Connection), Connection->CurrentNetSpeed);
		}
		break;
	case NMT_BeaconJoin:
		{
			if (!ConnState->bHasSetNetspeed || ConnState->bHasJoined || (bAuthRequired && !ConnState->bHasAuthenticated))
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconControlFlowError,
									FText::Format(Error_ControlFlow, FText::FromString(TEXT("NMT_BeaconJoin"))));

				return false;
			}
			ConnState->bHasJoined = true;

			FString BeaconType;
			FUniqueNetIdRepl UniqueId;
			if (!FNetControlMessage<NMT_BeaconJoin>::Receive(Bunch, BeaconType, UniqueId))
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconUnableToParsePacket,
									FText::Format(Error_UnableToParsePacket, FText::FromString(TEXT("NMT_BeaconJoin"))));

				return false;
			}

			// validate the player ID
			if (bAuthRequired)
			{
				// this should have already been set on the connection when we validated them, so it must match
				if (!UniqueId.IsValid() || UniqueId != Connection->PlayerId)
				{
					static const FText AuthErrorText = NSLOCTEXT("NetworkErrors", "BeaconAuthError", "Unable to authenticate for beacon. {0} is not valid for connection owned by {1}");

					SendFailurePacket(Connection, ENetCloseResult::BeaconAuthError,
										FText::Format(AuthErrorText, FText::FromString(UniqueId.ToDebugString()),
														FText::FromString(Connection->PlayerId.ToDebugString())));

					return false;
				}

				if (!VerifyJoinForBeaconType(*UniqueId, BeaconType))
				{
					static const FText AuthErrorText = NSLOCTEXT("NetworkErrors", "BeaconAuthVerifyError", "Unable to authenticate for beacon. Verifying auth for beacon type {0} failed for connection owned by {1}");

					SendFailurePacket(Connection, ENetCloseResult::BeaconAuthError,
										FText::Format(AuthErrorText, FText::FromString(BeaconType),
														FText::FromString(Connection->PlayerId.ToDebugString())));

					return false;
				}

				UE_LOGF(LogBeacon, Log, "%ls: Beacon Join %ls %ls", *GetDebugName(Connection), *BeaconType, *UniqueId.ToDebugString());
			}
			else
			{
				UE_LOGF(LogBeacon, Log, "%ls: Beacon Join %ls %ls (unauthenticated)", *GetDebugName(Connection), *BeaconType, *UniqueId.ToDebugString());
			}

			// set the connection's client world package
			if (Connection->GetClientWorldPackageName() != NAME_None)
			{
				static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnClientWorldPackageNameError", "Join failure, existing ClientWorldPackageName.");
				SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnClientWorldPackageNameError, ErrorTxt);
				return false;
			}
			AOnlineBeaconClient* ClientActor = GetClientActor(Connection);
			if (ClientActor != nullptr)
			{
				static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnExistingActorError", "Join failure, existing beacon actor.");
				SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnExistingActorError, ErrorTxt);
				return false;
			}
			UWorld* World = GetWorld();
			Connection->SetClientWorldPackageName(World->GetOutermost()->GetFName());

			// spawn the beacon actor for this client
			AOnlineBeaconClient* NewClientActor = nullptr;
			if (FOnBeaconSpawned* OnBeaconSpawnedDelegate = OnBeaconSpawnedMapping.Find(BeaconType);
				OnBeaconSpawnedDelegate && OnBeaconSpawnedDelegate->IsBound())
			{
				NewClientActor = OnBeaconSpawnedDelegate->Execute(Connection);
			}
			else
			{
				static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnFailureError.UnknownSpawner", "Join failure, Couldn't spawn client beacon actor (unknown beacon spawner).");
				SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnFailureError, ErrorTxt);
				return false;
			}

			// make sure it spawned correctly
			if (NewClientActor == nullptr || BeaconType != NewClientActor->GetBeaconType())
			{
				static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnFailureError", "Join failure, Couldn't spawn client beacon actor.");
				SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnFailureError, ErrorTxt);
				return false;
			}

			Connection->SetClientLoginState(EClientLoginState::ReceivedJoin);
			NewClientActor->SetConnectionState(EBeaconConnectionState::Pending);

			UE::Net::FNetIDVariant NetID = Connection->Driver->CreateNetIDForObject(NewClientActor);
			NewClientActor->SetNetConnection(Connection);
			Connection->PlayerId = UniqueId;
			NewClientActor->SetRole(ROLE_Authority);
			NewClientActor->SetReplicates(false);
			check(NetDriverName == NetDriver->NetDriverName);
			NewClientActor->SetNetDriverName(NetDriverName);
			ClientActors.Add(NewClientActor);

			FNetControlMessage<NMT_BeaconAssignGUID>::Send(Connection, NetID);
			Connection->FlushNet(true);
		}
		break;
	case NMT_BeaconNetGUIDAck:
		{
			if (!ConnState->bHasJoined || ConnState->bHasCompletedAck)
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconControlFlowError,
									FText::Format(Error_ControlFlow, FText::FromString(TEXT("NMT_BeaconNetGUIDAck"))));

				return false;
			}
			ConnState->bHasCompletedAck = true;

			FString BeaconType;
			if (!FNetControlMessage<NMT_BeaconNetGUIDAck>::Receive(Bunch, BeaconType))
			{
				SendFailurePacket(Connection, ENetCloseResult::BeaconUnableToParsePacket,
									FText::Format(Error_UnableToParsePacket, FText::FromString(TEXT("NMT_BeaconNetGUIDAck"))));

				return false;
			}

#if UE_BUILD_SHIPPING
			const float Delay = 0.0f;
			const FString DelayBeaconType;
			const bool bDelayAllowedForBeaconType = false;
#else
			const float Delay = BeaconConsoleVariables::CVarDelayFinishHandshake.GetValueOnGameThread();
			const FString DelayBeaconType = BeaconConsoleVariables::CVarDelayFinishHandshakeType.GetValueOnGameThread();
			const bool bDelayAllowedForBeaconType = DelayBeaconType.IsEmpty() || DelayBeaconType == BeaconType;
#endif
			if (Delay == 0.0f || !bDelayAllowedForBeaconType)
			{
				FinishHandshake(Connection, MoveTemp(BeaconType));
			}
			else if (Delay > 0.0f)
			{
				UE_LOGF(LogBeacon, Verbose, "%ls: BeaconType: %ls, Delay for handshake completion: %.3f", *GetDebugName(Connection), *BeaconType, Delay);
				FTimerDelegate TimerDelegate = FTimerDelegate::CreateUObject(this, &AOnlineBeaconHost::FinishHandshake, Connection, MoveTemp(BeaconType));
				GetWorldTimerManager().SetTimer(ConnState->FinishHandshakeTimerHandle, TimerDelegate, Delay, false);
			}
			else
			{
				UE_LOGF(LogBeacon, Verbose, "%ls: BeaconType: %ls, Handshake will never complete, client will either timeout or hang if timeouts are disabled.", *GetDebugName(Connection), *BeaconType);
			}
		}

		break;

	default:
		{
			static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnUnexpectedError", "Join failure, unexpected control message.");
			SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnUnexpectedError, ErrorTxt);
			return false;
		}
	}

	// if we made it here, leave the connection open
	return true;
}

void AOnlineBeaconHost::FinishHandshake(UNetConnection* Connection, FString BeaconType)
{
	AOnlineBeaconClient* ClientActor = GetClientActor(Connection);
	if (ClientActor == nullptr || BeaconType != ClientActor->GetBeaconType())
	{
		static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnNetGUIDAckError2", "Join failure, no actor at NetGUIDAck.");
		SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnNetGUIDAckNoActor, ErrorTxt);
		CloseHandshakeConnection(Connection);
		return;
	}

	FOnBeaconConnected* OnBeaconConnectedDelegate = OnBeaconConnectedMapping.Find(BeaconType);
	if (OnBeaconConnectedDelegate == nullptr)
	{
		static const FText ErrorTxt = NSLOCTEXT("NetworkErrors", "BeaconSpawnNetGUIDAckError1", "Join failure, no host object at NetGUIDAck.");
		SendFailurePacket(Connection, ENetCloseResult::BeaconSpawnNetGUIDAckNoHost, ErrorTxt);
		CloseHandshakeConnection(Connection);
		return;
	}

	// once we change the owning actor we can't clean up state anymore, but at this point we don't need to
	ConnectionState.Remove(Connection);
	Connection->OwningActor = ClientActor;

	ClientActor->SetReplicates(true);
	ClientActor->SetAutonomousProxy(true);
	ClientActor->SetConnectionState(EBeaconConnectionState::Open);

	// Send an RPC to the client to open the actor channel and guarantee RPCs will work
	ClientActor->ClientOnConnected();
	Connection->FlushNet(true);

	UE_LOGF(LogBeacon, Log, "%ls: Handshake complete.", *GetDebugName(Connection));
	OnBeaconConnectedDelegate->ExecuteIfBound(ClientActor, Connection);
}

AOnlineBeaconHost::FConnectionState::~FConnectionState()
{
	BeaconHost.GetWorldTimerManager().ClearTimer(FinishHandshakeTimerHandle);
}

void AOnlineBeaconHost::SendFailurePacket(UNetConnection* Connection, FNetCloseResult&& CloseReason, const FText& ErrorText)
{
	if (Connection != nullptr)
	{
		FString ErrorMsg = ErrorText.ToString();
		FNetCloseResult CloseReasonCopy = CloseReason;

		UE_LOGF(LogBeacon, Log, "%ls: Send failure: %ls", ToCStr(GetDebugName(Connection)), ToCStr(ErrorMsg));

		Connection->SendCloseReason(MoveTemp(CloseReasonCopy));
		FNetControlMessage<NMT_Failure>::Send(Connection, ErrorMsg);
		Connection->FlushNet(true);
		Connection->Close(MoveTemp(CloseReason));
	}
}

void AOnlineBeaconHost::CloseHandshakeConnection(UNetConnection* Connection)
{
	if (Connection && Connection->GetConnectionState() != USOCK_Closed)
	{
		UE_LOGF(LogBeacon, Verbose, "%ls: Closing connection: %ls", *GetDebugName(Connection), *Connection->PlayerId.ToDebugString());
		OnConnectionClosed(Connection);
		Connection->OwningActor = nullptr; // don't notify us again
		Connection->FlushNet(true);
		Connection->Close();
		UE_LOGF(LogBeacon, Verbose, "--------------------------------");
	}
}

void AOnlineBeaconHost::DisconnectClient(AOnlineBeaconClient* ClientActor)
{
	if (IsValid(ClientActor) && ClientActor->GetConnectionState() != EBeaconConnectionState::Closed)
	{
		ClientActor->SetConnectionState(EBeaconConnectionState::Closed);

		UNetConnection* Connection = ClientActor->GetNetConnection();

		UE_LOGF(LogBeacon, Log, "%ls: DisconnectClient for %ls. UNetDriver %ls State %d",
			*GetDebugName(Connection),
			*GetNameSafe(ClientActor),
			Connection ? *GetNameSafe(Connection->Driver) : TEXT("null"),
			Connection ? Connection->GetConnectionState() : -1);

		// Closing the connection will start the chain of events leading to the removal from lists and destruction of the actor
		if (Connection && Connection->GetConnectionState() != USOCK_Closed)
		{
			Connection->FlushNet(true);
			Connection->Close();
		}
	}
}

AOnlineBeaconClient* AOnlineBeaconHost::GetClientActor(UNetConnection* Connection)
{
	for (int32 ClientIdx=0; ClientIdx < ClientActors.Num(); ClientIdx++)
	{
		if (ensure(ClientActors[ClientIdx]) && ClientActors[ClientIdx]->GetNetConnection() == Connection)
		{
			return ClientActors[ClientIdx];
		}
	}

	return nullptr;
}

void AOnlineBeaconHost::RemoveClientActor(AOnlineBeaconClient* ClientActor)
{
	if (ClientActor)
	{
		ClientActors.RemoveSingleSwap(ClientActor);
		if (!ClientActor->IsPendingKillPending())
		{
			ClientActor->Destroy();
		}
	}
}

bool AOnlineBeaconHost::StartVerifyAuthentication(const FUniqueNetId& PlayerId, const FString& LoginOptions, const FString& AuthenticationToken, const FOnAuthenticationVerificationCompleteDelegate& OnComplete)
{
	return false;
}

bool AOnlineBeaconHost::VerifyJoinForBeaconType(const FUniqueNetId& PlayerId, const FString& BeaconType)
{
	return true;
}

void AOnlineBeaconHost::OnAuthenticationVerificationComplete(UNetConnection* Connection, const FOnlineError& Error)
{
	if (FConnectionState* ConnState = ConnectionState.Find(Connection))
	{
		if (ConnState->bHasSentLogin && !ConnState->bHasSentWelcome)
		{
			// Gating login on valid authentication. Do not fail open so that users are known to have access.
			if (Error.WasSuccessful())
			{
				// Mark user as authenticated.
				ConnState->bHasAuthenticated = true;

				// send the welcome packet
				UE_LOGF(LogBeacon, Verbose, "%ls: OnAuthenticationVerificationComplete: Successfully authenticated.", *GetDebugName(Connection));
				ConnState->bHasSentWelcome = true;
				FNetControlMessage<NMT_BeaconWelcome>::Send(Connection);
				Connection->FlushNet();
			}
			else
			{
				// Auth failure.
				UE_LOGF(LogBeacon, Log, "%ls: OnAuthenticationVerificationComplete: Failed to authenticate error: %ls", *GetDebugName(Connection), *Error.ToLogString());
				SendFailurePacket(Connection, ENetCloseResult::BeaconAuthenticationFailure, Error_Authentication);
				CloseHandshakeConnection(Connection);
			}
		}
		else
		{
			UE_LOGF(LogBeacon, Warning, "%ls: OnAuthenticationVerificationComplete: Invalid beacon connection state for auth challenge result. error: %ls", *GetDebugName(Connection), *Error.ToLogString());
		}
	}
	else
	{
		UE_LOGF(LogBeacon, Warning, "%ls: OnAuthenticationVerificationComplete: Failed to find connection state. error: %ls", *GetDebugName(Connection), *Error.ToLogString());
	}
}

FString AOnlineBeaconHost::GetDebugName(UNetConnection* Connection)
{
	return FString::Printf(TEXT("%s[%s]"), *GetName(), *GetNameSafe(Connection));
}

void AOnlineBeaconHost::OnHelloSequenceComplete(UNetConnection* Connection)
{
	if (Connection == nullptr)
	{
		UE_LOGF(LogBeacon, Log, "%ls: OnlineBeaconHost::OnHelloSequenceComplete: Connection is null.", *GetDebugName());
		return;
	}
	if (Connection->GetConnectionState() == USOCK_Invalid || Connection->GetConnectionState() == USOCK_Closed || Connection->Driver == nullptr)
	{
		UE_LOGF(LogBeacon, Log, "%ls: OnlineBeaconHost::OnHelloSequenceComplete: connection in invalid state. %ls", *GetDebugName(Connection), *Connection->Describe());
		return;
	}
	FConnectionState* ConnState = ConnectionState.Find(Connection);
	if (ConnState == nullptr)
	{
		UE_LOGF(LogBeacon, Log, "%ls: OnlineBeaconHost::OnHelloSequenceComplete: unable to find connection state.", *GetDebugName(Connection));
		return;
	}
	if (ConnState->bHasSentWelcome)
	{
		UE_LOGF(LogBeacon, Log, "%ls: OnlineBeaconHost::OnHelloSequenceComplete: sequence error, user already sent welcome.", *GetDebugName(Connection));
		return;
	}

	if (bAuthRequired)
	{
		// ask the client to send login
		UE_LOGF(LogBeacon, Verbose, "%ls: Sending challenge.", *GetDebugName(Connection));
		ConnState->bHasSentChallenge = true;
		Connection->SetClientLoginState(EClientLoginState::LoggingIn);
		Connection->SendChallengeControlMessage();
	}
	else
	{
		// send the welcome method (no challenge)
		UE_LOGF(LogBeacon, Verbose, "%ls: Connection welcomed.", *GetDebugName(Connection));
		Connection->Challenge = FString::Printf(TEXT("%08X"), FPlatformTime::Cycles());
		ConnState->bHasSentWelcome = true;
		FNetControlMessage<NMT_BeaconWelcome>::Send(Connection);
	}
	Connection->FlushNet();
}

void AOnlineBeaconHost::OnEncryptionResponse(const FEncryptionKeyResponse& Response, TWeakObjectPtr<UNetConnection> WeakConnection)
{
	UNetConnection* Connection = WeakConnection.Get();
	if (Connection)
	{
		if (Connection->GetConnectionState() != USOCK_Invalid && Connection->GetConnectionState() != USOCK_Closed && Connection->Driver)
		{
			if (Response.Response == EEncryptionResponse::Success)
			{
				Connection->EnableEncryptionServer(Response.EncryptionData);
				OnHelloSequenceComplete(Connection);
			}
			else
			{
				FString ResponseStr(LexToString(Response.Response));
				UE_LOGF(LogBeacon, Warning, "%ls: OnlineBeaconHost::OnEncryptionResponse: encryption failure [%ls] %ls", *GetDebugName(Connection), *ResponseStr, *Response.ErrorMsg);

				Connection->SendCloseReason(ENetCloseResult::EncryptionFailure);
				FNetControlMessage<NMT_Failure>::Send(Connection, ResponseStr);
				Connection->FlushNet(true);
				Connection->Close(ENetCloseResult::EncryptionFailure);
			}
		}
		else
		{
			UE_LOGF(LogBeacon, Warning, "%ls: OnlineBeaconHost::OnEncryptionResponse: connection in invalid state. %ls", *GetDebugName(Connection), *Connection->Describe());
		}
	}
	else
	{
		UE_LOGF(LogBeacon, Warning, "%ls: OnlineBeaconHost::OnEncryptionResponse: Connection is null.", *GetDebugName());
	}
}

void AOnlineBeaconHost::RegisterHost(AOnlineBeaconHostObject* NewHostObject)
{
	const FString& BeaconType = NewHostObject->GetBeaconType();
	if (GetHost(BeaconType) == NULL)
	{
		NewHostObject->SetOwner(this);
		OnBeaconSpawned(BeaconType).BindUObject(NewHostObject, &AOnlineBeaconHostObject::SpawnBeaconActor);
		OnBeaconConnected(BeaconType).BindUObject(NewHostObject, &AOnlineBeaconHostObject::OnClientConnected);
	}
	else
	{
		UE_LOGF(LogBeacon, Warning, "%ls: Beacon host type %ls already exists", *GetName(), *BeaconType);
	}
}

void AOnlineBeaconHost::UnregisterHost(const FString& BeaconType)
{
	AOnlineBeaconHostObject* HostObject = GetHost(BeaconType);
	if (HostObject)
	{
		HostObject->Unregister();
	}
	
	OnBeaconSpawned(BeaconType).Unbind();
	OnBeaconConnected(BeaconType).Unbind();
}

AOnlineBeaconHostObject* AOnlineBeaconHost::GetHost(const FString& BeaconType)
{
	for (int32 HostIdx=0; HostIdx < Children.Num(); HostIdx++)
	{
		AOnlineBeaconHostObject* HostObject = Cast<AOnlineBeaconHostObject>(Children[HostIdx]);
		if (HostObject && HostObject->GetBeaconType() == BeaconType)
		{
			return HostObject;
		}
	}

	return nullptr;
}

AOnlineBeaconHost::FOnBeaconSpawned& AOnlineBeaconHost::OnBeaconSpawned(const FString& BeaconType)
{ 
	FOnBeaconSpawned* BeaconDelegate = OnBeaconSpawnedMapping.Find(BeaconType);
	if (BeaconDelegate == nullptr)
	{
		FOnBeaconSpawned NewDelegate;
		OnBeaconSpawnedMapping.Add(BeaconType, NewDelegate);
		BeaconDelegate = OnBeaconSpawnedMapping.Find(BeaconType);
	}

	return *BeaconDelegate; 
}

AOnlineBeaconHost::FOnBeaconConnected& AOnlineBeaconHost::OnBeaconConnected(const FString& BeaconType)
{ 
	FOnBeaconConnected* BeaconDelegate = OnBeaconConnectedMapping.Find(BeaconType);
	if (BeaconDelegate == nullptr)
	{
		FOnBeaconConnected NewDelegate;
		OnBeaconConnectedMapping.Add(BeaconType, NewDelegate);
		BeaconDelegate = OnBeaconConnectedMapping.Find(BeaconType);
	}

	return *BeaconDelegate; 
}

