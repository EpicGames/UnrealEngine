// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconHostObject.h"
#include "Engine/NetConnection.h"
#include "Engine/World.h"
#include "OnlineBeaconHost.h"
#include "OnlineBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconHostObject)

AOnlineBeaconHostObject::AOnlineBeaconHostObject(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	BeaconTypeName(TEXT("UNDEFINED"))
{
	PrimaryActorTick.bCanEverTick = true;
}

AOnlineBeaconClient* AOnlineBeaconHostObject::SpawnBeaconActor(UNetConnection* ClientConnection)
{
	if (ClientBeaconActorClass)
	{
		FActorSpawnParameters SpawnInfo;
		AOnlineBeaconClient* BeaconActor = GetWorld()->SpawnActor<AOnlineBeaconClient>(ClientBeaconActorClass, FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
		if (BeaconActor)
		{
			BeaconActor->SetBeaconOwner(this);
		}

		return BeaconActor;
	}
	
	UE_LOGF(LogBeacon, Warning, "Invalid client beacon actor class of type %ls", *GetBeaconType());
	return nullptr;
}

void AOnlineBeaconHostObject::OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection)
{
	if (NewClientActor == nullptr)
	{
		return;
	}

	UNetConnection* ActorConnection = NewClientActor->GetNetConnection();
	UE_LOGF(LogBeacon, Verbose, "OnClientConnected %ls from (%ls)",
		*NewClientActor->GetName(),
		ActorConnection ? *ActorConnection->LowLevelDescribe() : TEXT("NULL"));

	ClientActors.Add(NewClientActor);
}

void AOnlineBeaconHostObject::DisconnectClient(AOnlineBeaconClient* ClientActor)
{
	AOnlineBeaconHost* BeaconHost = Cast<AOnlineBeaconHost>(GetOwner());
	if (BeaconHost)
	{
		BeaconHost->DisconnectClient(ClientActor);
	}
}

void AOnlineBeaconHostObject::NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor)
{
	UE_LOGF(LogBeacon, Verbose, "NotifyClientDisconnected %ls",
		LeavingClientActor ? *LeavingClientActor->GetName() : TEXT("NULL"));

	// Remove from local list of clients
	if (LeavingClientActor)
	{
		ClientActors.RemoveSingleSwap(LeavingClientActor);
	}

	// Remove from global list of clients
	AOnlineBeaconHost* BeaconHost = Cast<AOnlineBeaconHost>(GetOwner());
	if (BeaconHost)
	{
		BeaconHost->RemoveClientActor(LeavingClientActor);
	}
}

void AOnlineBeaconHostObject::Unregister()
{
	// Kill all the client connections associated with this host object
	for (AOnlineBeaconClient* ClientActor : ClientActors)
	{
		DisconnectClient(ClientActor);
	}

	SetOwner(nullptr);
}

FName AOnlineBeaconHostObject::GetNetDriverName() const
{
	AActor* BeaconHost = GetOwner();
	if (BeaconHost)
	{
		return BeaconHost->GetNetDriverName();
	}

	return NAME_None;
}

EBeaconState::Type AOnlineBeaconHostObject::GetBeaconState() const
{
	AOnlineBeaconHost* BeaconHost = Cast<AOnlineBeaconHost>(GetOwner());
	if (BeaconHost)
	{
		return BeaconHost->GetBeaconState();
	}

	return EBeaconState::DenyRequests;
}

