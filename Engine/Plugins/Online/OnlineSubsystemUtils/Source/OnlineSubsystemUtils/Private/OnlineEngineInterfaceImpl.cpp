// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineEngineInterfaceImpl.h"
#include "Interfaces/OnlineChatInterface.h"
#include "Interfaces/OnlinePartyInterface.h"
#include "OnlineSubsystemUtils.h"
#include "Engine/NetConnection.h"
#include "Interfaces/OnlineSessionDelegates.h"
#include "Online.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineEngineInterfaceImpl)

namespace UOnlineEngineInterfaceImplPrivate
{

static FName ConvertToCompatibilityOnlineIdentifier(FName OnlineIdentifier)
{
	FName CompatibilityOnlineIdentifier;

	FString OnlineIdentifierStr = OnlineIdentifier.ToString();
	int32 Index = -1;
	bool bFound = OnlineIdentifierStr.FindChar(':', Index);
	if (bFound)
	{
		CompatibilityOnlineIdentifier = FName(OnlineIdentifierStr.Mid(Index+1));
	}
	else
	{
		CompatibilityOnlineIdentifier = OnlineIdentifier;
	}

	return CompatibilityOnlineIdentifier;
}

}

UOnlineEngineInterfaceImpl::UOnlineEngineInterfaceImpl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VoiceSubsystemNameOverride(NAME_None)
{
}

void UOnlineEngineInterfaceImpl::PostInitProperties()
{
	Super::PostInitProperties();
	InitCompatibilityInterface();
}

bool UOnlineEngineInterfaceImpl::IsLoaded(FName OnlineIdentifier)
{
	return IOnlineSubsystem::IsLoaded(OnlineIdentifier);
}

FName UOnlineEngineInterfaceImpl::GetOnlineIdentifier(FWorldContext& WorldContext)
{
	IOnlineSubsystemUtils* Utils = Online::GetUtils();
	if (Utils)
	{
		return Utils->GetOnlineIdentifier(WorldContext);
	}
	return NAME_None;
}

FName UOnlineEngineInterfaceImpl::GetOnlineIdentifier(UWorld* World)
{
	IOnlineSubsystemUtils* Utils = Online::GetUtils();
	if (Utils)
	{
		return Utils->GetOnlineIdentifier(World);
	}

	return NAME_None;
}

bool UOnlineEngineInterfaceImpl::DoesInstanceExist(FName OnlineIdentifier)
{
	return IOnlineSubsystem::DoesInstanceExist(OnlineIdentifier);
}

void UOnlineEngineInterfaceImpl::ShutdownOnlineSubsystem(FName OnlineIdentifier)
{
	if (UOnlineEngineInterface* CompatibilityInterface = OnlineServicesCompatibilityInterface.Get())
	{
		FName CompatibilityOnlineIdentifier = UOnlineEngineInterfaceImplPrivate::ConvertToCompatibilityOnlineIdentifier(OnlineIdentifier);
		CompatibilityInterface->ShutdownOnlineSubsystem(CompatibilityOnlineIdentifier);
	}

	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get(OnlineIdentifier);
	if (OnlineSub)
	{
		OnlineSub->Shutdown();
	}
}

void UOnlineEngineInterfaceImpl::DestroyOnlineSubsystem(FName OnlineIdentifier)
{
	if (UOnlineEngineInterface* CompatibilityInterface = OnlineServicesCompatibilityInterface.Get())
	{
		FName CompatibilityOnlineIdentifier = UOnlineEngineInterfaceImplPrivate::ConvertToCompatibilityOnlineIdentifier(OnlineIdentifier);
		CompatibilityInterface->DestroyOnlineSubsystem(CompatibilityOnlineIdentifier);
	}

	IOnlineSubsystem::Destroy(OnlineIdentifier);
}

FName UOnlineEngineInterfaceImpl::GetDefaultOnlineSubsystemName() const
{
	// World context (PIE) isn't necessary here as it's just the name of the default
	// no matter how many instances actually exist
	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
	return OnlineSub ? OnlineSub->GetSubsystemName() : NAME_None;
}

bool UOnlineEngineInterfaceImpl::IsCompatibleUniqueNetId(const FUniqueNetIdWrapper& InUniqueNetId) const
{
	return InUniqueNetId.IsV1() && (InUniqueNetId.GetType() == GetDefaultOnlineSubsystemName() || CompatibleUniqueNetIdTypes.Contains(InUniqueNetId.GetType()));
}

uint8 UOnlineEngineInterfaceImpl::GetReplicationHashForSubsystem(FName InSubsystemName) const
{
	return Online::GetUtils()->GetReplicationHashForSubsystem(InSubsystemName);
}

FName UOnlineEngineInterfaceImpl::GetSubsystemFromReplicationHash(uint8 InHash) const
{
	return Online::GetUtils()->GetSubsystemFromReplicationHash(InHash);
}

FName UOnlineEngineInterfaceImpl::GetDedicatedServerSubsystemNameForSubsystem(const FName Subsystem) const
{
	// For console platforms with their own online subsystem, there may be a separate
	// online system that can run on dedicated servers, since the console one typically
	// won't compile/run on dedicated server platforms. The console and server OSSs should
	// maintain compatibility with serialized data, such as voice packets, so that the server
	// OSS can properly forward them to other clients using the console OSS.

	// Clients may send their platform subsystem name via the "OnlinePlatform=" login URL option,
	// then the server can pass the value of that option to this function to get the name of
	// the corresponding server OSS for that client, if one exists.

	if (Subsystem == PS4_SUBSYSTEM)
	{
		return PS4SERVER_SUBSYSTEM;
	}

	return NAME_None;
}

FUniqueNetIdWrapper UOnlineEngineInterfaceImpl::CreateUniquePlayerIdWrapper(const FString& Str, FName Type)
{
	// Foreign types may be passed into this function, do not load OSS modules explicitly here
	FUniqueNetIdPtr UniqueId = nullptr;
	//Configuration driven mapping for UniqueNetIds so we don't treat the mapped ids as foreign
	const FName* MappedUniqueNetIdType = MappedUniqueNetIdTypes.Find(Type);

	bool bIsPrimaryLoaded = IsLoaded(Type);
	bool bIsMappedUniqueNetIdTypeLoaded = (MappedUniqueNetIdType != nullptr ? IsLoaded(*MappedUniqueNetIdType) : false);
	
	if (bIsPrimaryLoaded || bIsMappedUniqueNetIdTypeLoaded)
	{
		// No UWorld here, but ok since this is just a factory
		UWorld* World = nullptr;
		IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(World, bIsMappedUniqueNetIdTypeLoaded ? *MappedUniqueNetIdType : Type);
		if (IdentityInt.IsValid())
		{
			UniqueId = IdentityInt->CreateUniquePlayerId(Str);
		}
	}
	
	if (!UniqueId.IsValid())
	{
		if (IOnlineSubsystemUtils* Utils = Online::GetUtils())
		{
			// Create a unique id for other platforms unknown to this instance
			// Will not compare correctly against native types (do not use on platform where native type is available)
			// Used to maintain opaque unique id that will compare against other non native types
			UniqueId = Utils->CreateForeignUniqueNetId(Str, Type);
		}
	}
	return UniqueId;
}

FUniqueNetIdWrapper UOnlineEngineInterfaceImpl::GetUniquePlayerIdWrapper(UWorld* World, int32 LocalUserNum, FName Type)
{
	IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(World, Type);
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdPtr UniqueId = IdentityInt->GetUniquePlayerId(LocalUserNum);
		return UniqueId;
	}

	UE_LOG_ONLINE(Verbose, TEXT("GetUniquePlayerId() returning null, can't find OSS of type %s"), *Type.ToString());
	return FUniqueNetIdWrapper();
}

FString UOnlineEngineInterfaceImpl::GetPlayerNickname(UWorld* World, const FUniqueNetIdWrapper& UniqueId)
{
	check(UniqueId.IsValid() && UniqueId.IsV1());
	IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(World, UniqueId.GetType());
	if (IdentityInt.IsValid())
	{	
		return IdentityInt->GetPlayerNickname(*UniqueId);
	}

	static FString InvalidName(TEXT("InvalidOSSUser"));
	return InvalidName;
}

bool UOnlineEngineInterfaceImpl::GetPlayerPlatformNickname(UWorld* World, int32 LocalUserNum, FString& OutNickname)
{
	IOnlineSubsystem* PlatformSubsystem = IOnlineSubsystem::GetByPlatform(false);
	if (PlatformSubsystem)
	{
		IOnlineIdentityPtr OnlineIdentityInt = PlatformSubsystem->GetIdentityInterface();
		if (OnlineIdentityInt.IsValid())
		{
			OutNickname = OnlineIdentityInt->GetPlayerNickname(LocalUserNum);
			if (!OutNickname.IsEmpty())
			{
				return true;
			}
		}
	}
	return false;
}

bool UOnlineEngineInterfaceImpl::AutoLogin(UWorld* World, int32 LocalUserNum, const FOnlineAutoLoginComplete& InCompletionDelegate)
{
	IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(World);
	if (IdentityInt.IsValid())
	{
		FName OnlineIdentifier = GetOnlineIdentifier(World);

		OnLoginCompleteDelegateHandle = IdentityInt->AddOnLoginCompleteDelegate_Handle(LocalUserNum, FOnLoginCompleteDelegate::CreateUObject(this, &ThisClass::OnAutoLoginComplete, OnlineIdentifier, InCompletionDelegate));
		if (IdentityInt->AutoLogin(LocalUserNum))
		{
			// Async login started
			return true;
		}
	}

	// Not waiting for async login
	return false;
}

void UOnlineEngineInterfaceImpl::OnAutoLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FName OnlineIdentifier, FOnlineAutoLoginComplete InCompletionDelegate)
{
	IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(OnlineIdentifier);
	if (IdentityInt.IsValid())
	{
		IdentityInt->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, OnLoginCompleteDelegateHandle);
	}

	InCompletionDelegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, Error);
}

bool UOnlineEngineInterfaceImpl::IsLoggedIn(UWorld* World, int32 LocalUserNum)
{
	IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(World);
	if (IdentityInt.IsValid())
	{
		return (IdentityInt->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn);
	}

	return false;
}

void UOnlineEngineInterfaceImpl::StartSession(UWorld* World, FName SessionName, FOnlineSessionStartComplete& InCompletionDelegate)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
		if (Session && (Session->SessionState == EOnlineSessionState::Pending || Session->SessionState == EOnlineSessionState::Ended))
		{
			FName OnlineIdentifier = GetOnlineIdentifier(World);

			FDelegateHandle StartSessionCompleteHandle = SessionInt->AddOnStartSessionCompleteDelegate_Handle(FOnStartSessionCompleteDelegate::CreateUObject(this, &ThisClass::OnStartSessionComplete, OnlineIdentifier, InCompletionDelegate));
			OnStartSessionCompleteDelegateHandles.Add(OnlineIdentifier, StartSessionCompleteHandle);

			SessionInt->StartSession(SessionName);
		}
		else
		{
			InCompletionDelegate.ExecuteIfBound(SessionName, false);
		}
	}
	else
	{
		InCompletionDelegate.ExecuteIfBound(SessionName, false);
	}
}

void UOnlineEngineInterfaceImpl::OnStartSessionComplete(FName SessionName, bool bWasSuccessful, FName OnlineIdentifier, FOnlineSessionStartComplete CompletionDelegate)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(OnlineIdentifier);
	if (SessionInt.IsValid())
	{
		// Cleanup the login delegate before calling create below
		FDelegateHandle* DelegateHandle = OnStartSessionCompleteDelegateHandles.Find(OnlineIdentifier);
		if (DelegateHandle)
		{
			SessionInt->ClearOnStartSessionCompleteDelegate_Handle(*DelegateHandle);
		}
	}

	CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
}

void UOnlineEngineInterfaceImpl::EndSession(UWorld* World, FName SessionName, FOnlineSessionEndComplete& InCompletionDelegate)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		FName OnlineIdentifier = GetOnlineIdentifier(World);

		FDelegateHandle EndSessionCompleteHandle = SessionInt->AddOnEndSessionCompleteDelegate_Handle(FOnEndSessionCompleteDelegate::CreateUObject(this, &ThisClass::OnEndSessionComplete, OnlineIdentifier, InCompletionDelegate));
		OnEndSessionCompleteDelegateHandles.Add(OnlineIdentifier, EndSessionCompleteHandle);

		SessionInt->EndSession(SessionName);
	}
	else
	{
		InCompletionDelegate.ExecuteIfBound(SessionName, false);
	}
}

void UOnlineEngineInterfaceImpl::OnEndSessionComplete(FName SessionName, bool bWasSuccessful, FName OnlineIdentifier, FOnlineSessionEndComplete CompletionDelegate)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(OnlineIdentifier);
	if (SessionInt.IsValid())
	{
		FDelegateHandle* DelegateHandle = OnEndSessionCompleteDelegateHandles.Find(OnlineIdentifier);
		if (DelegateHandle)
		{
			SessionInt->ClearOnEndSessionCompleteDelegate_Handle(*DelegateHandle);
		}
	}

	CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
}

bool UOnlineEngineInterfaceImpl::DoesSessionExist(UWorld* World, FName SessionName)
{
	FOnlineSessionSettings* SessionSettings = nullptr;
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		SessionSettings = SessionInt->GetSessionSettings(SessionName);
	}

	return SessionSettings != nullptr;
}

bool UOnlineEngineInterfaceImpl::GetSessionJoinability(UWorld* World, FName SessionName, FJoinabilitySettings& OutSettings)
{
	bool bValidData = false;

	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		FOnlineSessionSettings* SessionSettings = SessionInt->GetSessionSettings(SessionName);
		if (SessionSettings)
		{
			OutSettings.SessionName = SessionName;
			OutSettings.bPublicSearchable = SessionSettings->bShouldAdvertise;
			OutSettings.bAllowInvites = SessionSettings->bAllowInvites;
			OutSettings.bJoinViaPresence = SessionSettings->bAllowJoinViaPresence;
			OutSettings.bJoinViaPresenceFriendsOnly = SessionSettings->bAllowJoinViaPresenceFriendsOnly;
			bValidData = true;
		}
	}

	return bValidData;
}

void UOnlineEngineInterfaceImpl::UpdateSessionJoinability(UWorld* World, FName SessionName, bool bPublicSearchable, bool bAllowInvites, bool bJoinViaPresence, bool bJoinViaPresenceFriendsOnly)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		FOnlineSessionSettings* SessionSettings = SessionInt->GetSessionSettings(SessionName);
		if (SessionSettings != nullptr)
		{
			SessionSettings->bShouldAdvertise = bPublicSearchable;
			SessionSettings->bAllowInvites = bAllowInvites;
			SessionSettings->bAllowJoinViaPresence = bJoinViaPresence && !bJoinViaPresenceFriendsOnly;
			SessionSettings->bAllowJoinViaPresenceFriendsOnly = bJoinViaPresenceFriendsOnly;
			SessionInt->UpdateSession(SessionName, *SessionSettings, true);
		}
	}
}

void UOnlineEngineInterfaceImpl::RegisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId, bool bWasInvited)
{
	check(UniqueId.IsValid() && UniqueId.IsV1());
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid() && UniqueId.IsValid())
	{
		SessionInt->RegisterPlayer(SessionName, *UniqueId, bWasInvited);
	}
}

void UOnlineEngineInterfaceImpl::UnregisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId)
{
	check(UniqueId.IsValid() && UniqueId.IsV1());
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		SessionInt->UnregisterPlayer(SessionName, *UniqueId);
	}
}

void UOnlineEngineInterfaceImpl::UnregisterPlayers(UWorld* World, FName SessionName, const TArray<FUniqueNetIdWrapper>& Players)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		TArray<FUniqueNetIdRef> PlayerIdsAsRefs;
		for (const FUniqueNetIdWrapper& PlayerId : Players)
		{
			check(PlayerId.IsValid() && PlayerId.IsV1());
			PlayerIdsAsRefs.Emplace(PlayerId.GetV1().ToSharedRef());
		}
		SessionInt->UnregisterPlayers(SessionName, PlayerIdsAsRefs);
	}
}

bool UOnlineEngineInterfaceImpl::GetResolvedConnectString(UWorld* World, FName SessionName, FString& URL)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid() && SessionInt->GetResolvedConnectString(SessionName, URL))
	{
		return true;
	}

	return false;
}

TSharedPtr<FVoicePacket> UOnlineEngineInterfaceImpl::GetLocalPacket(UWorld* World, uint8 LocalUserNum)
{
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		TSharedPtr<FVoicePacket> LocalPacket = VoiceInt->GetLocalPacket(LocalUserNum);
		return LocalPacket;
	}

	return nullptr;
}

TSharedPtr<FVoicePacket> UOnlineEngineInterfaceImpl::SerializeRemotePacket(UWorld* World, const UNetConnection* const RemoteConnection, FArchive& Ar)
{
	FName VoiceSubsystemName = VoiceSubsystemNameOverride;
	if (RemoteConnection && RemoteConnection->Driver && RemoteConnection->Driver->GetNetMode() == NM_DedicatedServer)
	{
		FName DedicatedVoiceSubsystemName = GetDedicatedServerSubsystemNameForSubsystem(RemoteConnection->GetPlayerOnlinePlatformName());

		if (DedicatedVoiceSubsystemName != NAME_None)
		{
			VoiceSubsystemName = DedicatedVoiceSubsystemName;
		}
	}

	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemName);
	if (VoiceInt.IsValid())
	{
		return VoiceInt->SerializeRemotePacket(Ar);
	}
	return nullptr;
}

void UOnlineEngineInterfaceImpl::StartNetworkedVoice(UWorld* World, uint8 LocalUserNum)
{
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		VoiceInt->StartNetworkedVoice(LocalUserNum);
	}
}

void UOnlineEngineInterfaceImpl::StopNetworkedVoice(UWorld* World, uint8 LocalUserNum)
{
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		VoiceInt->StopNetworkedVoice(LocalUserNum);
	}
}

void UOnlineEngineInterfaceImpl::ClearVoicePackets(UWorld* World)
{
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		VoiceInt->ClearVoicePackets();
	}
}

bool UOnlineEngineInterfaceImpl::MuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide)
{
	check(PlayerId.IsValid() && PlayerId.IsV1());
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		return VoiceInt->MuteRemoteTalker(LocalUserNum, *PlayerId, bIsSystemWide);
	}
	return false;
}

bool UOnlineEngineInterfaceImpl::UnmuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide)
{
	check(PlayerId.IsValid() && PlayerId.IsV1());
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		return VoiceInt->UnmuteRemoteTalker(LocalUserNum, *PlayerId, bIsSystemWide);
	}
	return false;
}

int32 UOnlineEngineInterfaceImpl::GetNumLocalTalkers(UWorld* World)
{
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World, VoiceSubsystemNameOverride);
	if (VoiceInt.IsValid())
	{
		return VoiceInt->GetNumLocalTalkers();
	}

	return 0;
}

void UOnlineEngineInterfaceImpl::ShowLeaderboardUI(UWorld* World, const FString& CategoryName)
{
	IOnlineExternalUIPtr ExternalUI = Online::GetExternalUIInterface(World);
	if(ExternalUI.IsValid())
	{
		ExternalUI->ShowLeaderboardUI(CategoryName);
	}
}

void UOnlineEngineInterfaceImpl::ShowAchievementsUI(UWorld* World, int32 LocalUserNum)
{
	IOnlineExternalUIPtr ExternalUI = Online::GetExternalUIInterface(World);
	if (ExternalUI.IsValid())
	{
		ExternalUI->ShowAchievementsUI(LocalUserNum);
	}
}

#ifdef OSS_ADDED_SHOW_WEB
void UOnlineEngineInterfaceImpl::ShowWebURL(const FString& CurrentURL, const UOnlineEngineInterface::FShowWebUrlParams& ShowParams, const FOnlineShowWebUrlClosed& CompletionDelegate)
{
	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
	if (OnlineSub)
	{
		IOnlineExternalUIPtr ExternalUI = OnlineSub->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			::FShowWebUrlParams Params;
			Params.bEmbedded = ShowParams.bEmbedded;
			Params.bShowBackground = ShowParams.bShowBackground;
			Params.bShowCloseButton = ShowParams.bShowCloseButton;
			Params.bHideCursor = ShowParams.bHideCursor;
			Params.OffsetX = ShowParams.OffsetX;
			Params.OffsetY = ShowParams.OffsetY;
			Params.SizeX = ShowParams.SizeX;
			Params.SizeY = ShowParams.SizeY;

			ExternalUI->ShowWebURL(CurrentURL, Params, CompletionDelegate);
		}
	}
}

bool UOnlineEngineInterfaceImpl::CloseWebURL()
{
	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
	if (OnlineSub)
	{
		IOnlineExternalUIPtr ExternalUI = OnlineSub->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			return ExternalUI->CloseWebURL();
		}
	}

	return false;
}
#endif

void UOnlineEngineInterfaceImpl::BindToExternalUIOpening(const FOnlineExternalUIChanged& Delegate)
{
	IOnlineSubsystemUtils* Utils = Online::GetUtils();
	if (Utils)
	{
		FOnExternalUIChangeDelegate OnExternalUIChangeDelegate;
		OnExternalUIChangeDelegate.BindUObject(this, &ThisClass::OnExternalUIChange, Delegate);
		Utils->SetEngineExternalUIBinding(OnExternalUIChangeDelegate);
	}

	if (UOnlineEngineInterface* CompatibilityInterface = OnlineServicesCompatibilityInterface.Get())
	{
		CompatibilityInterface->BindToExternalUIOpening(Delegate);
	}
}

void UOnlineEngineInterfaceImpl::OnExternalUIChange(bool bInIsOpening, FOnlineExternalUIChanged Delegate)
{
	Delegate.ExecuteIfBound(bInIsOpening);
}

void UOnlineEngineInterfaceImpl::DumpSessionState(UWorld* World)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		SessionInt->DumpSessionState();
	}
}

void UOnlineEngineInterfaceImpl::DumpPartyState(UWorld* World)
{
	IOnlinePartyPtr PartyInt = Online::GetPartyInterface(World);
	if (PartyInt.IsValid())
	{
		PartyInt->DumpPartyState();
	}
}

void UOnlineEngineInterfaceImpl::DumpVoiceState(UWorld* World)
{
	IOnlineVoicePtr VoiceInt = Online::GetVoiceInterface(World);
	if (VoiceInt.IsValid())
	{
		UE_LOG_ONLINE(Verbose, TEXT("\n%s"), *VoiceInt->GetVoiceDebugState());
	}
}

void UOnlineEngineInterfaceImpl::DumpChatState(UWorld* World)
{
	IOnlineChatPtr ChatInt = Online::GetChatInterface(World);
	if (ChatInt.IsValid())
	{
		ChatInt->DumpChatState();
	}
}

#if WITH_EDITOR
bool UOnlineEngineInterfaceImpl::SupportsOnlinePIE()
{
	return Online::GetUtils()->SupportsOnlinePIE();
}

void UOnlineEngineInterfaceImpl::SetShouldTryOnlinePIE(bool bShouldTry)
{
	Online::GetUtils()->SetShouldTryOnlinePIE(bShouldTry);
}

int32 UOnlineEngineInterfaceImpl::GetNumPIELogins()
{
	return Online::GetUtils()->GetNumPIELogins();
}

FString UOnlineEngineInterfaceImpl::GetPIELoginCommandLineArgs(int32 Index)
{
	TArray<FOnlineAccountCredentials> Logins;
	Online::GetUtils()->GetPIELogins(Logins);
	if (Logins.IsValidIndex(Index))
	{
		return FString::Printf(TEXT(" -AUTH_TYPE=%s -AUTH_LOGIN=%s -AUTH_PASSWORD=%s"), *Logins[Index].Type, *Logins[Index].Id, *Logins[Index].Token);
	}
	return TEXT("");
}

void UOnlineEngineInterfaceImpl::SetForceDedicated(FName OnlineIdentifier, bool bForce)
{
	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get(OnlineIdentifier);
	if (OnlineSub)
	{
		OnlineSub->SetForceDedicated(bForce);
	}
}

void UOnlineEngineInterfaceImpl::LoginPIEInstance(FName OnlineIdentifier, int32 LocalUserNum, int32 PIELoginNum, FOnPIELoginComplete& CompletionDelegate)
{
	FString ErrorStr;
	if (SupportsOnlinePIE())
	{
		TArray<FOnlineAccountCredentials> PIELogins;
		Online::GetUtils()->GetPIELogins(PIELogins);
		if (PIELogins.IsValidIndex(PIELoginNum))
		{
			IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(OnlineIdentifier);
			if (IdentityInt.IsValid())
			{
				FDelegateHandle DelegateHandle = IdentityInt->AddOnLoginCompleteDelegate_Handle(LocalUserNum, FOnLoginCompleteDelegate::CreateUObject(this, &ThisClass::OnPIELoginComplete, OnlineIdentifier, CompletionDelegate));
				OnLoginPIECompleteDelegateHandlesForPIEInstances.Add(OnlineIdentifier, DelegateHandle);
				IdentityInt->Login(LocalUserNum, PIELogins[PIELoginNum]);
			}
			else
			{
				ErrorStr = TEXT("No identify interface to login");
			}
		}
		else
		{ 
			ErrorStr = FString::Printf(TEXT("Invalid credential index for PIE login. Index: %d NumLogins: %d"), PIELoginNum, PIELogins.Num());
		}
	}
	else
	{
		ErrorStr = TEXT("PIE login not supported");
	}

	if (!ErrorStr.IsEmpty())
	{
		CompletionDelegate.ExecuteIfBound(LocalUserNum, false, ErrorStr);
	}
}

void UOnlineEngineInterfaceImpl::OnPIELoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FName OnlineIdentifier, FOnlineAutoLoginComplete InCompletionDelegate)
{
	IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface(OnlineIdentifier);

	// Cleanup the login delegate before calling create below
	FDelegateHandle* DelegateHandle = OnLoginPIECompleteDelegateHandlesForPIEInstances.Find(OnlineIdentifier);
	if (DelegateHandle)
	{
		IdentityInt->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, *DelegateHandle);
		OnLoginPIECompleteDelegateHandlesForPIEInstances.Remove(OnlineIdentifier);
	}

	InCompletionDelegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, Error);
}

void UOnlineEngineInterfaceImpl::PIEEndCleanup(FName OnlineIdentifier)
{
	IOnlineSubsystem* Subsystem = IOnlineSubsystem::Get(OnlineIdentifier);

	if (FOnlineSubsystemImpl* SubsystemImpl = StaticCast<FOnlineSubsystemImpl*>(Subsystem))
	{
		SubsystemImpl->PIEEndCleanup();
	}
}

#endif

void UOnlineEngineInterfaceImpl::InitCompatibilityInterface()
{
	if (bOnlineServicesCompatibilityEnabled)
	{
		const UClass* OnlineEngineInterfaceClass = StaticLoadClass(UOnlineEngineInterface::StaticClass(), NULL, TEXT("/Script/OnlineSubsystemUtils.OnlineServicesEngineInterfaceImpl"), NULL, LOAD_Quiet, NULL);
		check(OnlineEngineInterfaceClass);

		OnlineServicesCompatibilityInterface = NewObject<UOnlineEngineInterface>(GetTransientPackage(), OnlineEngineInterfaceClass);
		check(OnlineServicesCompatibilityInterface);
	}
}
