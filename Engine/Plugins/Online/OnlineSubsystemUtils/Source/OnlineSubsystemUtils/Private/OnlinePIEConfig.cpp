// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePIEConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlinePIEConfig)

UOnlinePIEConfig::UOnlinePIEConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
