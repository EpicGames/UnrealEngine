// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "OnlinePIEConfig.generated.h"

struct FPropertyChangedEvent;

UCLASS(config=Editor)
class UOnlinePIEConfig : public UObject
{
	GENERATED_BODY()
public:
	UOnlinePIEConfig(const FObjectInitializer& ObjectInitializer);

	/** Type types of login credentials which allow duplicate entries. */
	UPROPERTY(config)
	TArray<FString> LoginTypesAllowingDuplicates;
};
