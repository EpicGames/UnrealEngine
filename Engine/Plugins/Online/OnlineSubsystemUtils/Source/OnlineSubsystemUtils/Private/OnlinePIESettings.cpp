// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePIESettings.h"
#include "OnlinePIEConfig.h"
#include "OnlineSubsystem.h"
#include "Misc/AES.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlinePIESettings)

const int32 ONLINEPIE_XOR_KEY = 0xdeadbeef;
void FOnlineAccountStoredCredentials::Encrypt()
{
	if (Token.Len() > 0)
	{
		TArray<TCHAR, FString::AllocatorType> SrcCharArray = Token.GetCharArray();
		int32 SrcSize = SrcCharArray.Num() * sizeof(TCHAR);
		const int64 PaddedEncryptedFileSize = Align(SrcSize + 1, FAES::AESBlockSize);

		TokenBytes.Empty(int32(PaddedEncryptedFileSize));
		TokenBytes.AddUninitialized(int32(PaddedEncryptedFileSize));

		// Store the length of the password
		ensure(SrcSize < MAX_uint8);
		TokenBytes[0] = static_cast<uint8>(SrcSize);

		// Copy the password in, leaving the random uninitialized data at the end
		FMemory::Memcpy(TokenBytes.GetData() + 1, SrcCharArray.GetData(), SrcSize);

		// XOR Cipher
		int32 NumXors = PaddedEncryptedFileSize / sizeof(int32);
		int32* Password = (int32*)(TokenBytes.GetData());
		for (int32 i = 0; i < NumXors; i++)
		{
			Password[i] = Password[i] ^ ONLINEPIE_XOR_KEY;
		}

		//FAES::EncryptData(TokenBytes.GetData(), PaddedEncryptedFileSize);
	}
	else
	{
		TokenBytes.Empty();
	}
}

void FOnlineAccountStoredCredentials::Decrypt()
{
	if (TokenBytes.Num() > 0)
	{
		const int64 PaddedEncryptedFileSize = Align(TokenBytes.Num(), FAES::AESBlockSize);
		if (PaddedEncryptedFileSize > 0 && PaddedEncryptedFileSize == TokenBytes.Num())
		{
			TArray<uint8> TempArray;
			TempArray.Empty(int32(PaddedEncryptedFileSize));
			TempArray.AddUninitialized(int32(PaddedEncryptedFileSize));
			FMemory::Memcpy(TempArray.GetData(), TokenBytes.GetData(), PaddedEncryptedFileSize);
			
			// XOR Cipher
			int32 NumXors = PaddedEncryptedFileSize / sizeof(int32);
			int32* TempArrayPtr = (int32*)(TempArray.GetData());
			for (int32 i = 0; i < NumXors; i++)
			{
				TempArrayPtr[i] = TempArrayPtr[i] ^ ONLINEPIE_XOR_KEY;
			}

			//FAES::DecryptData(TempArray.GetData(), PaddedEncryptedFileSize);

			// Validate the unencrypted data (stored size less than total data, null terminated character where it should be)
			int32 PasswordDataSize = TempArray[0];
			int32 PasswordLength = PasswordDataSize / sizeof(TCHAR);
			TCHAR* Password = (TCHAR*)(TempArray.GetData() + 1);
			if (PasswordDataSize < TempArray.Num() &&
				Password[PasswordLength - 1] == '\0')
			{
				Token = FString(Password);
			}
			else
			{
				Token.Empty();
				TokenBytes.Empty();
			}
		}
	}
	else
	{
		Token.Empty();
	}
}

UOnlinePIESettings::UOnlinePIESettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bOnlinePIEEnabled = false;
	CategoryName = FName(TEXT("LevelEditor"));
}

void UOnlinePIESettings::PostInitProperties()
{
	Super::PostInitProperties();

	for (FOnlineAccountStoredCredentials& Login : Logins)
	{
		Login.Decrypt();
	}
}

#if WITH_EDITOR
void UOnlinePIESettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != NULL)
	{
		FName MemberPropName = PropertyChangedEvent.MemberProperty->GetFName();
		if (MemberPropName == GET_MEMBER_NAME_CHECKED(UOnlinePIESettings, bOnlinePIEEnabled))
		{
			// Possibly get rid of the null subsystem in favor of the real default or if we are disabling online pie then get rid of the real subsystem to replace it with null
			IOnlineSubsystem::ReloadDefaultSubsystem();
		}
		else if (MemberPropName == GET_MEMBER_NAME_CHECKED(UOnlinePIESettings, Logins))
		{
			FName SubPropName = PropertyChangedEvent.Property->GetFName();

			// If we paste on top of the whole login entry, all fields will have changed and need their checks run.
			const bool bPastedAllValues = SubPropName == GET_MEMBER_NAME_CHECKED(UOnlinePIESettings, Logins);
			
			if (bPastedAllValues || SubPropName == GET_MEMBER_NAME_CHECKED(FOnlineAccountStoredCredentials, Id))
			{
				TSet<FString> Ids;
				for (FOnlineAccountStoredCredentials& Login : Logins)
				{
					// Remove any whitespace from login input
					Login.Id.TrimStartAndEndInline();

					const bool bIsExistingLogin = Ids.Contains(Login.Id);
					// Allow duplicate credentials or when allowed.  This is intended to support logins where other fields uniquely identify the login but require the same id.
					const UOnlinePIEConfig* OnlinePIEConfig = GetDefault<UOnlinePIEConfig>();
					const bool bDuplicateAllowed = OnlinePIEConfig->LoginTypesAllowingDuplicates.Contains(Login.Type);
					if (bIsExistingLogin && !bDuplicateAllowed)
					{
						// Don't allow duplicate login ids
						Login.Id.Reset();
					}
					else
					{
						Ids.Add(Login.Id);
					}
				}
			}

			if (bPastedAllValues || SubPropName == GET_MEMBER_NAME_CHECKED(FOnlineAccountStoredCredentials, Token))
			{
				for (FOnlineAccountStoredCredentials& Login : Logins)
				{
					// Remove any whitespace from login input
					Login.Token.TrimStartAndEndInline();
					// Encrypt the password
					Login.Encrypt();
				}
			}

			if (bPastedAllValues || SubPropName == GET_MEMBER_NAME_CHECKED(FOnlineAccountStoredCredentials, Type))
			{
				for (FOnlineAccountStoredCredentials& Login : Logins)
				{
					// Remove any whitespace from login input
					Login.Type.TrimStartAndEndInline();
				}
			}
		}
	}
}

#endif // WITH_EDITOR

