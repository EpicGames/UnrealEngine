// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "OnlineAccountStoredCredentials.h"
#include "OnlinePIESettings.generated.h"

struct FPropertyChangedEvent;

/**
 * Setup login credentials for the Play In Editor (PIE) feature
 */
UCLASS(config=EditorPerProjectUserSettings, meta = (DisplayName = "Play Credentials"))
class UOnlinePIESettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UOnlinePIESettings(const FObjectInitializer& ObjectInitializer);

	/** Will Play In Editor (PIE) attempt to login to a platform service before launching the instance */
	UPROPERTY(config, EditAnywhere, Category = "Logins", meta = (DisplayName = "Enable Logins", Tooltip = "Attempt to login with user credentials on a backend service before launching the PIE instance."))
	bool bOnlinePIEEnabled;

	/** Array of credentials to use, one for each Play In Editor (PIE) instance */
	UPROPERTY(config, EditAnywhere, Category = "Logins", meta = (DisplayName = "Credentials", Tooltip = "Login credentials, at least one for each instance of PIE that is intended to be run"))
	TArray<FOnlineAccountStoredCredentials> Logins;

	// Begin UObject Interface
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR
	// End UObject Interface
};
