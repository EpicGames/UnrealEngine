// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineServicesEngineInterfaceImpl.h"
#include "Online/OnlineResult.h"
#include "Online/OnlineServicesEngineUtils.h"

#include "Online/OnlineAsyncOpHandle.h"
#include "Online/OnlineServicesRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineServicesEngineInterfaceImpl)

UOnlineServicesEngineInterfaceImpl::UOnlineServicesEngineInterfaceImpl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UOnlineServicesEngineInterfaceImpl::IsLoaded(FName OnlineIdentifier)
{
	// TODO: something like OnlineServicesTypeFromName(OnlineIdentifier); ? This seems to only be called with NAME_None
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
	return UE::Online::IsLoaded(OnlineServicesType, OnlineIdentifier);
}

FName UOnlineServicesEngineInterfaceImpl::GetOnlineIdentifier(FWorldContext& WorldContext)
{
	UE::Online::IOnlineServicesEngineUtils* Utils = UE::Online::GetServicesEngineUtils();
	if (Utils)
	{
		return Utils->GetOnlineIdentifier(WorldContext);
	}
	return NAME_None;
}

FName UOnlineServicesEngineInterfaceImpl::GetOnlineIdentifier(UWorld* World) const
{
	UE::Online::IOnlineServicesEngineUtils* Utils = UE::Online::GetServicesEngineUtils();
	if (Utils)
	{
		return Utils->GetOnlineIdentifier(World);
	}
	return NAME_None;
}

bool UOnlineServicesEngineInterfaceImpl::DoesInstanceExist(FName OnlineIdentifier)
{
	// TODO:  Does this need to support multiple online service types?  Other accessors seem to not differentiate and just use the default
	return UE::Online::IsLoaded(UE::Online::EOnlineServices::Default, OnlineIdentifier);
}

void UOnlineServicesEngineInterfaceImpl::ShutdownOnlineSubsystem(FName OnlineIdentifier)
{
	// While using OSS Adaptors, need to destroy OnlineServices instances here instead of shutting down, to 
	// make sure the shared ptr references to OSSv1 interfaces in oss adaptors are removed
	UE::Online::DestroyAllServicesWithName(OnlineIdentifier);
}

void UOnlineServicesEngineInterfaceImpl::DestroyOnlineSubsystem(FName OnlineIdentifier)
{
}

FName UOnlineServicesEngineInterfaceImpl::GetDefaultOnlineSubsystemName() const
{
	// TODO: Does this have any meaning?  Does this need to be replaced with some kind of FName/EOnlineServices wrapper?
	return NAME_None;
}

bool UOnlineServicesEngineInterfaceImpl::IsCompatibleUniqueNetId(const FUniqueNetIdWrapper& InUniqueNetId) const
{
	// TODO: Compatibility rules
	return InUniqueNetId.IsV2();
}

uint8 UOnlineServicesEngineInterfaceImpl::GetReplicationHashForSubsystem(FName InSubsystemName) const
{
	// This is used for net id replication, which doesn't need to call this for V2 ids.
	checkNoEntry();
	return 0;
}

FName UOnlineServicesEngineInterfaceImpl::GetSubsystemFromReplicationHash(uint8 InHash) const
{
	// This is used for net id replication, which doesn't need to call this for V2 ids.
	checkNoEntry();
	return NAME_None;
}

FUniqueNetIdWrapper UOnlineServicesEngineInterfaceImpl::CreateUniquePlayerIdWrapper(const FString& Str, FName Type)
{
	// This should not be called for OSSv2
	checkNoEntry();
	return FUniqueNetIdWrapper();
}

FUniqueNetIdWrapper UOnlineServicesEngineInterfaceImpl::GetUniquePlayerIdWrapper(UWorld* World, int32 LocalUserNum, FName Type)
{
	// TODO: something like OnlineServicesTypeFromName(Type); ? This seems to only be called with NAME_None in LocalPlayer and with the platform service in OnlineReplStructs in a test command
	FName OnlineIdentifier = GetOnlineIdentifier(World);
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		if (UE::Online::IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
		{
			UE::Online::FAuthGetLocalOnlineUserByPlatformUserId::Params GetAccountParams = { FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum) };
			UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByPlatformUserId> GetAccountResult = AuthPtr->GetLocalOnlineUserByPlatformUserId(MoveTemp(GetAccountParams));
			if (GetAccountResult.IsOk())
			{
				return FUniqueNetIdWrapper(GetAccountResult.GetOkValue().AccountInfo->AccountId);
			}
		}
	}
	return FUniqueNetIdWrapper();
}

FString UOnlineServicesEngineInterfaceImpl::GetPlayerNickname(UWorld* World, const FUniqueNetIdWrapper& UniqueId)
{
	check(UniqueId.IsValid() && UniqueId.IsV2());
	FName OnlineIdentifier = GetOnlineIdentifier(World);
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default; // TODO: Get from UniqueId
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		if (UE::Online::IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
		{
			UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId::Params GetAccountParams = { UniqueId.GetV2() };
			UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> GetAccountResult = AuthPtr->GetLocalOnlineUserByOnlineAccountId(MoveTemp(GetAccountParams));
			if (GetAccountResult.IsOk())
			{
				const UE::Online::FSchemaVariant* DisplayName = GetAccountResult.GetOkValue().AccountInfo->Attributes.Find(UE::Online::AccountAttributeData::DisplayName);
				return DisplayName ? DisplayName->GetString() : FString();
			}
		}
	}
	static FString InvalidName(TEXT("InvalidOSSUser"));
	return InvalidName;
}

bool UOnlineServicesEngineInterfaceImpl::GetPlayerPlatformNickname(UWorld* World, int32 LocalUserNum, FString& OutNickname)
{
	FName OnlineIdentifier = GetOnlineIdentifier(World);
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Platform;
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		if (UE::Online::IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
		{
			UE::Online::FAuthGetLocalOnlineUserByPlatformUserId::Params GetAccountParams = { FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum) };
			UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByPlatformUserId> GetAccountResult = AuthPtr->GetLocalOnlineUserByPlatformUserId(MoveTemp(GetAccountParams));
			if (GetAccountResult.IsOk())
			{
				if (const UE::Online::FSchemaVariant* DisplayName = GetAccountResult.GetOkValue().AccountInfo->Attributes.Find(UE::Online::AccountAttributeData::DisplayName))
				{
					OutNickname = DisplayName->GetString();
				}
				return !OutNickname.IsEmpty();
			}
		}
	}
	return false;
}

bool UOnlineServicesEngineInterfaceImpl::AutoLogin(UWorld* World, int32 LocalUserNum, const FOnlineAutoLoginComplete& InCompletionDelegate)
{
	FName OnlineIdentifier = GetOnlineIdentifier(World);
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		if (UE::Online::IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
		{
			UE::Online::FAuthLogin::Params LoginParameters;
			LoginParameters.PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum);
			// Leave other LoginParameters as default to allow the online service to determine how to try to automatically log in the user
			UE::Online::TOnlineAsyncOpHandle<UE::Online::FAuthLogin> LoginHandle = AuthPtr->Login(MoveTemp(LoginParameters));
			LoginHandle.OnComplete(this, [LocalUserNum, InCompletionDelegate](const UE::Online::TOnlineResult<UE::Online::FAuthLogin>& Result)
			{
				FString ErrorCode = Result.IsError() ? Result.GetErrorValue().GetLogString() : FString();
				InCompletionDelegate.ExecuteIfBound(LocalUserNum, Result.IsOk(), ErrorCode);
			});
			return true;
		}
	}
	// Not waiting for async login
	return false;
}

bool UOnlineServicesEngineInterfaceImpl::IsLoggedIn(UWorld* World, int32 LocalUserNum)
{
	FName OnlineIdentifier = GetOnlineIdentifier(World);
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		if (UE::Online::IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
		{
			UE::Online::FAuthGetLocalOnlineUserByPlatformUserId::Params GetAccountParams = { FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum) };
			UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByPlatformUserId> GetAccountResult = AuthPtr->GetLocalOnlineUserByPlatformUserId(MoveTemp(GetAccountParams));
			if (GetAccountResult.IsOk())
			{
				return GetAccountResult.GetOkValue().AccountInfo->LoginStatus == UE::Online::ELoginStatus::LoggedIn;
			}
		}
	}
	return false;
}

void UOnlineServicesEngineInterfaceImpl::StartSession(UWorld* World, FName SessionName, FOnlineSessionStartComplete& InCompletionDelegate)
{
	// TODO: Implement when we have session/lobby support
	InCompletionDelegate.ExecuteIfBound(SessionName, false);
}

void UOnlineServicesEngineInterfaceImpl::EndSession(UWorld* World, FName SessionName, FOnlineSessionEndComplete& InCompletionDelegate)
{
	// TODO: Implement when we have session/lobby support
	InCompletionDelegate.ExecuteIfBound(SessionName, false);
}

bool UOnlineServicesEngineInterfaceImpl::DoesSessionExist(UWorld* World, FName SessionName)
{
	// TODO: Implement when we have session/lobby support
	return false;
}

bool UOnlineServicesEngineInterfaceImpl::GetSessionJoinability(UWorld* World, FName SessionName, FJoinabilitySettings& OutSettings)
{
	// TODO: Implement when we have session/lobby support
	return false;
}

void UOnlineServicesEngineInterfaceImpl::UpdateSessionJoinability(UWorld* World, FName SessionName, bool bPublicSearchable, bool bAllowInvites, bool bJoinViaPresence, bool bJoinViaPresenceFriendsOnly)
{
	// TODO: Implement when we have session/lobby support
}

void UOnlineServicesEngineInterfaceImpl::RegisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId, bool bWasInvited)
{
	// TODO: Implement when we have session/lobby support
	check(UniqueId.IsValid() && UniqueId.IsV2());
}

void UOnlineServicesEngineInterfaceImpl::UnregisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId)
{
	// TODO: Implement when we have session/lobby support
	check(UniqueId.IsValid() && UniqueId.IsV2());
}

void UOnlineServicesEngineInterfaceImpl::UnregisterPlayers(UWorld* World, FName SessionName, const TArray<FUniqueNetIdWrapper>& Players)
{
	// TODO: Implement when we have session/lobby support
	for (const FUniqueNetIdWrapper& PlayerId : Players)
	{
		check(PlayerId.IsValid() && PlayerId.IsV2());
	}
}

bool UOnlineServicesEngineInterfaceImpl::GetResolvedConnectString(UWorld* World, FName SessionName, FString& URL)
{
	// TODO: Implement when we have session/lobby support
	return false;
}

TSharedPtr<FVoicePacket> UOnlineServicesEngineInterfaceImpl::GetLocalPacket(UWorld* World, uint8 LocalUserNum)
{
	// TODO: Implement when we have voice support
	return nullptr;
}

TSharedPtr<FVoicePacket> UOnlineServicesEngineInterfaceImpl::SerializeRemotePacket(UWorld* World, const UNetConnection* const RemoteConnection, FArchive& Ar)
{
	// TODO: Implement when we have voice support
	return nullptr;
}

void UOnlineServicesEngineInterfaceImpl::StartNetworkedVoice(UWorld* World, uint8 LocalUserNum)
{
	// TODO: Implement when we have voice support
}

void UOnlineServicesEngineInterfaceImpl::StopNetworkedVoice(UWorld* World, uint8 LocalUserNum)
{
	// TODO: Implement when we have voice support
}

void UOnlineServicesEngineInterfaceImpl::ClearVoicePackets(UWorld* World)
{
	// TODO: Implement when we have voice support
}

bool UOnlineServicesEngineInterfaceImpl::MuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide)
{
	// TODO: Implement when we have voice support
	check(PlayerId.IsValid() && PlayerId.IsV2());
	return false;
}

bool UOnlineServicesEngineInterfaceImpl::UnmuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide)
{
	// TODO: Implement when we have voice support
	check(PlayerId.IsValid() && PlayerId.IsV2());
	return false;
}

int32 UOnlineServicesEngineInterfaceImpl::GetNumLocalTalkers(UWorld* World)
{
	// TODO: Implement when we have voice support
	return 0;
}

void UOnlineServicesEngineInterfaceImpl::ShowLeaderboardUI(UWorld* World, const FString& CategoryName)
{
	FName OnlineIdentifier = GetOnlineIdentifier(World);
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		UE::Online::IExternalUIPtr ExternalUI = OnlineServices->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			// TODO: When we have more external UI support
		}
	}
}

void UOnlineServicesEngineInterfaceImpl::ShowAchievementsUI(UWorld* World, int32 LocalUserNum)
{
	TArray<TSharedRef<UE::Online::IOnlineServices>> ServicesInstances;
	UE::Online::FOnlineServicesRegistry::Get().GetAllServicesInstances(ServicesInstances);
	for (const TSharedRef<UE::Online::IOnlineServices>& OnlineServices : ServicesInstances)
	{
		UE::Online::IExternalUIPtr ExternalUI = OnlineServices->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			// TODO: When we have more external UI support
		}
	}
}

void UOnlineServicesEngineInterfaceImpl::ShowWebURL(const FString& CurrentURL, const UOnlineEngineInterface::FShowWebUrlParams& ShowParams, const FOnlineShowWebUrlClosed& CompletionDelegate)
{
	TArray<TSharedRef<UE::Online::IOnlineServices>> ServicesInstances;
	UE::Online::FOnlineServicesRegistry::Get().GetAllServicesInstances(ServicesInstances);
	for (const TSharedRef<UE::Online::IOnlineServices>& OnlineServices : ServicesInstances)
	{
		UE::Online::IExternalUIPtr ExternalUI = OnlineServices->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			// TODO: When we have more external UI support
		}
	}
}

bool UOnlineServicesEngineInterfaceImpl::CloseWebURL()
{
	TArray<TSharedRef<UE::Online::IOnlineServices>> ServicesInstances;
	UE::Online::FOnlineServicesRegistry::Get().GetAllServicesInstances(ServicesInstances);
	for (const TSharedRef<UE::Online::IOnlineServices>& OnlineServices : ServicesInstances)
	{
		UE::Online::IExternalUIPtr ExternalUI = OnlineServices->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			// TODO: When we have more external UI support
		}
	}
	return false;
}

void UOnlineServicesEngineInterfaceImpl::BindToExternalUIOpening(const FOnlineExternalUIChanged& Delegate)
{
	UE::Online::IOnlineServicesEngineUtils* Utils = UE::Online::GetServicesEngineUtils();
	if (Utils)
	{
		FOnExternalUIChangeDelegate OnExternalUIChangeDelegate;
		OnExternalUIChangeDelegate.BindWeakLambda(this, [Delegate](bool bInIsOpening)
		{
			Delegate.ExecuteIfBound(bInIsOpening);
		});
		Utils->SetEngineExternalUIBinding(OnExternalUIChangeDelegate);
	}
}

void UOnlineServicesEngineInterfaceImpl::DumpSessionState(UWorld* World)
{
	// TODO: Implement when we have session/lobby support
}

void UOnlineServicesEngineInterfaceImpl::DumpPartyState(UWorld* World)
{
	// TODO: Implement when we have lobby support
}

void UOnlineServicesEngineInterfaceImpl::DumpVoiceState(UWorld* World)
{
	// TODO: Implement when we have voice support
}

void UOnlineServicesEngineInterfaceImpl::DumpChatState(UWorld* World)
{
	// TODO: Implement when we have chat support
}

#if WITH_EDITOR
bool UOnlineServicesEngineInterfaceImpl::SupportsOnlinePIE()
{
	return UE::Online::GetServicesEngineUtils()->SupportsOnlinePIE();
}

void UOnlineServicesEngineInterfaceImpl::SetShouldTryOnlinePIE(bool bShouldTry)
{
	UE::Online::GetServicesEngineUtils()->SetShouldTryOnlinePIE(bShouldTry);
}

int32 UOnlineServicesEngineInterfaceImpl::GetNumPIELogins()
{
	return UE::Online::GetServicesEngineUtils()->GetNumPIELogins();
}

FString UOnlineServicesEngineInterfaceImpl::GetPIELoginCommandLineArgs(int32 Index)
{
	TArray<UE::Online::FAuthLogin::Params> LoginParams;
	UE::Online::GetServicesEngineUtils()->GetPIELogins(LoginParams);
	if (LoginParams.IsValidIndex(Index))
	{
		const FString& Type = LoginParams[Index].CredentialsType.ToString();
		const FString& Id = LoginParams[Index].CredentialsId;
		const FString& Token = LoginParams[Index].CredentialsToken.IsType<FString>() ? LoginParams[Index].CredentialsToken.Get<FString>() : LoginParams[Index].CredentialsToken.Get<UE::Online::FExternalAuthToken>().Data;

		return FString::Printf(TEXT(" -AUTH_TYPE=%s -AUTH_LOGIN=%s -AUTH_PASSWORD=%s"), *Type, *Id, *Token);
	}
	return TEXT("");
}

void UOnlineServicesEngineInterfaceImpl::SetForceDedicated(FName OnlineIdentifier, bool bForce)
{
	// TODO:  Support other services? This is only called from PlayLevel with the result of GetOnlineIdentifier which only returns the default service id
	UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
	if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
	{
		// TODO:  Add SetForceDedicated method to OnlineServices
	}
}

void UOnlineServicesEngineInterfaceImpl::LoginPIEInstance(FName OnlineIdentifier, int32 LocalUserNum, int32 PIELoginNum, FOnPIELoginComplete& CompletionDelegate)
{
	FString ErrorStr;
	if (SupportsOnlinePIE())
	{
		TArray<UE::Online::FAuthLogin::Params> PIELogins;
		UE::Online::GetServicesEngineUtils()->GetPIELogins(PIELogins);
		if (PIELogins.IsValidIndex(PIELoginNum))
		{
			// TODO:  Support other services? This is only called from PlayLevel with the result of GetOnlineIdentifier which only returns the default service id
			UE::Online::EOnlineServices OnlineServicesType = UE::Online::EOnlineServices::Default;
			if (UE::Online::IOnlineServicesPtr OnlineServices = UE::Online::GetServices(OnlineServicesType, OnlineIdentifier))
			{
				if (UE::Online::IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
				{
					UE::Online::FAuthLogin::Params LoginParameters = PIELogins[PIELoginNum];
					LoginParameters.PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum);
					AuthPtr->Login(MoveTemp(LoginParameters)).OnComplete(this, [LocalUserNum, CompletionDelegate](const UE::Online::TOnlineResult<UE::Online::FAuthLogin>& Result)
					{
						FString AuthLoginErrorStr;
						if (Result.IsError())
						{
							AuthLoginErrorStr = Result.GetErrorValue().GetLogString();
						}
						CompletionDelegate.ExecuteIfBound(LocalUserNum, Result.IsOk(), AuthLoginErrorStr);
					});
					return;
				}
				else
				{
					ErrorStr = TEXT("No auth interface to login");
				}
			}
			else
			{
				ErrorStr = TEXT("No online service to login");
			}
		}
		else
		{
			ErrorStr = FString::Printf(TEXT("Invalid credential index for PIE login. Index: %d NumLogins: %d"), PIELoginNum, PIELogins.Num());
		}
	}
	else
	{
		ErrorStr = TEXT("PIE login not supported");
	}

	if (!ErrorStr.IsEmpty())
	{
		CompletionDelegate.ExecuteIfBound(LocalUserNum, false, ErrorStr);
	}
}

#endif

