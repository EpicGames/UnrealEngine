// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/OnlineEngineInterface.h"
#include "OnlineServicesEngineInterfaceImpl.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FJoinabilitySettings;
struct FUniqueNetIdWrapper;

class Error;
class FVoicePacket;
struct FWorldContext;

/**
 * Implementation of UOnlineEngineInterface that uses Online Services (also known as Online Subsystem v2)
 */
UCLASS(MinimalAPI, config = Engine)
class UOnlineServicesEngineInterfaceImpl : public UOnlineEngineInterface
{
	GENERATED_UCLASS_BODY()

	/**
	 * Subsystem
	 */
public:

	UE_API virtual bool IsLoaded(FName OnlineIdentifier) override;
	UE_API virtual FName GetOnlineIdentifier(FWorldContext& WorldContext) override;
	UE_API virtual bool DoesInstanceExist(FName OnlineIdentifier) override;
	// NOTE: In OnlineServices it destroys all types of OnlineServices instances which contains this identifier
	UE_API virtual void ShutdownOnlineSubsystem(FName OnlineIdentifier) override;
	UE_API virtual void DestroyOnlineSubsystem(FName OnlineIdentifier) override;
	UE_API virtual FName GetDefaultOnlineSubsystemName() const override;
	UE_API virtual bool IsCompatibleUniqueNetId(const FUniqueNetIdWrapper& InUniqueNetId) const override;

	/**
	 * Utils
	 */
	UE_API virtual uint8 GetReplicationHashForSubsystem(FName InSubsystemName) const override;
	UE_API virtual FName GetSubsystemFromReplicationHash(uint8 InHash) const override;

private:
	/** @return the identifier/context handle associated with a UWorld */
	FName GetOnlineIdentifier(UWorld* World) const;

	/**
	 * Identity
	 */
public:

	UE_API virtual FUniqueNetIdWrapper CreateUniquePlayerIdWrapper(const FString& Str, FName Type) override;
	UE_API virtual FUniqueNetIdWrapper GetUniquePlayerIdWrapper(UWorld* World, int32 LocalUserNum, FName Type) override;

	UE_API virtual FString GetPlayerNickname(UWorld* World, const FUniqueNetIdWrapper& UniqueId) override;
	UE_API virtual bool GetPlayerPlatformNickname(UWorld* World, int32 LocalUserNum, FString& OutNickname) override;

	UE_API virtual bool AutoLogin(UWorld* World, int32 LocalUserNum, const FOnlineAutoLoginComplete& InCompletionDelegate) override;
	UE_API virtual bool IsLoggedIn(UWorld* World, int32 LocalUserNum) override;

	/**
	 * Session
	 */
public:

	UE_API virtual void StartSession(UWorld* World, FName SessionName, FOnlineSessionStartComplete& InCompletionDelegate) override;
	UE_API virtual void EndSession(UWorld* World, FName SessionName, FOnlineSessionEndComplete& InCompletionDelegate) override;
	UE_API virtual bool DoesSessionExist(UWorld* World, FName SessionName) override;

	UE_API virtual bool GetSessionJoinability(UWorld* World, FName SessionName, FJoinabilitySettings& OutSettings) override;
	UE_API virtual void UpdateSessionJoinability(UWorld* World, FName SessionName, bool bPublicSearchable, bool bAllowInvites, bool bJoinViaPresence, bool bJoinViaPresenceFriendsOnly) override;

	UE_API virtual void RegisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId, bool bWasInvited) override;
	UE_API virtual void UnregisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId) override;
	UE_API virtual void UnregisterPlayers(UWorld* World, FName SessionName, const TArray<FUniqueNetIdWrapper>& Players) override;

	UE_API virtual bool GetResolvedConnectString(UWorld* World, FName SessionName, FString& URL) override;

	/**
	 * Voice
	 */
public:

	UE_API virtual TSharedPtr<FVoicePacket> GetLocalPacket(UWorld* World, uint8 LocalUserNum) override;
	UE_API virtual TSharedPtr<FVoicePacket> SerializeRemotePacket(UWorld* World, const UNetConnection* const RemoteConnection, FArchive& Ar) override;

	UE_API virtual void StartNetworkedVoice(UWorld* World, uint8 LocalUserNum) override;
	UE_API virtual void StopNetworkedVoice(UWorld* World, uint8 LocalUserNum) override;
	UE_API virtual void ClearVoicePackets(UWorld* World) override;

	UE_API virtual bool MuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide) override;
	UE_API virtual bool UnmuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide) override;

	UE_API virtual int32 GetNumLocalTalkers(UWorld* World) override;

	/**
	 * External UI
	 */
public:

	UE_API virtual void ShowLeaderboardUI(UWorld* World, const FString& CategoryName) override;
	UE_API virtual void ShowAchievementsUI(UWorld* World, int32 LocalUserNum) override;
	UE_API virtual void BindToExternalUIOpening(const FOnlineExternalUIChanged& Delegate) override;
	UE_API virtual void ShowWebURL(const FString& CurrentURL, const UOnlineEngineInterface::FShowWebUrlParams& ShowParams, const FOnlineShowWebUrlClosed& CompletionDelegate) override;
	UE_API virtual bool CloseWebURL() override;

	/**
	 * Debug
	 */
public:

	UE_API virtual void DumpSessionState(UWorld* World) override;
	UE_API virtual void DumpPartyState(UWorld* World) override;
	UE_API virtual void DumpVoiceState(UWorld* World) override;
	UE_API virtual void DumpChatState(UWorld* World) override;

#if WITH_EDITOR
	/**
	 * PIE Utilities
	 */
public:

	UE_API virtual bool SupportsOnlinePIE() override;
	UE_API virtual void SetShouldTryOnlinePIE(bool bShouldTry) override;
	UE_API virtual int32 GetNumPIELogins() override;
	UE_API virtual FString GetPIELoginCommandLineArgs(int32 Index) override;
	UE_API virtual void SetForceDedicated(FName OnlineIdentifier, bool bForce) override;
	UE_API virtual void LoginPIEInstance(FName OnlineIdentifier, int32 LocalUserNum, int32 PIELoginNum, FOnPIELoginComplete& CompletionDelegate) override;
#endif
};

#undef UE_API
