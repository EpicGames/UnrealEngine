// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineServicesEngineUtilsImpl.h"
#include "OnlinePIESettings.h"
#include "Modules/ModuleManager.h"
#include "Engine/Engine.h"
#include "OnlineSubsystemUtilsModule.h"

#include "Online/ExternalUI.h"
#include "Online/OnlineServicesDelegates.h"
#include "Online/OnlineServicesRegistry.h"

#if WITH_EDITOR
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#endif


namespace UE::Online
{

IOnlineServicesEngineUtils* GetServicesEngineUtils()
{
	static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystemUtils");
	FOnlineSubsystemUtilsModule* OSSUtilsModule = FModuleManager::GetModulePtr<FOnlineSubsystemUtilsModule>(OnlineSubsystemModuleName);
	if (OSSUtilsModule != nullptr)
	{
		return OSSUtilsModule->GetServicesUtils();
	}

	return nullptr;
}

FOnlineServicesEngineUtils::~FOnlineServicesEngineUtils()
{
	UE::Online::OnOnlineServicesCreated.Remove(OnOnlineServicesCreatedDelegateHandle);
}

FName FOnlineServicesEngineUtils::GetOnlineIdentifier(const FWorldContext& WorldContext) const
{
#if WITH_EDITOR
	if (WorldContext.WorldType == EWorldType::PIE)
	{
		return WorldContext.ContextHandle;
	}
#endif
	return NAME_None;
}

FName FOnlineServicesEngineUtils::GetOnlineIdentifier(const UWorld* World) const
{
#if WITH_EDITOR
	if (const FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World))
	{
		return GetOnlineIdentifier(*WorldContext);
	}
#endif
	return NAME_None;
}

void FOnlineServicesEngineUtils::OnOnlineServicesCreated(TSharedRef<class IOnlineServices> NewServices)
{
	if (OnExternalUIChangeDelegate.IsBound())
	{
		IExternalUIPtr ExternalUI = NewServices->GetExternalUIInterface();
		if (ExternalUI.IsValid())
		{
			ExternalUIDelegateHandles.Emplace(ExternalUI->OnExternalUIStatusChanged().Add([this](const FExternalUIStatusChanged& EventParams)
			{
				OnExternalUIChangeDelegate.ExecuteIfBound(EventParams.bIsOpening);
			}));
		}
	}
}

void FOnlineServicesEngineUtils::SetEngineExternalUIBinding(const FOnExternalUIChangeDelegate& InOnExternalUIChangeDelegate)
{
	OnExternalUIChangeDelegate = InOnExternalUIChangeDelegate;
	TArray<TSharedRef<IOnlineServices>> ServicesInstances;
	FOnlineServicesRegistry::Get().GetAllServicesInstances(ServicesInstances);
	for (const TSharedRef<IOnlineServices>& ServiceInstance : ServicesInstances)
	{
		if (IExternalUIPtr ExternalUI = ServiceInstance->GetExternalUIInterface())
		{
			ExternalUIDelegateHandles.Emplace(ExternalUI->OnExternalUIStatusChanged().Add([this](const FExternalUIStatusChanged& EventParams)
			{
				OnExternalUIChangeDelegate.ExecuteIfBound(EventParams.bIsOpening);
			}));
		}
	};
}

#if WITH_EDITOR

namespace FOnlineServicesEngineUtilsHelper
{
	bool HasAuthCommandLine(UE::Online::FAuthLogin::Params& LoginParameters)
    {
    	check(LoginParameters.CredentialsToken.IsType<FString>());
    	return FParse::Value(FCommandLine::Get(), TEXT("AUTH_LOGIN="), LoginParameters.CredentialsId ) &&
    		   FParse::Value(FCommandLine::Get(), TEXT("AUTH_PASSWORD="), LoginParameters.CredentialsToken.Get<FString>()) &&
    		   FParse::Value(FCommandLine::Get(), TEXT("AUTH_TYPE="), LoginParameters.CredentialsType);
    }
	bool HasAuthCommandLine()
	{
		UE::Online::FAuthLogin::Params LoginParameters;
		return HasAuthCommandLine(LoginParameters);
	}
}

bool FOnlineServicesEngineUtils::SupportsOnlinePIE() const
{
	check(UObjectInitialized());
	const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
	if ((OnlinePIESettings->bOnlinePIEEnabled && GetNumPIELogins() > 0) || FOnlineServicesEngineUtilsHelper::HasAuthCommandLine())
	{
		// If we can't get the auth then things are either not configured right or disabled
		EOnlineServices OnlineServicesType = EOnlineServices::Epic; // TODO:  Need Default support UE::Online::EOnlineServices::Default
		if (IOnlineServicesPtr OnlineServices = GetServices(OnlineServicesType))
		{
			if (IAuthPtr AuthPtr = OnlineServices->GetAuthInterface())
			{
				return true;
			}
		}
	}
	return false;
}

void FOnlineServicesEngineUtils::SetShouldTryOnlinePIE(bool bShouldTry)
{
	if (bShouldTryOnlinePIE != bShouldTry)
	{
		bShouldTryOnlinePIE = bShouldTry;
		// TODO:  Reload online services?
	}
}

bool FOnlineServicesEngineUtils::IsOnlinePIEEnabled() const
{
	check(UObjectInitialized());
	if (!bShouldTryOnlinePIE)
	{
		return false;
	}
	const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
	return OnlinePIESettings->bOnlinePIEEnabled || FOnlineServicesEngineUtilsHelper::HasAuthCommandLine();
}

int32 FOnlineServicesEngineUtils::GetNumPIELogins() const
{
	check(UObjectInitialized());

	int32 NumValidLogins = 0;
	const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
	for (const FOnlineAccountStoredCredentials& Login : OnlinePIESettings->Logins)
	{
		if (Login.IsValid())
		{
			NumValidLogins++;
		}
	}
	if (NumValidLogins == 0 && FOnlineServicesEngineUtilsHelper::HasAuthCommandLine())
	{
		NumValidLogins++;
	}
	
	return NumValidLogins;
}

void FOnlineServicesEngineUtils::GetPIELogins(TArray<FAuthLogin::Params>& Logins) const
{
	check(UObjectInitialized());
	const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
	for (const FOnlineAccountStoredCredentials& PIELoginCredentials : OnlinePIESettings->Logins)
	{
		UE::Online::FAuthLogin::Params LoginParameters;
		LoginParameters.CredentialsId = PIELoginCredentials.Id;
		LoginParameters.CredentialsToken.Set<FString>(PIELoginCredentials.Token);
		LoginParameters.CredentialsType = *PIELoginCredentials.Type;
		Logins.Emplace(MoveTemp(LoginParameters));
	}
	if (Logins.IsEmpty())
	{
		UE::Online::FAuthLogin::Params LoginParameters;
		if (FOnlineServicesEngineUtilsHelper::HasAuthCommandLine(LoginParameters))
		{
			Logins.Emplace(MoveTemp(LoginParameters));
		}
	}
}

#endif // WITH_EDITOR

void FOnlineServicesEngineUtils::Init()
{
	OnOnlineServicesCreatedDelegateHandle = UE::Online::OnOnlineServicesCreated.AddRaw(this, &FOnlineServicesEngineUtils::OnOnlineServicesCreated);
}

}
