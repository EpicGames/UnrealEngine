// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineAsyncOpHandle.h"
#include "Online/OnlineServicesEngineUtils.h"

class FOnlineSubsystemUtilsModule;

namespace UE::Online
{

class IOnlineServices;

/**
 * Concrete implementation of IOnlineServicesEngineUtils interface 
 */
class FOnlineServicesEngineUtils : public IOnlineServicesEngineUtils
{
public:

	FOnlineServicesEngineUtils()
		: bShouldTryOnlinePIE(true)
	{
	}

	virtual ~FOnlineServicesEngineUtils();

	virtual FName GetOnlineIdentifier(const FWorldContext& WorldContext) const override;
	virtual FName GetOnlineIdentifier(const UWorld* World) const override;
	virtual void SetEngineExternalUIBinding(const FOnExternalUIChangeDelegate& InOnExternalUIChangeDelegate) override;

#if WITH_EDITOR
	virtual bool SupportsOnlinePIE() const override;
	virtual void SetShouldTryOnlinePIE(bool bShouldTry) override;
	virtual bool IsOnlinePIEEnabled() const override;
	virtual int32 GetNumPIELogins() const override;
	virtual void GetPIELogins(TArray<FAuthLogin::Params>& Logins) const override;
#endif // WITH_EDITOR

private:

	void Init();
	void OnOnlineServicesCreated(TSharedRef<class IOnlineServices> NewServices);

	/** If false it will not try to do online PIE at all */
	bool bShouldTryOnlinePIE;

	/** Delegate set by the engine for notification of external UI operations */
	FOnExternalUIChangeDelegate OnExternalUIChangeDelegate;
	/** Array of delegate handles */
	TArray<FOnlineEventDelegateHandle> ExternalUIDelegateHandles;

	/** Delegate binding when new online services are created */
	FDelegateHandle OnOnlineServicesCreatedDelegateHandle;
	friend FOnlineSubsystemUtilsModule;
};

}
