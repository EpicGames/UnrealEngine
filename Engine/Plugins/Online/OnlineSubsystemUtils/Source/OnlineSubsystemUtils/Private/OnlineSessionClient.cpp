// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OnlineSessionClient.cpp: Online session related implementations 
	(creating/joining/leaving/destroying sessions)
=============================================================================*/

#include "OnlineSessionClient.h"
#include "Engine/GameInstance.h"
#include "OnlineSubsystemUtils.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineSessionClient)

UOnlineSessionClient::UOnlineSessionClient(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHandlingDisconnect = false;
	bIsFromInvite = false;
}

UGameInstance* UOnlineSessionClient::GetGameInstance() const
{
	return CastChecked<UGameInstance>(GetOuter());
}

UWorld* UOnlineSessionClient::GetWorld() const
{
	return GetGameInstance()->GetWorld();
}

IOnlineSessionPtr UOnlineSessionClient::GetSessionInt()
{
	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		UE_LOG_ONLINE(Warning, TEXT("UOnlineSessionClient::GetSessionInt: Called with NULL world."));
		return nullptr;
	}

	return Online::GetSessionInterface(World);
}

void UOnlineSessionClient::RegisterOnlineDelegates()
{
	OnJoinSessionCompleteDelegate           = FOnJoinSessionCompleteDelegate   ::CreateUObject(this, &ThisClass::OnJoinSessionComplete);
	OnEndForJoinSessionCompleteDelegate     = FOnEndSessionCompleteDelegate    ::CreateUObject(this, &ThisClass::OnEndForJoinSessionComplete);
	OnDestroyForJoinSessionCompleteDelegate = FOnDestroySessionCompleteDelegate::CreateUObject(this, &ThisClass::OnDestroyForJoinSessionComplete);
	OnDestroyForMainMenuCompleteDelegate	= FOnDestroySessionCompleteDelegate::CreateUObject(this, &ThisClass::OnDestroyForMainMenuComplete);
	OnSessionUserInviteAcceptedDelegate     = FOnSessionUserInviteAcceptedDelegate::CreateUObject(this, &ThisClass::OnSessionUserInviteAccepted);

	IOnlineSessionPtr SessionInt = GetSessionInt();
	if (SessionInt.IsValid())
	{
		OnSessionUserInviteAcceptedDelegateHandle = SessionInt->AddOnSessionUserInviteAcceptedDelegate_Handle(OnSessionUserInviteAcceptedDelegate);
	}
}

void UOnlineSessionClient::ClearOnlineDelegates()
{
	IOnlineSessionPtr SessionInt = GetSessionInt();
	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnSessionUserInviteAcceptedDelegate_Handle(OnSessionUserInviteAcceptedDelegateHandle);
	}
}

void UOnlineSessionClient::OnSessionUserInviteAccepted(bool bWasSuccessful, int32 ControllerId, FUniqueNetIdPtr UserId, const FOnlineSessionSearchResult& SearchResult)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnSessionUserInviteAccepted LocalUserNum: %d bSuccess: %d"), ControllerId, bWasSuccessful);
	// Don't clear invite accept delegate

	if (bWasSuccessful)
	{
		if (SearchResult.IsValid())
		{
			bIsFromInvite = true;
			JoinSession(NAME_GameSession, SearchResult);
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Invite accept returned no search result."));
		}
	}
}

void UOnlineSessionClient::OnEndForJoinSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnEndForJoinSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	IOnlineSessionPtr SessionInt = GetSessionInt();
	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnEndSessionCompleteDelegate_Handle(OnEndForJoinSessionCompleteDelegateHandle);
	}
	DestroyExistingSession_Impl(OnDestroyForJoinSessionCompleteDelegateHandle, SessionName, OnDestroyForJoinSessionCompleteDelegate);
}

void UOnlineSessionClient::EndExistingSession(FName SessionName, FOnEndSessionCompleteDelegate& Delegate)
{
	EndExistingSession_Impl(SessionName, Delegate);
}

FDelegateHandle UOnlineSessionClient::EndExistingSession_Impl(FName SessionName, FOnEndSessionCompleteDelegate& Delegate)
{
	FDelegateHandle Result;

	IOnlineSessionPtr SessionInt = GetSessionInt();

	if (SessionInt.IsValid())
	{
		Result = SessionInt->AddOnEndSessionCompleteDelegate_Handle(Delegate);
		SessionInt->EndSession(SessionName);
	}
	else
	{
		Delegate.ExecuteIfBound(SessionName, true);
	}

	return Result;
}

void UOnlineSessionClient::OnDestroyForJoinSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnDestroyForJoinSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);

	IOnlineSessionPtr SessionInt = GetSessionInt();

	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnDestroySessionCompleteDelegate_Handle(OnDestroyForJoinSessionCompleteDelegateHandle);
	}

	if (bWasSuccessful)
	{
		JoinSession(SessionName, CachedSessionResult);
	}

	bHandlingDisconnect = false;
}

void UOnlineSessionClient::OnDestroyForMainMenuComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnDestroyForMainMenuComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);

	IOnlineSessionPtr SessionInt = GetSessionInt();

	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnDestroySessionCompleteDelegate_Handle(OnDestroyForMainMenuCompleteDelegateHandle);
	}

	UWorld* World = GetWorld();
	UNetDriver* NetDriver = World ? World->GetNetDriver() : nullptr;
		
	// Call disconnect to force us back to the menu level
	GEngine->HandleDisconnect(World, NetDriver);

	bHandlingDisconnect = false;
}

void UOnlineSessionClient::DestroyExistingSession(FName SessionName, FOnDestroySessionCompleteDelegate& Delegate)
{
	FDelegateHandle UnusedHandle;
	DestroyExistingSession_Impl(UnusedHandle, SessionName, Delegate);
}

void UOnlineSessionClient::DestroyExistingSession_Impl(FDelegateHandle& OutResult, FName SessionName, FOnDestroySessionCompleteDelegate& Delegate)
{
	IOnlineSessionPtr SessionInt = GetSessionInt();

	if (SessionInt.IsValid())
	{
		OutResult = SessionInt->AddOnDestroySessionCompleteDelegate_Handle(Delegate);
		SessionInt->DestroySession(SessionName);
	}
	else
	{
		OutResult.Reset();
		Delegate.ExecuteIfBound(SessionName, true);
	}
}

void UOnlineSessionClient::OnJoinSessionComplete(FName SessionName, EOnJoinSessionCompleteResult::Type Result)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnJoinSessionComplete %s bSuccess: %d"), *SessionName.ToString(), static_cast<int32>(Result));

	IOnlineSessionPtr SessionInt = GetSessionInt();

	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnJoinSessionCompleteDelegate_Handle(OnJoinSessionCompleteDelegateHandle);

		if (Result == EOnJoinSessionCompleteResult::Success)
		{
			FString URL;
			if (SessionInt->GetResolvedConnectString(SessionName, URL))
			{
				APlayerController* PC = GetGameInstance()->GetFirstLocalPlayerController(GetWorld());
				if (PC)
				{
					if (bIsFromInvite)
					{
						URL += TEXT("?bIsFromInvite");
						bIsFromInvite = false;
					}
					PC->ClientTravel(URL, TRAVEL_Absolute);
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Failed to join session %s"), *SessionName.ToString());
			}
		}
	}
}

void UOnlineSessionClient::JoinSession(FName SessionName, const FOnlineSessionSearchResult& SearchResult)
{
	// Clean up existing sessions if applicable
	IOnlineSessionPtr SessionInt = GetSessionInt();

	if (SessionInt.IsValid())
	{
		EOnlineSessionState::Type SessionState = SessionInt->GetSessionState(SessionName);
		if (SessionState != EOnlineSessionState::NoSession)
		{
			CachedSessionResult = SearchResult;
			OnEndForJoinSessionCompleteDelegateHandle = EndExistingSession_Impl(SessionName, OnEndForJoinSessionCompleteDelegate);
		}
		else
		{
			GetGameInstance()->JoinSession(GetGameInstance()->GetFirstGamePlayer(), SearchResult);
		}
	}
}

void UOnlineSessionClient::HandleDisconnect(UWorld *World, UNetDriver *NetDriver)
{
	bool bWasHandled = HandleDisconnectInternal(World, NetDriver);
	
	if (!bWasHandled)
	{
		// This may have been a pending net game that failed, let the engine handle it (dont tear our stuff down)
		// (Would it be better to return true/false based on if we handled the disconnect or not? Let calling code call GEngine stuff
		GEngine->HandleDisconnect(World, NetDriver);
	}
}

bool UOnlineSessionClient::HandleDisconnectInternal(UWorld* World, UNetDriver* NetDriver)
{
	// This was a disconnect for our active world, we will handle it
	if (GetWorld() == World)
	{
		// Prevent multiple calls to this async flow
		if (!bHandlingDisconnect)
		{
			bHandlingDisconnect = true;
			DestroyExistingSession_Impl(OnDestroyForMainMenuCompleteDelegateHandle, NAME_GameSession, OnDestroyForMainMenuCompleteDelegate);
		}

		return true;
	}

	return false;
}

void UOnlineSessionClient::StartOnlineSession(FName SessionName)
{
	IOnlineSessionPtr SessionInterface = Online::GetSessionInterface(GetWorld());
	if (SessionInterface.IsValid())
	{
		FNamedOnlineSession* Session = SessionInterface->GetNamedSession(SessionName);
		if (Session &&
			(Session->SessionState == EOnlineSessionState::Pending || Session->SessionState == EOnlineSessionState::Ended))
		{
			StartSessionCompleteHandle = SessionInterface->AddOnStartSessionCompleteDelegate_Handle(FOnStartSessionCompleteDelegate::CreateUObject(this, &UOnlineSessionClient::OnStartSessionComplete));
			SessionInterface->StartSession(SessionName);
		}
	}
}

void UOnlineSessionClient::OnStartSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnStartSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);

	IOnlineSessionPtr SessionInterface = Online::GetSessionInterface(GetWorld());
	if (SessionInterface.IsValid())
	{
		SessionInterface->ClearOnStartSessionCompleteDelegate_Handle(StartSessionCompleteHandle);
	}
}

void UOnlineSessionClient::EndOnlineSession(FName SessionName)
{
	IOnlineSessionPtr SessionInterface = Online::GetSessionInterface(GetWorld());
	if (SessionInterface.IsValid())
	{
		FNamedOnlineSession* Session = SessionInterface->GetNamedSession(SessionName);
		if (Session &&
			Session->SessionState == EOnlineSessionState::InProgress)
		{
			EndSessionCompleteHandle = SessionInterface->AddOnEndSessionCompleteDelegate_Handle(FOnStartSessionCompleteDelegate::CreateUObject(this, &UOnlineSessionClient::OnEndSessionComplete));
			SessionInterface->EndSession(SessionName);
		}
	}
}

void UOnlineSessionClient::OnEndSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE(Verbose, TEXT("OnEndSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);

	IOnlineSessionPtr SessionInterface = Online::GetSessionInterface(GetWorld());
	if (SessionInterface.IsValid())
	{
		SessionInterface->ClearOnEndSessionCompleteDelegate_Handle(EndSessionCompleteHandle);
	}
}

void UOnlineSessionClient::SetInviteFlags(UWorld* World, const FJoinabilitySettings& Settings)
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
	if (SessionInt.IsValid())
	{
		FOnlineSessionSettings* GameSettings = SessionInt->GetSessionSettings(Settings.SessionName);
		if (GameSettings != NULL)
		{
			GameSettings->bShouldAdvertise = Settings.bPublicSearchable;
			GameSettings->bAllowInvites = Settings.bAllowInvites;
			GameSettings->bAllowJoinViaPresence = Settings.bJoinViaPresence && !Settings.bJoinViaPresenceFriendsOnly;
			GameSettings->bAllowJoinViaPresenceFriendsOnly = Settings.bJoinViaPresenceFriendsOnly;
			GameSettings->NumPublicConnections = Settings.MaxPlayers;
			SessionInt->UpdateSession(Settings.SessionName, *GameSettings, false);
		}
	}
}

