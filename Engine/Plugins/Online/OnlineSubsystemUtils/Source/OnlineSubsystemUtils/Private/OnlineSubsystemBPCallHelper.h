// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnlineFwd.h"

class APlayerController;
class IOnlineSubsystem;

// Helper class for various methods to reduce the call hierarchy
struct FOnlineSubsystemBPCallHelper
{
public:
	FOnlineSubsystemBPCallHelper(const TCHAR* CallFunctionContext, UObject* WorldContextObject, FName SystemName = NAME_None);

	void QueryIDFromPlayerController(APlayerController* PlayerController);

	bool IsValid() const
	{
		return UserID.IsValid() && (OnlineSub != nullptr);
	}

public:
	FUniqueNetIdPtr UserID;
	IOnlineSubsystem* const OnlineSub;
	const TCHAR* FunctionContext;
};

#define INVALID_INDEX -1



