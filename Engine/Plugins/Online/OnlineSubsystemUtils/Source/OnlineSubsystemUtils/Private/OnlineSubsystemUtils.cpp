// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemUtils.h"
#include "Algo/Copy.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/PlayerState.h"
#include "Engine/GameEngine.h"
#include "GameFramework/PlayerController.h"
#include "Engine/NetConnection.h"
#include "SocketSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "OnlineBeaconClient.h"
#include "UObject/UObjectHash.h"

#include "Sound/SoundGroups.h"
#include "VoiceModule.h"
#include "VoipListenerSynthComponent.h"
#include "AudioDevice.h"

// Testing classes
#include "Tests/TestFriendsInterface.h"
#include "Tests/TestSessionInterface.h"
#include "Tests/TestCloudInterface.h"
#include "Tests/TestLeaderboardInterface.h"
#include "Tests/TestTimeInterface.h"
#include "Tests/TestIdentityInterface.h"
#include "Tests/TestTitleFileInterface.h"
#include "Tests/TestEntitlementsInterface.h"
#include "Tests/TestAchievementsInterface.h"
#include "Tests/TestSharingInterface.h"
#include "Tests/TestUserInterface.h"
#include "Tests/TestMessageInterface.h"
#include "Tests/TestVoice.h"
#include "Tests/TestExternalUIInterface.h"
#include "Tests/TestPresenceInterface.h"
#include "Tests/TestStoreInterface.h"
#include "Tests/TestPurchaseInterface.h"
#include "Tests/TestStatsInterface.h"

static FAutoConsoleCommand GSendRemoteTalkersToEndpointCommand(
	TEXT("voice.sendRemoteTalkersToEndpoint"),
	TEXT("This will send audio output for all incoming voip audio to the named endpoint. if given no arguments, this will route voice output through the game engine."),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray< FString >& Args)
{
	IOnlineSubsystem* PlatformSubsystem = IOnlineSubsystem::Get();
	IOnlineVoicePtr PlatformVoiceInterface = PlatformSubsystem ? PlatformSubsystem->GetVoiceInterface() : nullptr;

	check(PlatformVoiceInterface.IsValid());

	if (Args.Num() == 0)
	{
		PlatformVoiceInterface->DisconnectAllEndpoints();
	}
	else
	{
		FString DeviceName;
		for (int32 Index = 0; Index < Args.Num(); Index++)
		{
			DeviceName.Append(Args[Index]);
			DeviceName.Append(TEXT(" "), 1);
		}

		PlatformVoiceInterface->PatchRemoteTalkerOutputToEndpoint(DeviceName);
	}
})
);

static FAutoConsoleCommand GSendLocalTalkersToEndpointCommand(
	TEXT("voice.sendLocalTalkersToEndpoint"),
	TEXT("This will send audio output for all outgoing voip audio to the named endpoint. if given no arguments, this will disconnect all external endpoints."),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray< FString >& Args)
{
	IOnlineSubsystem* PlatformSubsystem = IOnlineSubsystem::Get();
	IOnlineVoicePtr PlatformVoiceInterface = PlatformSubsystem ? PlatformSubsystem->GetVoiceInterface() : nullptr;

	check(PlatformVoiceInterface.IsValid());


	if (Args.Num() == 0)
	{
		PlatformVoiceInterface->DisconnectAllEndpoints();
	}
	else
	{
		FString DeviceName;
		for (int32 Index = 0; Index < Args.Num(); Index++)
		{
			DeviceName.Append(Args[Index]);
			DeviceName.Append(TEXT(" "), 1);
		}

		PlatformVoiceInterface->PatchLocalTalkerOutputToEndpoint(DeviceName);
	}
})
);

static int32 CvarAlwaysPlayVoipComponent = 1;
FAutoConsoleVariableRef CVarAlwaysPlayVoipComponent(
	TEXT("au.voip.AlwaysPlayVoiceComponent"),
	CvarAlwaysPlayVoipComponent,
	TEXT("When set to 1, guarantees that voip components won't get deprioritized. \n")
	TEXT("0: Let voip components get killed, 1: force VOIP components to be higher priority than all other audio sources."),
	ECVF_Default);

UAudioComponent* CreateVoiceAudioComponent(uint32 SampleRate, int32 NumChannels)
{
	UAudioComponent* AudioComponent = nullptr;
	if (GEngine != nullptr)
	{
		if (FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice())
		{
			USoundWaveProcedural* SoundStreaming = NewObject<USoundWaveProcedural>();
			SoundStreaming->SetSampleRate(SampleRate);
			SoundStreaming->NumChannels = NumChannels;
			SoundStreaming->Duration = INDEFINITELY_LOOPING_DURATION;
			SoundStreaming->SoundGroup = SOUNDGROUP_Voice;
			SoundStreaming->bLooping = false;

			SoundStreaming->bCanProcessAsync = true;

			AudioComponent = AudioDevice->CreateComponent(SoundStreaming);
			if (AudioComponent)
			{
				AudioComponent->bIsUISound = true;
				AudioComponent->bAllowSpatialization = false;
				AudioComponent->SetVolumeMultiplier(1.5f);

				const FSoftObjectPath VoiPSoundClassName = GetDefault<UAudioSettings>()->VoiPSoundClass;
				if (VoiPSoundClassName.IsValid())
				{
					AudioComponent->SoundClassOverride = LoadObject<USoundClass>(nullptr, *VoiPSoundClassName.ToString());
				}
			}
			else
			{
				UE_LOGF(LogVoiceDecode, Warning, "Unable to create voice audio component!");
			}
		}
	}

	return AudioComponent;
}

UVoipListenerSynthComponent* CreateVoiceSynthComponent(UWorld* World, uint32 SampleRate)
{
	UVoipListenerSynthComponent* SynthComponentPtr = nullptr;

	if (World)
	{
		if (FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice())
		{
			SynthComponentPtr = NewObject<UVoipListenerSynthComponent>();
			if (SynthComponentPtr)
			{
				const FSoftObjectPath VoiPSoundClassName = GetDefault<UAudioSettings>()->VoiPSoundClass;
				if (VoiPSoundClassName.IsValid())
				{
					SynthComponentPtr->SoundClass = LoadObject<USoundClass>(nullptr, *VoiPSoundClassName.ToString());
				}

				SynthComponentPtr->bAlwaysPlay = CvarAlwaysPlayVoipComponent;
				SynthComponentPtr->RegisterComponentWithWorld(World);
				SynthComponentPtr->Initialize(SampleRate);
			}
			else
			{
				UE_LOGF(LogVoiceDecode, Warning, "Unable to create voice synth component!");
			}
		}
	}

	return SynthComponentPtr;
}

void ApplyVoiceSettings(UVoipListenerSynthComponent* InSynthComponent, const FVoiceSettings& InSettings)
{
	InSynthComponent->CreateAudioComponent();

	InSynthComponent->bAllowSpatialization = true;
	UAudioComponent* AudioComponent = InSynthComponent->GetAudioComponent();

	check(AudioComponent != nullptr);

	if (InSettings.ComponentToAttachTo)
	{
		//If this component is simulating physics, it won't correctly attach to the parent.
		check(AudioComponent->IsSimulatingPhysics() == false);

		if (AudioComponent->GetAttachParent() == nullptr)
		{
			AudioComponent->SetupAttachment(InSettings.ComponentToAttachTo);
		}
		else
		{
			AudioComponent->AttachToComponent(InSettings.ComponentToAttachTo, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
		}

		// Since the Synth Component's internal audio component was created as a subobject when this
		// SynthComponent did not have an owning world, we need to register it independently.
		if (!AudioComponent->IsRegistered())
		{
			AudioComponent->RegisterComponentWithWorld(InSettings.ComponentToAttachTo->GetWorld());

			// By ensuring that this Audio Component's device handle is INDEX_NONE, we ensure that we will revert to
			// using the audio device associated with the World we just registered this audio component on.
			AudioComponent->AudioDeviceID = INDEX_NONE;
		}
	}

	if (InSettings.AttenuationSettings != nullptr)
	{
		InSynthComponent->AttenuationSettings = InSettings.AttenuationSettings;
	}

	if (InSettings.SourceEffectChain != nullptr)
	{
		InSynthComponent->SourceEffectChain = InSettings.SourceEffectChain;
	}
}

UWorld* GetWorldForOnline(FName InstanceName)
{
	UWorld* World = NULL;
#if WITH_EDITOR
	if (InstanceName != FOnlineSubsystemImpl::DefaultInstanceName && InstanceName != NAME_None)
	{
		FWorldContext* WorldContext = GEngine->GetWorldContextFromHandle(InstanceName);
		if (WorldContext)
		{
			check(WorldContext->WorldType == EWorldType::Game || WorldContext->WorldType == EWorldType::PIE);
			World = WorldContext->World();
		}
	}
	else
#endif
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		World = GameEngine ? GameEngine->GetGameWorld() : NULL;
	}

	return World;
}

int32 GetPortFromNetDriver(FName InstanceName)
{
	int32 Port = 0;
#if WITH_ENGINE
	if (GEngine)
	{
		UWorld* World = GetWorldForOnline(InstanceName);
		UNetDriver* NetDriver = World ? GEngine->FindNamedNetDriver(World, NAME_GameNetDriver) : NULL;
		if (NetDriver && NetDriver->GetNetMode() < NM_Client)
		{
			FString AddressStr = NetDriver->LowLevelGetNetworkNumber();
			int32 Colon = AddressStr.Find(":", ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (Colon != INDEX_NONE)
			{
				FString PortStr = AddressStr.Mid(Colon + 1);
				// Require fully-numeric trailing token: rejects non-IP addresses like EOS:<PUID> where Atoi would consume hex-prefix digits and yield a bogus port.
				if (PortStr.IsNumeric())
				{
					const int32 ParsedPort = FCString::Atoi(*PortStr);
					if (ParsedPort >= 0 && ParsedPort <= MAX_uint16)
					{
						Port = ParsedPort;
					}
					else
					{
						UE_LOGF(LogOnline, Verbose, "GetPortFromNetDriver: parsed port %d from address '%ls' is out of range [0, %u]; returning 0.", ParsedPort, *AddressStr, MAX_uint16);
					}
				}
				else
				{
					UE_LOGF(LogOnline, Verbose, "GetPortFromNetDriver: trailing token '%ls' from address '%ls' is not numeric; returning 0.", *PortStr, *AddressStr);
				}
			}
		}
	}
#endif
	return Port;
}

int32 GetClientPeerIp(FName InstanceName, const FUniqueNetId& UserId)
{
	int32 PeerIp = 0;
#if WITH_ENGINE
	if (GEngine)
	{
		UWorld* World = GetWorldForOnline(InstanceName);
		UNetDriver* NetDriver = World ? GEngine->FindNamedNetDriver(World, NAME_GameNetDriver) : NULL;
		if (NetDriver && NetDriver->GetNetMode() < NM_Client)
		{
			for (UNetConnection* ClientConnection : NetDriver->ClientConnections)
			{
				if (ClientConnection && 
					ClientConnection->PlayerId.ToString() == UserId.ToString())
				{
					TSharedPtr<const FInternetAddr> ClientAddr = ClientConnection->GetRemoteAddr();
					if (ClientAddr.IsValid())
					{
						uint32 TempAddr = 0;
						ClientAddr->GetIp(TempAddr);
						PeerIp = static_cast<int32>(TempAddr);
					}

					break;
				}
			}
		}
	}
#endif
	return PeerIp;
}

#if WITH_ENGINE
uint64 GetBaseVoiceChatTeamId(const UWorld* World)
{
	uint64 VoiceChatIdBase = 0;

	UNetDriver* NetDriver = World ? GEngine->FindNamedNetDriver(World, NAME_GameNetDriver) : nullptr;
	if (NetDriver)
	{
		FString AddressStr = NetDriver->LowLevelGetNetworkNumber();
		if (!AddressStr.IsEmpty())
		{
			TSharedRef<FInternetAddr> LocalAddr = ISocketSubsystem::Get()->CreateInternetAddr();

			bool bIsValid = false;
			LocalAddr->SetIp(*AddressStr, bIsValid);

			uint32 OutAddr = 0;
			if (bIsValid)
			{
				LocalAddr->GetIp(OutAddr);
			}
			else
			{
				// Address is not parseable as IP (e.g. EOS:<PUID>); hash it for a deterministic per-host base id.
				OutAddr = GetTypeHash(AddressStr);
			}

			const uint32 ProcId = FPlatformProcess::GetCurrentProcessId();
			// Layout: bits 32..63 = IP-or-address-hash | bits 24..31 = reserved for TeamIndex (filled in by GetVoiceChatTeamId) | bits 0..23 = ProcessId
			VoiceChatIdBase = ((static_cast<uint64>(OutAddr) << 32) & 0xFFFFFFFF00000000);
			VoiceChatIdBase |= (ProcId & 0x0000000000FFFFFF);
		}
	}

	return VoiceChatIdBase;
}

uint64 GetVoiceChatTeamId(uint64 VoiceChatIdBase, uint8 TeamIndex)
{
	// <32bit IP Addr> | <8bit team index> | <24bit ProcessId>
	return VoiceChatIdBase | static_cast<uint64>(((TeamIndex << 24) & 0xFF000000));
}
#endif

bool HandleVoiceCommands(IOnlineSubsystem* InOnlineSub, const UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = true;

	if (FParse::Command(&Cmd, TEXT("DUMP")))
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogVoice, ELogVerbosity::Display);
		bool bVoiceModule = FVoiceModule::IsAvailable();
		bool bVoiceModuleEnabled = false;
		if (bVoiceModule)
		{
			bVoiceModuleEnabled = FVoiceModule::Get().IsVoiceEnabled();
		}

		bool bRequiresPushToTalk = false;
		if (!GConfig->GetBool(TEXT("/Script/Engine.GameSession"), TEXT("bRequiresPushToTalk"), bRequiresPushToTalk, GGameIni))
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing bRequiresPushToTalk key in [/Script/Engine.GameSession] of DefaultGame.ini"));
		}

		int32 MaxLocalTalkers = 0;
		if (!GConfig->GetInt(TEXT("OnlineSubsystem"), TEXT("MaxLocalTalkers"), MaxLocalTalkers, GEngineIni))
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing MaxLocalTalkers key in OnlineSubsystem of DefaultEngine.ini"));
		}

		int32 MaxRemoteTalkers = 0;
		if (!GConfig->GetInt(TEXT("OnlineSubsystem"), TEXT("MaxRemoteTalkers"), MaxRemoteTalkers, GEngineIni))
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing MaxRemoteTalkers key in OnlineSubsystem of DefaultEngine.ini"));
		}
		
		float VoiceNotificationDelta = 0.0f;
		if (!GConfig->GetFloat(TEXT("OnlineSubsystem"), TEXT("VoiceNotificationDelta"), VoiceNotificationDelta, GEngineIni))
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing VoiceNotificationDelta key in OnlineSubsystem of DefaultEngine.ini"));
		}

		bool bHasVoiceInterfaceEnabled = false;
		if (!GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("bHasVoiceEnabled"), bHasVoiceInterfaceEnabled, GEngineIni))
		{
			UE_LOG_ONLINE_VOICE(Log, TEXT("Voice interface disabled by config [OnlineSubsystem].bHasVoiceEnabled"));
		}

		bool bDuckingOptOut = false;
		if (!GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("bDuckingOptOut"), bDuckingOptOut, GEngineIni))
		{
			UE_LOG_ONLINE_VOICE(Log, TEXT("Voice ducking not set by config [OnlineSubsystem].bDuckingOptOut"));
		}

		FString VoiceDump;

		bool bVoiceInterface = false;
		IOnlineVoicePtr VoiceInt = InOnlineSub->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
			bVoiceInterface = true;
			VoiceDump = VoiceInt->GetVoiceDebugState();
		}

		UE_LOG_ONLINE_VOICE(Display, TEXT("Voice Module Available: %s"), bVoiceModule ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_VOICE(Display, TEXT("Voice Module Enabled: %s"), bVoiceModuleEnabled ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_VOICE(Display, TEXT("Voice Interface Available: %s"), bVoiceInterface ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_VOICE(Display, TEXT("Voice Interface Enabled: %s"), bHasVoiceInterfaceEnabled ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_VOICE(Display, TEXT("Ducking Opt Out Enabled: %s"), bDuckingOptOut ? TEXT("true") : TEXT("false"));
		UE_LOG_ONLINE_VOICE(Display, TEXT("Max Local Talkers: %d"), MaxLocalTalkers);
		UE_LOG_ONLINE_VOICE(Display, TEXT("Max Remote Talkers: %d"), MaxRemoteTalkers);
		UE_LOG_ONLINE_VOICE(Display, TEXT("Notification Delta: %0.2f"), VoiceNotificationDelta);
		UE_LOG_ONLINE_VOICE(Display, TEXT("Voice Requires Push To Talk: %s"), bRequiresPushToTalk ? TEXT("true") : TEXT("false"));

		TArray<FString> OutArray;
		VoiceDump.ParseIntoArray(OutArray, TEXT("\n"), false);
		for (const FString& Str : OutArray)
		{
			UE_LOG_ONLINE_VOICE(Display, TEXT("%s"), *Str);
		}
	}
	else
	{
		IOnlineVoicePtr VoiceInt = InOnlineSub->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
		}
	}

	return bWasHandled;
}

enum class FOnlineBeaconClientFilterFlags : uint8
{
	None = 0,

	/** Whether to match all beacons when no filter is set. */
	WildcardAllowed,
};
ENUM_CLASS_FLAGS(FOnlineBeaconClientFilterFlags);

class FOnlineBeaconClientFilter final
{
public:
	static bool Parse(const TCHAR* Cmd, FOnlineBeaconClientFilter& OutFilter, FOnlineBeaconClientFilterFlags Flags);

	FOnlineBeaconClientFilter() = default;
	FString ToString() const;

	// Returns true if the beacon matches the filter.
	bool operator()(const UObject* Obj) const;
	bool operator()(const AOnlineBeaconClient* Beacon) const;

private:
	FOnlineBeaconClientFilter(const FString& Name, const FString& Type, const FUniqueNetIdRepl& UniqueId, FOnlineBeaconClientFilterFlags Flags);

	FString Name;
	FString Type;
	FUniqueNetIdRepl UniqueId;
	bool bIsWildcard = false;
	FOnlineBeaconClientFilterFlags Flags;
};

bool FOnlineBeaconClientFilter::Parse(const TCHAR* Cmd, FOnlineBeaconClientFilter& OutFilter, FOnlineBeaconClientFilterFlags Flags)
{
	bool bArgumentsValid = true;

	FString BeaconName;
	FParse::Value(Cmd, TEXT("Name="), BeaconName);

	FString BeaconType;
	FParse::Value(Cmd, TEXT("Type="), BeaconType);

	FUniqueNetIdRepl UniqueId;
	FString UniqueIdStr;
	if (FParse::Value(Cmd, TEXT("UniqueId="), UniqueIdStr))
	{
		UniqueId.FromJson(UniqueIdStr);
		if (!UniqueId.IsValid())
		{
			UE_LOGF(LogBeacon, Warning, "Beacon filter parse failure - unable to parse UniqueId: %ls", *UniqueIdStr);
			bArgumentsValid = false;
		}
	}

	const bool bHasAnyArguments = !BeaconName.IsEmpty() || !BeaconType.IsEmpty() || UniqueId.IsValid();
	if (!EnumHasAnyFlags(Flags, FOnlineBeaconClientFilterFlags::WildcardAllowed) && !bHasAnyArguments)
	{
		UE_LOGF(LogBeacon, Warning, "Beacon filter parse failure - no arguments found. Wildcard filter is not allowed.");
		bArgumentsValid = false;
	}

	if (!bArgumentsValid)
	{
		return false;
	}

	OutFilter = FOnlineBeaconClientFilter(BeaconName, BeaconType, UniqueId, Flags);
	return true;
}

FOnlineBeaconClientFilter::FOnlineBeaconClientFilter(const FString& Name, const FString& Type, const FUniqueNetIdRepl& UniqueId, FOnlineBeaconClientFilterFlags Flags)
	: Name(Name)
	, Type(Type)
	, UniqueId(UniqueId)
	, Flags(Flags)
{
	if (EnumHasAnyFlags(Flags, FOnlineBeaconClientFilterFlags::WildcardAllowed))
	{
		bIsWildcard = Name.IsEmpty() && Type.IsEmpty() && !UniqueId.IsValid();
	}
}

FString FOnlineBeaconClientFilter::ToString() const
{
	if (bIsWildcard)
	{
		return FString(TEXT("Wildcard"));
	}
	else
	{
		return FString::Printf(TEXT("Name: %s, Type: %s, UniqueId: %s"),
			!Name.IsEmpty() ? *Name : TEXT("<empty>"),
			!Type.IsEmpty() ? *Type : TEXT("<empty>"),
			*UniqueId.ToDebugString());
	}
}

bool FOnlineBeaconClientFilter::operator()(const UObject* Obj) const
{
	return (*this)(Cast<AOnlineBeaconClient>(Obj));
}

bool FOnlineBeaconClientFilter::operator()(const AOnlineBeaconClient* Beacon) const
{
	if (!IsValid(Beacon))
	{
		return false;
	}

	const bool bNameMatches = Name.IsEmpty() || Name == Beacon->GetName();
	const bool bTypeMatches = Type.IsEmpty() || Type == Beacon->GetBeaconType();
	const bool bUniqueIdMatches = !UniqueId.IsValid() || UniqueId == Beacon->GetUniqueId();
	return bIsWildcard || (bNameMatches && bTypeMatches && bUniqueIdMatches);
}

TArray<AOnlineBeaconClient*> GetFilteredOnlineBeaconClientInstances(const FOnlineBeaconClientFilter& Filter)
{
	TArray<UObject*> Objects;
	GetObjectsOfClass(AOnlineBeaconClient::StaticClass(), Objects);

	TArray<AOnlineBeaconClient*> ClientBeacons;
	Algo::TransformIf(Objects, ClientBeacons, Filter,
		[](UObject* Obj)->AOnlineBeaconClient* { return Cast<AOnlineBeaconClient>(Obj); });

	return ClientBeacons;
}

FString GetBeaconLogString(const AOnlineBeaconClient& ClientBeacon)
{
	return FString::Printf(TEXT("Name: %s, Type: %s, UniqueId: %s, State: %s"), *ClientBeacon.GetName(), *ClientBeacon.GetBeaconType(), *ClientBeacon.GetUniqueId().ToDebugString(), *UEnum::GetValueAsString(ClientBeacon.GetConnectionState()));
}

bool HandleBeaconCommands(IOnlineSubsystem* InOnlineSub, const UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = true;

	if (FParse::Command(&Cmd, TEXT("DUMP")))
	{
		FOnlineBeaconClientFilter BeaconFilter;
		if (FOnlineBeaconClientFilter::Parse(Cmd, BeaconFilter, FOnlineBeaconClientFilterFlags::WildcardAllowed))
		{
			UE_LOGF(LogBeacon, Log, "Dumping beacon info based on filter: %ls.", *BeaconFilter.ToString());
			TArray<AOnlineBeaconClient*> FilteredBeacons = GetFilteredOnlineBeaconClientInstances(BeaconFilter);
			for (AOnlineBeaconClient* ClientBeacon : FilteredBeacons)
			{
				UE_LOGF(LogBeacon, Log, "    AOnlineBeaconClient: %ls", *GetBeaconLogString(*ClientBeacon));
			}
			UE_LOGF(LogBeacon, Log, "Dump complete. Found %d beacons.", FilteredBeacons.Num());
		}
		else
		{
			UE_LOGF(LogBeacon, Warning, "DUMP command failure - unable to parse arguments.");
		}
	}
	else if (FParse::Command(&Cmd, TEXT("DISCONNECT")))
	{
		FOnlineBeaconClientFilter BeaconFilter;
		if (FOnlineBeaconClientFilter::Parse(Cmd, BeaconFilter, FOnlineBeaconClientFilterFlags::None))
		{
			UE_LOGF(LogBeacon, Log, "Disconnecting beacons based on filter: %ls.", *BeaconFilter.ToString());

			int32 NumDisconnected = 0;
			for (AOnlineBeaconClient* ClientBeacon : GetFilteredOnlineBeaconClientInstances(BeaconFilter))
			{
				UNetConnection* Connection = ClientBeacon->GetNetConnection();

				// Closing the connection will start the chain of events leading to the removal from lists and destruction of the actor
				if (Connection && Connection->GetConnectionState() != USOCK_Closed)
				{
					UE_LOGF(LogBeacon, Log, "    Disconnecting beacon: %ls", *GetBeaconLogString(*ClientBeacon));
					Connection->FlushNet(true);
					Connection->Close();
					++NumDisconnected;
				}
				else
				{
					UE_LOGF(LogBeacon, Log, "    Beacon not in a valid state to disconnect: %ls", *GetBeaconLogString(*ClientBeacon));
				}
			}

			UE_LOGF(LogBeacon, Log, "Disconnect complete. %d beacons affected.", NumDisconnected);
		}
		else
		{
			UE_LOGF(LogBeacon, Warning, "DISCONNECT command failure - unable to parse arguments.");
		}
	}

	return bWasHandled;
}

/**
 * Exec handler that routes online specific execs to the proper subsystem
 *
 * @param InWorld World context
 * @param Cmd 	the exec command being executed
 * @param Ar 	the archive to log results to
 *
 * @return true if the handler consumed the input, false to continue searching handlers
 */
static bool OnlineExec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	bool bWasHandled = false;

	// Ignore any execs that don't start with ONLINE
	if (FParse::Command(&Cmd, TEXT("ONLINE")))
	{
		FString SubName;
		FParse::Value(Cmd, TEXT("Sub="), SubName);
		// Allow for either Sub=<platform> or Subsystem=<platform>
		if (SubName.Len() > 0)
		{
			Cmd += FCString::Strlen(TEXT("Sub=")) + SubName.Len();
		}
		else
		{ 
			FParse::Value(Cmd, TEXT("Subsystem="), SubName);
			if (SubName.Len() > 0)
			{
				Cmd += FCString::Strlen(TEXT("Subsystem=")) + SubName.Len();
			}
		}

		IOnlineSubsystem* OnlineSub = NULL;
		// If the exec requested a specific subsystem, the grab that one for routing
		if (SubName.Len())
		{
			OnlineSub = Online::GetSubsystem(InWorld, *SubName);
		}
		// Otherwise use the default subsystem and route to that
		else
		{
			OnlineSub = Online::GetSubsystem(InWorld);
		}

		if (OnlineSub != NULL)
		{
			bWasHandled = OnlineSub->Exec(InWorld, Cmd, Ar);
			// If this wasn't handled, see if this is a testing request
			if (!bWasHandled)
			{
				if (FParse::Command(&Cmd, TEXT("TEST")))
				{
#if WITH_DEV_AUTOMATION_TESTS
					if (FParse::Command(&Cmd, TEXT("FRIENDS")))
					{
						TArray<FString> Invites;
						for (FString FriendId=FParse::Token(Cmd, false); !FriendId.IsEmpty(); FriendId=FParse::Token(Cmd, false))
						{
							Invites.Add(FriendId);
						}
						// This class deletes itself once done
						(new FTestFriendsInterface(SubName))->Test(InWorld, Invites);
						bWasHandled = true;
					}
					// Spawn the object that will exercise all of the session methods as host
					else if (FParse::Command(&Cmd, TEXT("SESSIONHOST")))
					{
						bool bTestLAN = FParse::Command(&Cmd, TEXT("LAN")) ? true : false;

						FOnlineSessionSettings SettingsOverride;

						FString ParamOverride;
						while (FParse::Token(Cmd, ParamOverride, false))
						{
							FString Value;
							FParse::Token(Cmd, Value, false);

							if (Value.IsNumeric())
							{
								SettingsOverride.Set(FName(*ParamOverride), FCString::Atoi(*Value));
							}
							else
							{
								SettingsOverride.Set(FName(*ParamOverride), Value);
							}
						}

						// This class deletes itself once done
						(new FTestSessionInterface(SubName, true))->Test(InWorld, bTestLAN, false, SettingsOverride);
						bWasHandled = true;
					}
					// Spawn the object that will exercise all of the session methods as client
					else if (FParse::Command(&Cmd, TEXT("SESSIONCLIENT")))
					{
						bool bTestLAN = FParse::Command(&Cmd, TEXT("LAN")) ? true : false;

						FOnlineSessionSettings SettingsOverride;

						// This class deletes itself once done
						(new FTestSessionInterface(SubName, false))->Test(InWorld, bTestLAN, false, SettingsOverride);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("STARTMATCHMAKING")))
					{
						FOnlineSessionSettings SettingsOverride;

						FString ParamOverride;
						while (FParse::Token(Cmd, ParamOverride, false))
						{
							FString Value;
							FParse::Token(Cmd, Value, false);

							if (Value.IsNumeric())
							{
								SettingsOverride.Set(FName(*ParamOverride), FCString::Atoi(*Value));
							}
							else
							{
								SettingsOverride.Set(FName(*ParamOverride), Value);
							}
						}

						// This class deletes itself once done
						(new FTestSessionInterface(SubName, false))->Test(InWorld, false, true, SettingsOverride);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("CLOUD")))
					{
						// This class deletes itself once done
						(new FTestCloudInterface(SubName))->Test(InWorld);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("LEADERBOARDS")))
					{
						// This class deletes itself once done
						FString LeaderboardName, SortedColumn, UserId;
						
						if (!FParse::Token(Cmd, LeaderboardName, false) || !FParse::Token(Cmd, SortedColumn, false))
						{
							// If the user didn't pass in any parameters, try to get them from the config file instead.
							(new FTestLeaderboardInterface(SubName))->TestFromConfig(InWorld);
						}
						else
						{
							TMap<FString, EOnlineKeyValuePairDataType::Type> Columns;
							FString ColumnName;
							while (FParse::Token(Cmd, ColumnName, false))
							{
								FString ColumnFormat;
								if (FParse::Token(Cmd, ColumnFormat, false))
								{
									Columns.Add(ColumnName, EOnlineKeyValuePairDataType::FromString(ColumnFormat));
								}
								else
								{
									UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("Setting %s as UserId for LEADERBOARDS TEST"), *ColumnName);
									UserId = ColumnName;
								}
							}

							(new FTestLeaderboardInterface(SubName))->Test(InWorld, LeaderboardName, SortedColumn, MoveTemp(Columns), UserId);
						}
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("STATS")))
					{
						(new FTestStatsInterface(SubName))->Test(InWorld, Cmd);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("PRESENCE")))
					{
						// Takes a user id/name of a non-friend user for the sole usage of querying out
						// Pass nothing if the platform doesn't support it
						(new FTestPresenceInterface(SubName))->Test(InWorld, FParse::Token(Cmd, false));
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("VOICE")))
					{
						// This class deletes itself once done
						(new FTestVoice())->Test();
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("TIME")))
					{
						// This class deletes itself once done
						(new FTestTimeInterface(SubName))->Test(InWorld);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("IDENTITY")))
					{
						FString Id = FParse::Token(Cmd, false);
						FString Auth = FParse::Token(Cmd, false);
						FString Type = FParse::Token(Cmd, false);

						bool bLogout = Id.Equals(TEXT("logout"), ESearchCase::IgnoreCase);

						// This class deletes itself once done
						(new FTestIdentityInterface(SubName))->Test(InWorld, FOnlineAccountCredentials(Type, Id, Auth), bLogout);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("UNIQUEIDREPL")))
					{
						extern void TestUniqueIdRepl(class UWorld* InWorld);
						TestUniqueIdRepl(InWorld);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("KEYVALUEPAIR")))
					{
						extern void TestKeyValuePairs();
						TestKeyValuePairs();
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("TITLEFILE")))
					{
						// This class deletes itself once done
						(new FTestTitleFileInterface(SubName))->Test(InWorld);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("ENTITLEMENTS")))
					{
						// This class also deletes itself once done
						(new FTestEntitlementsInterface(SubName))->Test(InWorld);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("ACHIEVEMENTS")))
					{
						// This class also deletes itself once done
						(new FTestAchievementsInterface(SubName))->Test(InWorld);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("SHARING")))
					{
						bool bTestWithImage = FParse::Command(&Cmd, TEXT("IMG"));

						// This class also deletes itself once done
						(new FTestSharingInterface(SubName))->Test(InWorld, bTestWithImage);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("USER")))
					{
						TArray<FString> UserIds;
						for (FString Id=FParse::Token(Cmd, false); !Id.IsEmpty(); Id=FParse::Token(Cmd, false))
						{
							UserIds.Add(Id);
						}
						// This class also deletes itself once done
						(new FTestUserInterface(SubName))->Test(InWorld, UserIds);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("MESSAGE")))
					{
						TArray<FString> RecipientIds;
						for (FString UserId=FParse::Token(Cmd, false); !UserId.IsEmpty(); UserId=FParse::Token(Cmd, false))
						{
							RecipientIds.Add(UserId);
						}
						// This class also deletes itself once done
						(new FTestMessageInterface(SubName))->Test(InWorld, RecipientIds);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("EXTERNALUI")))
					{
						// Full command usage:    EXTERNALUI ACHIEVEMENTS FRIENDS INVITE LOGIN PROFILE WEBURL
						// Example for one test:  EXTERNALUI WEBURL
						// Example for store: EXTERNALUI STORE productid true
						// Example for send message: EXTERNALUI MESSAGE user "message"
						// Note that tests are enabled in alphabetical order
						bool bTestAchievementsUI = FParse::Command(&Cmd, TEXT("ACHIEVEMENTS")) ? true : false;
						bool bTestFriendsUI = FParse::Command(&Cmd, TEXT("FRIENDS")) ? true : false;
						bool bTestInviteUI = FParse::Command(&Cmd, TEXT("INVITE")) ? true : false;
						bool bTestLoginUI = FParse::Command(&Cmd, TEXT("LOGIN")) ? true : false;
						bool bTestProfileUI = FParse::Command(&Cmd, TEXT("PROFILE")) ? true : false;
						bool bTestWebURL = FParse::Command(&Cmd, TEXT("WEBURL")) ? true : false;

						// This class also deletes itself once done
						FTestExternalUIInterface* TestHarness = (new FTestExternalUIInterface(SubName, bTestLoginUI, bTestFriendsUI, 
							bTestInviteUI, bTestAchievementsUI, bTestWebURL, bTestProfileUI));

						if (FParse::Command(&Cmd, TEXT("STORE")))
						{
							FString AppID = FParse::Token(Cmd, false);
							bool bAddCart = FCString::ToBool(*FParse::Token(Cmd, false));
							TestHarness->TestStorePage(AppID, bAddCart);
						}
						else if (FParse::Command(&Cmd, TEXT("MESSAGE")))
						{
							FString UserID = FParse::Token(Cmd, false);
							FString Message = FParse::Token(Cmd, false);
							TestHarness->TestSendMessage(UserID, Message);
						}
						else
						{
							TestHarness->Test();
						}

						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("STORE")))
					{
						TArray<FString> OfferIds;
						for (FString OfferId = FParse::Token(Cmd, false); !OfferId.IsEmpty(); OfferId = FParse::Token(Cmd, false))
						{
							OfferIds.Add(OfferId);
						}
						// This class deletes itself once done
						(new FTestStoreInterface(SubName))->Test(InWorld, OfferIds);
						bWasHandled = true;
					}
					else if (FParse::Command(&Cmd, TEXT("PURCHASE")))
					{
						FString Namespace = FParse::Token(Cmd, false);
						TArray<FString> OfferIds;
						for (FString OfferId = FParse::Token(Cmd, false); !OfferId.IsEmpty(); OfferId = FParse::Token(Cmd, false))
						{
							OfferIds.Add(OfferId);
						}
						// This class deletes itself once done
						(new FTestPurchaseInterface(SubName))->Test(InWorld, Namespace, OfferIds);
						bWasHandled = true;
					}
#endif //WITH_DEV_AUTOMATION_TESTS
				}
				else if (FParse::Command(&Cmd, TEXT("VOICE")))
				{
					bWasHandled = HandleVoiceCommands(OnlineSub, InWorld, Cmd, Ar);
				}
				else if (FParse::Command(&Cmd, TEXT("BEACON")))
				{
					bWasHandled = HandleBeaconCommands(OnlineSub, InWorld, Cmd, Ar);
				}
			}
		}
	}
	return bWasHandled;
}

/** Our entry point for all online exec routing */
FStaticSelfRegisteringExec OnlineExecRegistration(OnlineExec);

//////////////////////////////////////////////////////////////////////////
// FOnlineSubsystemBPCallHelper

FOnlineSubsystemBPCallHelper::FOnlineSubsystemBPCallHelper(const TCHAR* CallFunctionContext, UObject* WorldContextObject, FName SystemName)
	: OnlineSub(Online::GetSubsystem(GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull), SystemName))
	, FunctionContext(CallFunctionContext)
{
	if (OnlineSub == nullptr)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("%s - Invalid or uninitialized OnlineSubsystem"), FunctionContext), ELogVerbosity::Warning);
	}
}

void FOnlineSubsystemBPCallHelper::QueryIDFromPlayerController(APlayerController* PlayerController)
{
	UserID.Reset();

	APlayerState* PlayerState = nullptr;
	if (PlayerController != nullptr)
	{
		PlayerState = ToRawPtr(PlayerController->PlayerState);
	}

	if (PlayerState != nullptr)
	{
		UserID = PlayerState->GetUniqueId().GetUniqueNetId();
		if (!UserID.IsValid())
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("%s - Cannot map local player to unique net ID"), FunctionContext), ELogVerbosity::Warning);
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("%s - Invalid player state"), FunctionContext), ELogVerbosity::Warning);
	}
}
