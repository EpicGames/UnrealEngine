// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemUtilsModule.h"
#include "Online.h"
#include "OnlineSubsystemUtils.h"
#include "OnlineServicesEngineUtilsImpl.h"
#include "OnlineDelegates.h"
#include "OnlinePIESettings.h"

#if WITH_EDITOR
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#endif

IMPLEMENT_MODULE(FOnlineSubsystemUtilsModule, OnlineSubsystemUtils);

#if WITH_EDITOR
static bool HasAuthCommandLineStatic()
{
	FString Id, Token, Type;
	return FParse::Value(FCommandLine::Get(), TEXT("AUTH_LOGIN="), Id) &&
		   FParse::Value(FCommandLine::Get(), TEXT("AUTH_PASSWORD="), Token) &&
		   FParse::Value(FCommandLine::Get(), TEXT("AUTH_TYPE="), Type);
}
#endif

/**
 * Concrete implementation of IOnlineSubsystemUtils interface
 */
class FOnlineSubsystemUtils : public IOnlineSubsystemUtils
{
public:

	FOnlineSubsystemUtils() 
		: bShouldTryOnlinePIE(true)
	{
	}

	virtual ~FOnlineSubsystemUtils()
	{
		FOnlineSubsystemDelegates::OnOnlineSubsystemCreated.Remove(OnOnlineSubsystemCreatedDelegateHandle);
	}

	virtual FName GetOnlineIdentifier(const FWorldContext& WorldContext, const FName Subsystem = NAME_None) const override
	{
#if WITH_EDITOR
		if (WorldContext.WorldType == EWorldType::PIE)
		{
			return FName(*FString::Printf(TEXT("%s:%s"), !Subsystem.IsNone() ? *Subsystem.ToString() : TEXT(""), *WorldContext.ContextHandle.ToString()));
		}
#endif

		return Subsystem;
	}

	virtual FName GetOnlineIdentifier(const UWorld* World, const FName Subsystem = NAME_None) const override
	{
#if WITH_EDITOR
		if (const FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World))
		{
			return GetOnlineIdentifier(*WorldContext, Subsystem);
		}
#endif
		return Subsystem;
	}

	virtual FUniqueNetIdPtr CreateForeignUniqueNetId(const FString& Str, FName Type) const
	{
		return FUniqueNetIdString::Create(Str, Type);
	}

	virtual uint8 GetReplicationHashForSubsystem(FName SubsystemName) const
	{
		const uint8* NameHash = SubsystemNameToHash.Find(SubsystemName);
		return NameHash ? *NameHash : 0;
	}

	virtual FName GetSubsystemFromReplicationHash(uint8 InHash) const
	{
		const FName* Name = HashToSubsystemName.Find(InHash);
		return Name ? *Name : NAME_None;
	}

	virtual void OnOnlineSubsystemCreated(IOnlineSubsystem* NewSubsystem)
	{
		if (OnExternalUIChangeDelegate.IsBound())
		{
			IOnlineExternalUIPtr ExternalUI = NewSubsystem->GetExternalUIInterface();
			if (ExternalUI.IsValid())
			{
				ExternalUI->AddOnExternalUIChangeDelegate_Handle(OnExternalUIChangeDelegate);
			}
		}
	}

	virtual void SetEngineExternalUIBinding(const FOnExternalUIChangeDelegate& InOnExternalUIChangeDelegate) override
	{
		OnExternalUIChangeDelegate = InOnExternalUIChangeDelegate;

		FOnlineSubsystemModule::FEnumerateOnlineSubsystemCb ExtDelegateCb = [this](IOnlineSubsystem* Subsystem)
		{
			IOnlineExternalUIPtr ExternalUI = Subsystem->GetExternalUIInterface();
			if (ExternalUI.IsValid())
			{
				FDelegateHandle* ExistingDelegateHandle = ExternalUIDelegateHandles.Find(Subsystem->GetSubsystemName());
				if (ExistingDelegateHandle && ExistingDelegateHandle->IsValid())
				{
					ExternalUI->ClearOnExternalUIChangeDelegate_Handle(*ExistingDelegateHandle);
				}

				FDelegateHandle DelegateHandle = ExternalUI->AddOnExternalUIChangeDelegate_Handle(OnExternalUIChangeDelegate);
				ExternalUIDelegateHandles.Add(Subsystem->GetSubsystemName(), DelegateHandle);
			}
		};
		
		FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
		OSS.EnumerateOnlineSubsystems(ExtDelegateCb);
	}

#if WITH_EDITOR

private:
	bool HasAuthCommandLine(FOnlineAccountCredentials& LoginParameters) const
	{
		return FParse::Value(FCommandLine::Get(), TEXT("AUTH_LOGIN="), LoginParameters.Id ) &&
			   FParse::Value(FCommandLine::Get(), TEXT("AUTH_PASSWORD="), LoginParameters.Token) &&
			   FParse::Value(FCommandLine::Get(), TEXT("AUTH_TYPE="), LoginParameters.Type);
	}
public:
	virtual bool SupportsOnlinePIE() const override
	{
		check(UObjectInitialized());
		const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
		if ((OnlinePIESettings->bOnlinePIEEnabled && GetNumPIELogins() > 0) || HasAuthCommandLineStatic())
		{
			// If we can't get the identity interface then things are either not configured right or disabled
			IOnlineIdentityPtr IdentityInt = Online::GetIdentityInterface();
			return IdentityInt.IsValid();
		}

		return false;
	}

	virtual void SetShouldTryOnlinePIE(bool bShouldTry) override
	{
		if (bShouldTryOnlinePIE != bShouldTry)
		{
			bShouldTryOnlinePIE = bShouldTry;

			// This will swap it back to the null subsystem if needed
			IOnlineSubsystem::ReloadDefaultSubsystem();
		}
	}

	virtual bool IsOnlinePIEEnabled() const override
	{
		check(UObjectInitialized());
		if (!bShouldTryOnlinePIE)
		{
			return false;
		}
		const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
		return OnlinePIESettings->bOnlinePIEEnabled || HasAuthCommandLineStatic();
	}

	virtual int32 GetNumPIELogins() const override
	{
		check(UObjectInitialized());

		int32 NumValidLogins = 0;
		const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
		for (const FOnlineAccountStoredCredentials& Login : OnlinePIESettings->Logins)
		{
			if (Login.IsValid())
			{
				NumValidLogins++;
			}
		}
		if (NumValidLogins == 0 && HasAuthCommandLineStatic())
		{
			NumValidLogins++;
		}
	
		return NumValidLogins;
	}

	virtual void GetPIELogins(TArray<FOnlineAccountCredentials>& Logins) override
	{
		check(UObjectInitialized());
		const UOnlinePIESettings* OnlinePIESettings = GetDefault<UOnlinePIESettings>();
		if (OnlinePIESettings->Logins.Num() > 0)
		{
			Logins.Empty(OnlinePIESettings->Logins.Num());
			for (const FOnlineAccountStoredCredentials& Login : OnlinePIESettings->Logins)
			{
				if (Login.IsValid())
				{
					Logins.Emplace(Login.Type, Login.Id, Login.Token);
				}
			}
		}
		if (Logins.IsEmpty())
		{
			FOnlineAccountCredentials LoginParameters;
			if (HasAuthCommandLine(LoginParameters))
			{
				Logins.Emplace(MoveTemp(LoginParameters));
			}
		}
	}

#endif // WITH_EDITOR

private:

	void Init()
	{
		FOnlineSubsystemDelegates::OnOnlineSubsystemCreated.AddRaw(this, &FOnlineSubsystemUtils::OnOnlineSubsystemCreated);
		CreateNameHashes();
	}

	void CreateNameHashes()
	{
		int32 HashId = 1;
#define CREATE_HASH(MacroName) \
{ \
 uint8 Hash = static_cast<uint8>(HashId); \
 SubsystemNameToHash.Add(MacroName, Hash); \
 HashToSubsystemName.Add(Hash, MacroName); \
 HashId++; \
}
		CREATE_HASH(NULL_SUBSYSTEM); 
		CREATE_HASH(MCP_SUBSYSTEM);
		CREATE_HASH(STEAM_SUBSYSTEM);
		CREATE_HASH(PS4_SUBSYSTEM);
		CREATE_HASH(GOOGLE_SUBSYSTEM);
		CREATE_HASH(GOOGLEPLAY_SUBSYSTEM);
		CREATE_HASH(FACEBOOK_SUBSYSTEM);
		CREATE_HASH(IOS_SUBSYSTEM);
		CREATE_HASH(APPLE_SUBSYSTEM);
		CREATE_HASH(TENCENT_SUBSYSTEM);
		CREATE_HASH(NINTENDO_SUBSYSTEM);
		CREATE_HASH(AMAZON_SUBSYSTEM);
		CREATE_HASH(GAMECIRCLE_SUBSYSTEM);
		CREATE_HASH(THUNDERHEAD_SUBSYSTEM); 
		CREATE_HASH(ONESTORE_SUBSYSTEM);
		// Shouldn't need these as they are mocking interfaces for existing platforms
		CREATE_HASH(PS4SERVER_SUBSYSTEM);

#undef CREATE_HASH

		ensure(SubsystemNameToHash.Num() == (HashId - 1));
		ensure(HashToSubsystemName.Num() == (HashId - 1));

		// FUniqueNetIdRepl uses 5 bits to transmit the HashId
		// 31 is used for OnlineSubsystems not included in this list
		// 30 is used for OnlineServices (v2) net ids
		check(HashId < 30);
	}

	/** If false it will not try to do online PIE at all */
	bool bShouldTryOnlinePIE;

	/** Delegate set by the engine for notification of external UI operations */
	FOnExternalUIChangeDelegate OnExternalUIChangeDelegate;
	TMap<FName, FDelegateHandle> ExternalUIDelegateHandles;

	/** Delegate binding when new online subsystems are created */
	FDelegateHandle OnOnlineSubsystemCreatedDelegateHandle;
	/** Mapping of OSS Names to uint8 hashes */
	TMap<FName, uint8> SubsystemNameToHash;
	TMap<uint8, FName> HashToSubsystemName;

	friend FOnlineSubsystemUtilsModule;
};

FOnlineSubsystemUtilsModule::FOnlineSubsystemUtilsModule()
{
}

FOnlineSubsystemUtilsModule::~FOnlineSubsystemUtilsModule()
{
}

void FOnlineSubsystemUtilsModule::StartupModule()
{
	TUniquePtr<FOnlineSubsystemUtils> TempUtils = MakeUnique<FOnlineSubsystemUtils>();
	TempUtils->Init();
	SubsystemUtils = MoveTemp(TempUtils);

	TUniquePtr<UE::Online::FOnlineServicesEngineUtils> TempServicesUtils = MakeUnique<UE::Online::FOnlineServicesEngineUtils>();
	TempServicesUtils->Init();
	OnlineServicesUtils = MoveTemp(TempServicesUtils);
}

void FOnlineSubsystemUtilsModule::ShutdownModule()
{
	SubsystemUtils.Reset();
	OnlineServicesUtils.Reset();
}

#if WITH_EDITOR
bool IOnlineSubsystemUtils::IsOnlinePIEEnabledStatic()
{
	if (UObjectInitialized())
	{
		if (IOnlineSubsystemUtils* OnlineUtils = Online::GetUtils())
		{
			return OnlineUtils->IsOnlinePIEEnabled();
		}
	}

	// Fallback: read INI directly (CDO not available yet)
	bool bOnlinePIEEnabled = false;
	GConfig->GetBool(TEXT("/Script/OnlineSubsystemUtils.OnlinePIESettings"), TEXT("bOnlinePIEEnabled"), bOnlinePIEEnabled, GEditorPerProjectIni);

	if (bOnlinePIEEnabled)
	{
		TArray<FString> PIELogins;
		GConfig->GetArray(TEXT("/Script/OnlineSubsystemUtils.OnlinePIESettings"), TEXT("Logins"), PIELogins, GEditorPerProjectIni);
		if (PIELogins.Num() == 0)
		{
			bOnlinePIEEnabled = false;
		}
	}

	return bOnlinePIEEnabled || HasAuthCommandLineStatic();
}
#endif
