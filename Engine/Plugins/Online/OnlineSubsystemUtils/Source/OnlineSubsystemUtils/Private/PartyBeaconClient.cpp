// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartyBeaconClient.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemUtils.h"
#include "PartyBeaconHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PartyBeaconClient)

namespace BeaconConsoleVariables
{
#if !UE_BUILD_SHIPPING
	/** Time to delay delegates firing a reservation request response */
	TAutoConsoleVariable<float> CVarDelayReservationResponse(
		TEXT("beacon.DelayReservationResponse"),
		0.0f,
		TEXT("Delay time between received response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);

	/** Time to delay delegates firing a cancel reservation request response */
	TAutoConsoleVariable<float> CVarDelayCancellationResponse(
		TEXT("beacon.DelayCancellationResponse"),
		0.0f,
		TEXT("Delay time between received cancel response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);

	/** Time to delay delegates firing a reservation update response */
	TAutoConsoleVariable<float> CVarDelayUpdateResponse(
		TEXT("beacon.DelayUpdateResponse"),
		0.0f,
		TEXT("Delay time between received update response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);

	/** Time to delay delegates firing a reservation full response */
	TAutoConsoleVariable<float> CVarDelayFullResponse(
		TEXT("beacon.DelayFullResponse"),
		0.0f,
		TEXT("Delay time between received full response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);
#endif

	/** Whether to restrict party reservation RPCs to require that the initiator is the party leader. */
	TAutoConsoleVariable<bool> CVarRequireInitiatorIsPartyLeader(
		TEXT("beacon.RequireInitiatorIsPartyLeader"),
		true,
		TEXT("Enforce RPC validation which checks whether the initiator of a reservation RPC is the party leader\n")
		TEXT("Enabled"),
		ECVF_Default);
}

namespace
{
	bool IsRPCInitiatorValid(const FUniqueNetIdRepl& BeaconOwner, const FUniqueNetIdRepl& RequestPartyLeader)
	{
		const bool bInitiatorValidationEnabled = BeaconConsoleVariables::CVarRequireInitiatorIsPartyLeader.GetValueOnGameThread();
		const bool bInitiatorIsValid = bInitiatorValidationEnabled ? (RequestPartyLeader.IsValid() && RequestPartyLeader == BeaconOwner) : true;
		return bInitiatorIsValid;
	}
}

/** Max time to wait for a response from the server for CancelReservation */
#define CANCEL_FAILSAFE 5.0f

APartyBeaconClient::APartyBeaconClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	RequestType(EClientRequestType::NonePending),
	bPendingReservationSent(false),
	bCancelReservation(false)
{
}

void APartyBeaconClient::BeginDestroy()
{
	ClearTimers(true);
	Super::BeginDestroy();
}

void APartyBeaconClient::ClearTimers(bool bCallFailSafeIfNeeded)
{
	if (bCallFailSafeIfNeeded &&
		CancelRPCFailsafe.IsValid())
	{
		UE_LOGF(LogBeacon, Verbose, "Clearing timers with cancel reservation in flight.  Calling Failsafe.");

		UWorld* World = GetWorld();
		if (World)
		{
			FTimerManager& TM = World->GetTimerManager();
			TM.ClearTimer(CancelRPCFailsafe);
		}
		OnCancelledFailsafe();
	}

	UWorld* World = GetWorld();
	if (World)
	{
		if (PendingResponseTimerHandle.IsValid())
		{
			UE_LOGF(LogPartyBeacon, Verbose, "ClearTimers: Pending reservation response cleared.");
		}

		if (PendingCancelResponseTimerHandle.IsValid())
		{
			UE_LOGF(LogPartyBeacon, Verbose, "ClearTimers: Pending cancel response cleared.");
		}

		if (PendingReservationUpdateTimerHandle.IsValid())
		{
			UE_LOGF(LogPartyBeacon, Verbose, "ClearTimers: Pending reservation update cleared.");
		}

		if (PendingReservationFullTimerHandle.IsValid())
		{
			UE_LOGF(LogPartyBeacon, Verbose, "ClearTimers: Pending reservation full cleared.");
		}

		if (CancelRPCFailsafe.IsValid())
		{
			UE_LOGF(LogPartyBeacon, Verbose, "ClearTimers: Cancel failsafe cleared.");
		}

		FTimerManager& TM = World->GetTimerManager();
		TM.ClearTimer(PendingResponseTimerHandle);
		TM.ClearTimer(PendingCancelResponseTimerHandle);
		TM.ClearTimer(PendingReservationUpdateTimerHandle);
		TM.ClearTimer(PendingReservationFullTimerHandle);
		TM.ClearTimer(CancelRPCFailsafe);

		PendingResponseTimerHandle.Invalidate();
		PendingCancelResponseTimerHandle.Invalidate();
		PendingReservationUpdateTimerHandle.Invalidate();
		PendingReservationFullTimerHandle.Invalidate();
		CancelRPCFailsafe.Invalidate();
	}
}

bool APartyBeaconClient::RequestReservation(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PartyMembers)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PartyBeaconClient_RequestReservation_FourParams);

	bool bSuccess = false;

	FURL ConnectURL(NULL, *ConnectInfoStr, TRAVEL_Absolute);
	if (InitClient(ConnectURL))
	{
		DestSessionId = InSessionId;
		PendingReservation.PartyLeader = RequestingPartyLeader;
		PendingReservation.PartyMembers = PartyMembers;
		bPendingReservationSent = false;
		RequestType = EClientRequestType::ExistingSessionReservation;
		bSuccess = true;
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "RequestReservation: Failure to init client beacon with %ls.", *ConnectURL.ToString());
		RequestType = EClientRequestType::NonePending;
	}

	if (!bSuccess)
	{
		AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_party_init_failure"));
	}

	return bSuccess;
}

bool APartyBeaconClient::RequestReservation(const FOnlineSessionSearchResult& DesiredHost, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PartyMembers)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PartyBeaconClient_RequestReservation_ThreeParams);

	if (DesiredHost.IsValid())
	{
		UWorld* World = GetWorld();

		IOnlineSubsystem* OnlineSub = Online::GetSubsystem(World);
		if (OnlineSub)
		{
			IOnlineSessionPtr SessionInt = OnlineSub->GetSessionInterface();
			if (SessionInt.IsValid())
			{
				FString ConnectInfo;
				if (SessionInt->GetResolvedConnectString(DesiredHost, NAME_BeaconPort, ConnectInfo))
				{
					FString SessionId = DesiredHost.Session.SessionInfo->GetSessionId().ToString();
					return RequestReservation(ConnectInfo, SessionId, RequestingPartyLeader, PartyMembers);
				}
			}
		}
	}

	
	AOnlineBeacon::OnFailure(EBeaconFailureReason::ApplicationError, TEXTVIEW("beacon_party_request_reservation_failure"));

	return false;
}

bool APartyBeaconClient::RequestReservationUpdate(const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PlayersToModify, bool bRemovePlayers)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PartyBeaconClient_RequestReservationUpdate_ThreeParams);

	bool bWasStarted = false;

	EBeaconConnectionState MyConnectionState = GetConnectionState();
	if (ensure(MyConnectionState == EBeaconConnectionState::Open))
	{
		RequestType = bRemovePlayers ? EClientRequestType::ReservationRemoveMembers : EClientRequestType::ReservationUpdate;
		PendingReservation.PartyLeader = RequestingPartyLeader;
		PendingReservation.PartyMembers = PlayersToModify;
		ServerUpdateReservationRequest(DestSessionId, PendingReservation);
		bPendingReservationSent = true;
		bWasStarted = true;
	}

	return bWasStarted;
}

bool APartyBeaconClient::RequestReservationUpdate(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PlayersToModify, bool bRemovePlayers)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PartyBeaconClient_RequestReservationUpdate_FiveParams);

	bool bWasStarted = false;

	if (!ConnectInfoStr.IsEmpty() && !InSessionId.IsEmpty())
	{
		EBeaconConnectionState MyConnectionState = GetConnectionState();
		if (MyConnectionState != EBeaconConnectionState::Open)
		{
			// create a new pending reservation for these players in the same way as a new reservation request
			bWasStarted = RequestReservation(ConnectInfoStr, InSessionId, RequestingPartyLeader, PlayersToModify);
			if (bWasStarted)
			{
				// Treat this reservation as an update to an existing reservation on the host
				RequestType = bRemovePlayers ? EClientRequestType::ReservationRemoveMembers : EClientRequestType::ReservationUpdate;
			}
		}
		else
		{
			bWasStarted = RequestReservationUpdate(RequestingPartyLeader, PlayersToModify, bRemovePlayers);
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "APartyBeaconClient::RequestReservationUpdate: Missing ConnectInfoStr ('%ls') or SessionId ('%ls').", *ConnectInfoStr, *InSessionId);
	}

	return bWasStarted;
}

bool APartyBeaconClient::RequestReservationUpdate(const FOnlineSessionSearchResult& DesiredHost, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PlayersToModify, bool bRemovePlayers)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PartyBeaconClient_RequestReservationUpdate_FourParams);

	bool bWasStarted = false;

	EBeaconConnectionState MyConnectionState = GetConnectionState();
	if (MyConnectionState != EBeaconConnectionState::Open)
	{
		// create a new pending reservation for these players in the same way as a new reservation request
		bWasStarted = RequestReservation(DesiredHost, RequestingPartyLeader, PlayersToModify);
		if (bWasStarted)
		{
			// Treat this reservation as an update to an existing reservation on the host
			RequestType = bRemovePlayers ? EClientRequestType::ReservationRemoveMembers : EClientRequestType::ReservationUpdate;
		}
	}
	else
	{
		RequestReservationUpdate(RequestingPartyLeader, PlayersToModify, bRemovePlayers);
	}

	return bWasStarted;
}

bool APartyBeaconClient::RequestAddOrUpdateReservation(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PartyMembers)
{
	bool bSuccess = false;

	FURL ConnectURL(NULL, *ConnectInfoStr, TRAVEL_Absolute);
	if (InitClient(ConnectURL))
	{
		DestSessionId = InSessionId;
		PendingReservation.PartyLeader = RequestingPartyLeader;
		PendingReservation.PartyMembers = PartyMembers;
		bPendingReservationSent = false;
		RequestType = EClientRequestType::AddOrUpdateReservation;
		bSuccess = true;
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "%ls - Failure to init client beacon with %ls.", ANSI_TO_TCHAR(__FUNCTION__ ), *ConnectURL.ToString());
		RequestType = EClientRequestType::NonePending;
	}

	if (!bSuccess)
	{
		AOnlineBeacon::OnFailure(EBeaconFailureReason::ApplicationError, TEXTVIEW("beacon_party_update_reservation_failure"));
	}

	return bSuccess;
}

void APartyBeaconClient::CancelReservation()
{
	if (ensure(PendingReservation.PartyLeader.IsValid()))
	{
		bCancelReservation = true;

		// Clear out any pending response handling, only the cancel matters
		ClearTimers(false);

		if (bPendingReservationSent)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Sending cancel reservation request.");
			ServerCancelReservationRequest(PendingReservation.PartyLeader);

			// In case the server is loading or unresponsive (ie no host beacon)
			FTimerDelegate TimerDelegate;
			TimerDelegate.BindUObject(this, &ThisClass::OnCancelledFailsafe);
			
			UWorld* World = GetWorld();
			check(World);
			World->GetTimerManager().SetTimer(CancelRPCFailsafe, TimerDelegate, CANCEL_FAILSAFE, false);
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Reservation request never sent, no need to send cancelation request.");
			OnCancelledComplete();
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Verbose, "Unable to cancel reservation request with invalid party leader.");
		OnCancelledComplete();
	}
}

void APartyBeaconClient::OnConnected()
{
	if (!bCancelReservation)
	{
		if (RequestType == EClientRequestType::AddOrUpdateReservation)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "%ls - Party beacon connection established, sending add or update reservation request.", ANSI_TO_TCHAR(__FUNCTION__));
			ServerAddOrUpdateReservationRequest(DestSessionId, PendingReservation);
			bPendingReservationSent = true;
		}
		else if (RequestType == EClientRequestType::ExistingSessionReservation)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Party beacon connection established, sending join reservation request.");
			ServerReservationRequest(DestSessionId, PendingReservation);
			bPendingReservationSent = true;
		}
		else if (RequestType == EClientRequestType::ReservationUpdate)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Party beacon connection established, sending reservation update request.");
			ServerUpdateReservationRequest(DestSessionId, PendingReservation);
			bPendingReservationSent = true;
		}
		else if (RequestType == EClientRequestType::ReservationRemoveMembers)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Party beacon connection established, sending reservation update request to remove player.");
			ServerRemoveMemberFromReservationRequest(DestSessionId, PendingReservation);
			bPendingReservationSent = true;
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Warning, "Failed to handle reservation request type %ls", ToString(RequestType));
			AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_party_reservation_failure"));
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Verbose, "Reservation request previously canceled, nothing sent.");
		OnCancelledComplete();
	}
}

void APartyBeaconClient::OnCancelledFailsafe()
{
	ClientCancelReservationResponse_Implementation(EPartyReservationResult::ReservationRequestCanceled);
}
 
void APartyBeaconClient::OnCancelledComplete()
{
	ReservationRequestComplete.ExecuteIfBound(EPartyReservationResult::ReservationRequestCanceled);
	RequestType = EClientRequestType::NonePending;
	bCancelReservation = false;
}

void APartyBeaconClient::OnFailure()
{
	ClearTimers(true);
	RequestType = EClientRequestType::NonePending;
	Super::OnFailure();
}

/// @cond DOXYGEN_WARNINGS

bool APartyBeaconClient::ServerReservationRequest_Validate(const FString& SessionId, const FPartyReservation& Reservation)
{
	const bool bInitiatorIsValid = IsRPCInitiatorValid(GetUniqueId(), Reservation.PartyLeader);
	const bool bRequestValid = bInitiatorIsValid && !SessionId.IsEmpty() && Reservation.PartyLeader.IsValid() && Reservation.PartyMembers.Num() > 0;
	return bRequestValid;
}

void APartyBeaconClient::ServerReservationRequest_Implementation(const FString& SessionId, const FPartyReservation& Reservation)
{
	APartyBeaconHost* BeaconHost = Cast<APartyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		PendingReservation = Reservation;
		RequestType = EClientRequestType::ExistingSessionReservation;
		BeaconHost->ProcessReservationRequest(this, SessionId, Reservation);
	}
}

bool APartyBeaconClient::ServerUpdateReservationRequest_Validate(const FString& SessionId, const FPartyReservation& ReservationUpdate)
{
	const bool bInitiatorIsValid = IsRPCInitiatorValid(GetUniqueId(), ReservationUpdate.PartyLeader);
	const bool bRequestValid = bInitiatorIsValid && !SessionId.IsEmpty() && ReservationUpdate.PartyLeader.IsValid() && ReservationUpdate.PartyMembers.Num() > 0;
	return bRequestValid;
}

void APartyBeaconClient::ServerUpdateReservationRequest_Implementation(const FString& SessionId, const FPartyReservation& ReservationUpdate)
{
	APartyBeaconHost* BeaconHost = Cast<APartyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		PendingReservation = ReservationUpdate;
		RequestType = EClientRequestType::ReservationUpdate;
		BeaconHost->ProcessReservationUpdateRequest(this, SessionId, ReservationUpdate, false);
	}
}

bool APartyBeaconClient::ServerAddOrUpdateReservationRequest_Validate(const FString& SessionId, const FPartyReservation& Reservation)
{
	const bool bInitiatorIsValid = IsRPCInitiatorValid(GetUniqueId(), Reservation.PartyLeader);
	const bool bRequestValid = bInitiatorIsValid && !SessionId.IsEmpty() && Reservation.PartyLeader.IsValid() && Reservation.PartyMembers.Num() > 0;
	return bRequestValid;
}

void APartyBeaconClient::ServerAddOrUpdateReservationRequest_Implementation(const FString& SessionId, const FPartyReservation& Reservation)
{
	if (APartyBeaconHost* BeaconHost = Cast<APartyBeaconHost>(GetBeaconOwner()))
	{
		PendingReservation = Reservation;
		RequestType = EClientRequestType::ExistingSessionReservation;
		BeaconHost->ProcessReservationAddOrUpdateRequest(this, SessionId, Reservation);
	}
}

bool APartyBeaconClient::ServerRemoveMemberFromReservationRequest_Validate(const FString& SessionId, const FPartyReservation& ReservationUpdate)
{
	const bool bInitiatorIsValid = IsRPCInitiatorValid(GetUniqueId(), ReservationUpdate.PartyLeader);
	const bool bRequestValid = bInitiatorIsValid && !SessionId.IsEmpty() && ReservationUpdate.PartyLeader.IsValid() && ReservationUpdate.PartyMembers.Num() > 0;
	return bRequestValid;
}

void APartyBeaconClient::ServerRemoveMemberFromReservationRequest_Implementation(const FString& SessionId, const FPartyReservation& ReservationUpdate)
{
	APartyBeaconHost* BeaconHost = Cast<APartyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		PendingReservation = ReservationUpdate;
		RequestType = EClientRequestType::ReservationRemoveMembers;
		BeaconHost->ProcessReservationUpdateRequest(this, SessionId, ReservationUpdate, true);
	}
}

bool APartyBeaconClient::ServerCancelReservationRequest_Validate(const FUniqueNetIdRepl& PartyLeader)
{
	const bool bInitiatorIsValid = IsRPCInitiatorValid(GetUniqueId(), PartyLeader);
	const bool bRequestValid = bInitiatorIsValid && PartyLeader.IsValid();
	return bRequestValid;
}

void APartyBeaconClient::ServerCancelReservationRequest_Implementation(const FUniqueNetIdRepl& PartyLeader)
{
	APartyBeaconHost* BeaconHost = Cast<APartyBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		bCancelReservation = true;
		BeaconHost->ProcessCancelReservationRequest(this, PartyLeader);
	}
}

void APartyBeaconClient::ClientReservationResponse_Implementation(EPartyReservationResult::Type ReservationResponse)
{
	if (!bCancelReservation)
	{
#if !UE_BUILD_SHIPPING
		const float Rate = BeaconConsoleVariables::CVarDelayReservationResponse.GetValueOnGameThread();
#else
		const float Rate = 0.0f;
#endif
		if (Rate > 0.0f)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Party beacon response received %ls, waiting %fs to notify", EPartyReservationResult::ToString(ReservationResponse), Rate);

			FTimerDelegate TimerDelegate;
			TimerDelegate.BindLambda([this, ReservationResponse]()
			{
				ProcessReservationResponse(ReservationResponse);
			});

			PendingResponseTimerHandle = DelayResponse(TimerDelegate, Rate);
		}
		else
		{
			ProcessReservationResponse(ReservationResponse);
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Verbose, "Party beacon response received %ls, ignored due to cancel in progress", EPartyReservationResult::ToString(ReservationResponse));
		// Cancel RPC or failsafe timer will trigger the cancel
	}
}

/// @endcond

void APartyBeaconClient::ProcessReservationResponse(EPartyReservationResult::Type ReservationResponse)
{
	if (!bCancelReservation)
	{
		UE_LOGF(LogPartyBeacon, Verbose, "Party beacon response received %ls", EPartyReservationResult::ToString(ReservationResponse));
		ReservationRequestComplete.ExecuteIfBound(ReservationResponse);
		RequestType = EClientRequestType::NonePending;
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Verbose, "Party beacon response received %ls, ignored due to cancel in progress", EPartyReservationResult::ToString(ReservationResponse));
		// Cancel RPC or failsafe timer will trigger the cancel
	}
}

/// @cond DOXYGEN_WARNINGS

void APartyBeaconClient::ClientCancelReservationResponse_Implementation(EPartyReservationResult::Type ReservationResponse)
{
	ensure(bCancelReservation);

	// Clear out any pending response handling (including failsafe timer)
	ClearTimers(false);
#if !UE_BUILD_SHIPPING	
	const float Rate = BeaconConsoleVariables::CVarDelayCancellationResponse.GetValueOnGameThread();
#else
	const float Rate = 0.0f;
#endif
	if (Rate > 0.0f)
	{
		UE_LOGF(LogPartyBeacon, Verbose, "Party beacon cancellation response received %ls, waiting %fs to notify", EPartyReservationResult::ToString(ReservationResponse), Rate);

		FTimerDelegate TimerDelegate;
		TimerDelegate.BindLambda([this, ReservationResponse]()
		{
			ProcessCancelReservationResponse(ReservationResponse);
		});

		PendingCancelResponseTimerHandle = DelayResponse(TimerDelegate, Rate);
	}
	else
	{
		ProcessCancelReservationResponse(ReservationResponse);
	}
}

/// @endcond

void APartyBeaconClient::ProcessCancelReservationResponse(EPartyReservationResult::Type ReservationResponse)
{
	ensure(ReservationResponse == EPartyReservationResult::ReservationRequestCanceled || ReservationResponse == EPartyReservationResult::ReservationNotFound);
	ensure(bCancelReservation);

	OnCancelledComplete();
}

/// @cond DOXYGEN_WARNINGS

void APartyBeaconClient::ClientSendReservationUpdates_Implementation(int32 NumRemainingReservations)
{
	if (!bCancelReservation)
	{
#if !UE_BUILD_SHIPPING
		const float Rate = BeaconConsoleVariables::CVarDelayUpdateResponse.GetValueOnGameThread();
#else
		const float Rate = 0.0f;
#endif
		if (Rate > 0.0f)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Party beacon reservations remaining %d, waiting %fs to notify", NumRemainingReservations, Rate);

			FTimerDelegate TimerDelegate;
			TimerDelegate.BindLambda([this, NumRemainingReservations]()
			{
				ProcessReservationUpdate(NumRemainingReservations);
			});

			PendingReservationUpdateTimerHandle = DelayResponse(TimerDelegate, Rate);
		}
		else
		{
			ProcessReservationUpdate(NumRemainingReservations);
		}
	}
}

/// @endcond

void APartyBeaconClient::ProcessReservationUpdate(int32 NumRemainingReservations)
{
	UE_LOGF(LogPartyBeacon, Verbose, "Party beacon reservations remaining %d", NumRemainingReservations);
	ReservationCountUpdate.ExecuteIfBound(NumRemainingReservations);
}

/// @cond DOXYGEN_WARNINGS

void APartyBeaconClient::ClientSendReservationFull_Implementation()
{
	if (!bCancelReservation)
	{
#if !UE_BUILD_SHIPPING
		const float Rate = BeaconConsoleVariables::CVarDelayFullResponse.GetValueOnGameThread();
#else
		const float Rate = 0.0f;
#endif
		if (Rate > 0.0f)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Party beacon reservations full, waiting %fs to notify", Rate);

			FTimerDelegate TimerDelegate;
			TimerDelegate.BindLambda([this]()
			{
				ProcessReservationFull();
			});

			PendingReservationFullTimerHandle = DelayResponse(TimerDelegate, Rate);
		}
		else
		{
			ProcessReservationFull();
		}
	}
}

/// @endcond

void APartyBeaconClient::ProcessReservationFull()
{
	UE_LOGF(LogPartyBeacon, Verbose, "Party beacon reservations full");
	ReservationFull.ExecuteIfBound();
}

FTimerHandle APartyBeaconClient::DelayResponse(FTimerDelegate& Delegate, float Delay)
{
	FTimerHandle TimerHandle;

	UWorld* World = GetWorld();
	if (ensure(World != nullptr))
	{
		World->GetTimerManager().SetTimer(TimerHandle, Delegate, Delay, false);
	}

	return TimerHandle;
}
