// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartyBeaconHost.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Misc/CommandLine.h"
#include "OnlineSessionSettings.h"
#include "UObject/Package.h"
#include "Engine/NetConnection.h"
#include "OnlineSubsystemUtils.h"
#include "PartyBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PartyBeaconHost)

namespace BeaconConsoleVariables
{
TAutoConsoleVariable<bool> CVarRemapReservationUpdateToExistingReservation(
	TEXT("beacon.RemapReservationUpdateToExistingReservation"),
	true,
	TEXT("Whether to remap the leader of a reservation update request to the leader of the reservation they appear in.\n")
	TEXT("Enabled"),
	ECVF_Default);
}

APartyBeaconHost::APartyBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	State(NULL),
	bLogoutOnSessionTimeout(true),
	bIsValidationStrRequired(true)
{
	ClientBeaconActorClass = APartyBeaconClient::StaticClass();
	BeaconTypeName = ClientBeaconActorClass->GetName();

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bAllowTickOnDedicatedServer = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
}

void APartyBeaconHost::PostInitProperties()
{
	Super::PostInitProperties();
#if !UE_BUILD_SHIPPING
	// This value is set on the CDO as well on purpose
	bLogoutOnSessionTimeout = FParse::Param(FCommandLine::Get(), TEXT("NoTimeouts")) ? false : true;
#endif
}

bool APartyBeaconHost::InitHostBeacon(int32 InTeamCount, int32 InTeamSize, int32 InMaxReservations, FName InSessionName, int32 InForceTeamNum, bool bInEnableRemovalRequests)
{
	UE_LOGF(LogPartyBeacon, Verbose, "InitHostBeacon TeamCount:%d TeamSize:%d MaxSize:%d", InTeamCount, InTeamSize, InMaxReservations);
	if (InMaxReservations > 0)
	{
		State = NewObject<UPartyBeaconState>(GetTransientPackage(), GetPartyBeaconHostClass());
		if (State->InitState(InTeamCount, InTeamSize, InMaxReservations, InSessionName, InForceTeamNum, bInEnableRemovalRequests))
		{
			return true;
		}
	}

	return false;
}

bool APartyBeaconHost::InitFromBeaconState(UPartyBeaconState* PrevState)
{
	if (!State && PrevState)
	{
		UE_LOGF(LogPartyBeacon, Verbose, "InitFromBeaconState TeamCount:%d TeamSize:%d MaxSize:%d", PrevState->NumTeams, PrevState->NumPlayersPerTeam, PrevState->MaxReservations);
		State = PrevState;
		return true;
	}

	return false;
}

bool APartyBeaconHost::ReconfigureTeamAndPlayerCount(int32 InNumTeams, int32 InNumPlayersPerTeam, int32 InNumReservations)
{
	bool bSuccess = false;
	if (GetOwner() != NULL && State != NULL)
	{
		bSuccess = State->ReconfigureTeamAndPlayerCount(InNumTeams, InNumPlayersPerTeam, InNumReservations);
		UE_LOGF(LogPartyBeacon, Log,
			"Beacon (%ls) reconfiguring team and player count.",
			*GetBeaconType());
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, can't change team and player count.",
			*GetBeaconType());
	}

	return bSuccess;
}

void APartyBeaconHost::SetCompetitiveIntegrity(bool bNewCompetitiveIntegrity)
{
	State->SetCompetitiveIntegrity(bNewCompetitiveIntegrity);
}

void APartyBeaconHost::SetTeamAssignmentMethod(FName NewAssignmentMethod)
{
	if (State)
	{
		State->SetTeamAssignmentMethod(NewAssignmentMethod);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "SetTeamAssignmentMethod failed for beacon with no state!");
	}
}

bool APartyBeaconHost::IsPlayerActive(const FUniqueNetIdRepl& PlayerId) const
{
	IOnlineSessionPtr SessionInt = Online::GetSessionInterface(GetWorld());
	if (SessionInt.IsValid())
	{
		return SessionInt->IsPlayerInSession(State->GetSessionName(), *PlayerId);
	}

	return false;
}

void APartyBeaconHost::Tick(float DeltaTime)
{
	if (State)
	{
		UWorld* World = GetWorld();
		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);

		if (SessionInt.IsValid())
		{
			FName SessionName = State->GetSessionName();
			TArray<FPartyReservation>& Reservations = State->GetReservations();

			FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
			if (Session)
			{
				TArray< FUniqueNetIdPtr > PlayersToLogout;
				for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
				{
					FPartyReservation& PartyRes = Reservations[ResIdx];

					// Determine if we have a client connection for the reservation 
					bool bIsConnectedReservation = false;
					for (int32 ClientIdx = 0; ClientIdx < ClientActors.Num(); ClientIdx++)
					{
						APartyBeaconClient* Client = Cast<APartyBeaconClient>(ClientActors[ClientIdx]);
						if (Client)
						{
							const FPartyReservation& ClientRes = Client->GetPendingReservation();
							if (ClientRes.PartyLeader == PartyRes.PartyLeader)
							{
								bIsConnectedReservation = true;
								break;
							}
						}
						else
						{
							UE_LOGF(LogPartyBeacon, Error, "Missing PartyBeaconClient found in ClientActors array");
							ClientActors.RemoveAtSwap(ClientIdx);
							ClientIdx--;
						}
					}

					// Don't update clients that are still connected
					if (bIsConnectedReservation)
					{
						for (int32 PlayerIdx = 0; PlayerIdx < PartyRes.PartyMembers.Num(); PlayerIdx++)
						{
							FPlayerReservation& PlayerEntry = PartyRes.PartyMembers[PlayerIdx];
							PlayerEntry.ElapsedTime = 0.0f;
						}
					}
					// Once a client beacon disconnects, update the elapsed time since they were found as a registrant in the game session
					else
					{
						for (int32 PlayerIdx = 0; PlayerIdx < PartyRes.PartyMembers.Num(); PlayerIdx++)
						{
							FPlayerReservation& PlayerEntry = PartyRes.PartyMembers[PlayerIdx];

							// Determine if the player is the owner of the session	
							const bool bIsSessionOwner = Session->OwningUserId.IsValid() && (*Session->OwningUserId == *PlayerEntry.UniqueId);

							// Determine if the player member is registered in the game session
							if (IsPlayerActive(PlayerEntry.UniqueId) ||
								// Never timeout the session owner
								bIsSessionOwner)
							{
								FUniqueNetIdMatcher PlayerMatch(*PlayerEntry.UniqueId);
								int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
								if (FoundIdx != INDEX_NONE)
								{
									UE_LOGF(LogPartyBeacon, Display, "Beacon (%ls): pending player %ls found in session (%ls), removing.",
										*GetName(),
										*(PlayerEntry.UniqueId.ToDebugString()),
										*SessionName.ToString());

									// reset elapsed time since found
									PlayerEntry.ElapsedTime = 0.0f;
									// also remove from pending join list
									State->PlayersPendingJoin.RemoveAtSwap(FoundIdx);
								}
							}
							else
							{
								// update elapsed time
								PlayerEntry.ElapsedTime += DeltaTime;

								if (bLogoutOnSessionTimeout)
								{
									// if the player is pending it's initial join then check against TravelSessionTimeoutSecs instead
									FUniqueNetIdMatcher PlayerMatch(*PlayerEntry.UniqueId);
									const int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
									const bool bIsPlayerPendingJoin = FoundIdx != INDEX_NONE;
									// if the timeout has been exceeded then add to list of players 
									// that need to be logged out from the beacon
									if ((bIsPlayerPendingJoin && PlayerEntry.ElapsedTime > TravelSessionTimeoutSecs) ||
										(!bIsPlayerPendingJoin && PlayerEntry.ElapsedTime > GetSessionTimeoutSecs(PlayerEntry.UniqueId)))
									{
										PlayersToLogout.AddUnique(PlayerEntry.UniqueId.GetUniqueNetId());
									}
								}
							}
						}
					}
				}

				if (bLogoutOnSessionTimeout)
				{
					// Logout any players that timed out
					for (int32 LogoutIdx = 0; LogoutIdx < PlayersToLogout.Num(); LogoutIdx++)
					{
						bool bFound = false;
						const FUniqueNetIdPtr& UniqueId = PlayersToLogout[LogoutIdx];
						float ElapsedSessionTime = 0.f;
						for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
						{
							const FPartyReservation& PartyRes = Reservations[ResIdx];
							for (int32 PlayerIdx = 0; PlayerIdx < PartyRes.PartyMembers.Num(); PlayerIdx++)
							{
								const FPlayerReservation& PlayerEntry = PartyRes.PartyMembers[PlayerIdx];
								if (*PlayerEntry.UniqueId == *UniqueId)
								{
									ElapsedSessionTime = PlayerEntry.ElapsedTime;
									bFound = true;
									break;
								}
							}

							if (bFound)
							{
								break;
							}
						}

						UE_LOGF(LogPartyBeacon, Display, "Beacon (%ls): pending player logout due to timeout for %ls, elapsed time = %0.3f, removing",
							*GetName(),
							*UniqueId->ToDebugString(),
							ElapsedSessionTime);
						// Also remove from pending join list
						State->PlayersPendingJoin.RemoveSingleSwap(UniqueId);
						// let the beacon handle the logout and notifications/delegates
						FUniqueNetIdRepl RemovedId(UniqueId);
						HandlePlayerLogout(RemovedId);
					}
				}
			}
		}
	}
}

int32 APartyBeaconHost::GetNumPlayersOnTeam(int32 TeamIdx) const
{
	int32 Result = 0;
	if (GetOwner() != NULL && State != NULL)
	{
		Result = State->GetNumPlayersOnTeam(TeamIdx);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, can't get team player count.",
			*GetBeaconType());
	}

	return Result;
}

int32 APartyBeaconHost::GetTeamForCurrentPlayer(const FUniqueNetId& PlayerId) const
{
	int32 TeamNum = INDEX_NONE;
	if (PlayerId.IsValid())
	{
		if (State)
		{
			TeamNum = State->GetTeamForCurrentPlayer(PlayerId);
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Display, "Invalid player when attempting to find team assignment");
	}

	return TeamNum;
}

int32 APartyBeaconHost::GetPlayersOnTeam(int32 TeamIndex, TArray<FUniqueNetIdRepl>& TeamMembers) const
{
	if (State)
	{
		if (TeamIndex < State->GetNumTeams())
		{
			return State->GetPlayersOnTeam(TeamIndex, TeamMembers);
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Warning, "GetPlayersOnTeam: Invalid team index %d", TeamIndex);
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "GetPlayersOnTeam failed for beacon with no state!");
	}

	return 0;
}

void APartyBeaconHost::SendReservationUpdates()
{
	if (State && ClientActors.Num() > 0)
	{
		int32 NumRemaining = State->GetRemainingReservations();
		int32 MaxReservations = State->GetMaxReservations();
		if (NumRemaining < MaxReservations)
		{
			if (NumRemaining > 0)
			{
				UE_LOGF(LogPartyBeacon, Verbose, "Sending reservation update %d", NumRemaining);
				for (AOnlineBeaconClient* ClientActor : ClientActors)
				{
					APartyBeaconClient* PartyBeaconClient = Cast<APartyBeaconClient>(ClientActor);
					if (PartyBeaconClient)
					{
						PartyBeaconClient->ClientSendReservationUpdates(NumRemaining);
					}
				}
			}
			else
			{
				UE_LOGF(LogPartyBeacon, Verbose, "Sending reservation full");
				for (AOnlineBeaconClient* ClientActor : ClientActors)
				{
					APartyBeaconClient* PartyBeaconClient = Cast<APartyBeaconClient>(ClientActor);
					if (PartyBeaconClient)
					{
						PartyBeaconClient->ClientSendReservationFull();
					}
				}
			}
		}
	}
}

void APartyBeaconHost::PlayerRemoved(const FPlayerReservation& RemovedPlayer)
{
	if (RemovedPlayer.UniqueId.IsValid())
	{
		if (State)
		{
			FUniqueNetIdMatcher PlayerMatch(*RemovedPlayer.UniqueId);
			int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
			if (FoundIdx != INDEX_NONE)
			{
				UE_LOGF(LogPartyBeacon, Verbose, "Beacon removing pending player %ls", *RemovedPlayer.UniqueId.ToDebugString());
				State->PlayersPendingJoin.Remove(RemovedPlayer.UniqueId.GetUniqueNetId());
			}
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Warning, "Beacon skipping PlayersPendingJoin for beacon with no state!");
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "Beacon skipping PlayersPendingJoin for invalid player!");
	}
}

void APartyBeaconHost::NewPlayerAdded(const FPlayerReservation& NewPlayer)
{
	if (NewPlayer.UniqueId.IsValid())
	{
		if (State)
		{
			FUniqueNetIdMatcher PlayerMatch(*NewPlayer.UniqueId);
			int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
			if (FoundIdx == INDEX_NONE)
			{
				UE_LOGF(LogPartyBeacon, Verbose, "Beacon adding pending player %ls", *NewPlayer.UniqueId.ToDebugString());
				State->PlayersPendingJoin.Add(NewPlayer.UniqueId.GetUniqueNetId());
				OnNewPlayerAdded().ExecuteIfBound(NewPlayer);
			}
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Warning, "Beacon skipping PlayersPendingJoin for beacon with no state!");
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "Beacon skipping PlayersPendingJoin for invalid player!");
	}
}

void APartyBeaconHost::NotifyReservationEventNextFrame(FOnReservationUpdate& ReservationEvent)
{
	UWorld* World = GetWorld();
	check(World);

	// Calling this on next tick to protect against re-entrance
	World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateLambda([ReservationEvent](){ ReservationEvent.ExecuteIfBound(); }));
}

void APartyBeaconHost::HandlePlayerLogout(const FUniqueNetIdRepl& PlayerId)
{
	if (PlayerId.IsValid())
	{
		UE_LOGF(LogPartyBeacon, Verbose, "HandlePlayerLogout %ls", *PlayerId->ToDebugString());

		if (State)
		{
			if (State->RemovePlayer(PlayerId))
			{
				SendReservationUpdates();
				NotifyReservationEventNextFrame(ReservationChanged);
			}
		}
	}
}

bool APartyBeaconHost::SwapTeams(const FUniqueNetIdRepl& PartyLeader, const FUniqueNetIdRepl& OtherPartyLeader)
{
	bool bSuccess = false;
	if (State)
	{
		if (State->SwapTeams(PartyLeader, OtherPartyLeader))
		{
			NotifyReservationEventNextFrame(ReservationChanged);
			bSuccess = true;
		}
	}

	return bSuccess;
}


bool APartyBeaconHost::ChangeTeam(const FUniqueNetIdRepl& PartyLeader, int32 NewTeamNum)
{
	bool bSuccess = false;
	if (State)
	{
		if (State->ChangeTeam(PartyLeader, NewTeamNum))
		{
			NotifyReservationEventNextFrame(ReservationChanged);
			bSuccess = true;
		}
	}

	return bSuccess;
}

bool APartyBeaconHost::PlayerHasReservation(const FUniqueNetId& PlayerId) const
{
	bool bHasReservation = false;
	if (State)
	{
		bHasReservation = State->PlayerHasReservation(PlayerId);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, no reservations.",
			*GetBeaconType());
	}

	return bHasReservation;
}

bool APartyBeaconHost::GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const
{
	OutValidation.Empty();

	bool bHasValidation = false;
	if (State)
	{
		bHasValidation = State->GetPlayerValidation(PlayerId, OutValidation);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, no validation.",
			*GetBeaconType());
	}

	return bHasValidation;
}

float APartyBeaconHost::GetSessionTimeoutSecs(const FUniqueNetIdRepl& PlayerId) const
{
	return SessionTimeoutSecs;
}

bool APartyBeaconHost::GetPartyLeader(const FUniqueNetIdRepl& InPartyMemberId, FUniqueNetIdRepl& OutPartyLeaderId) const
{
	bool bHasLeader = false;
	if (State)
	{
		bHasLeader = State->GetPartyLeader(InPartyMemberId, OutPartyLeaderId);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, no leader can be found.",
			*GetBeaconType());
	}

	return bHasLeader;
}

EPartyReservationResult::Type APartyBeaconHost::AddPartyReservation(const FPartyReservation& ReservationRequest)
{
	EPartyReservationResult::Type Result = EPartyReservationResult::GeneralError;

	if (!State || GetBeaconState() == EBeaconState::DenyRequests)
	{
		return EPartyReservationResult::ReservationDenied;
	}

	if (ReservationRequest.IsValid(bIsValidationStrRequired))
	{
		TArray<FPartyReservation>& Reservations = State->GetReservations();
		const int32 ExistingReservationIdx = State->GetExistingReservation(ReservationRequest.PartyLeader);
		if (ExistingReservationIdx != INDEX_NONE)
		{
			FPartyReservation& ExistingReservation = Reservations[ExistingReservationIdx];
			if (ReservationRequest.PartyMembers.Num() <= ExistingReservation.PartyMembers.Num())
			{
				// Verify the reservations are either the same, or that the incoming reservation is a subset of the existing one
				int32 NumMatchingReservations = 0;
				for (const FPlayerReservation& NewPlayerRes : ReservationRequest.PartyMembers)
				{
					const FPlayerReservation* const PlayerRes = ExistingReservation.PartyMembers.FindByPredicate(
						[NewPlayerRes](const FPlayerReservation& ExistingPlayerRes)
					{
						return NewPlayerRes.UniqueId == ExistingPlayerRes.UniqueId;
					});

					if (PlayerRes)
					{
						NumMatchingReservations++;
					}
				}

				if (NumMatchingReservations == ReservationRequest.PartyMembers.Num())
				{
					for (const FPlayerReservation& NewPlayerRes : ReservationRequest.PartyMembers)
					{
						if (!NewPlayerRes.ValidationStr.IsEmpty())
						{
							FPlayerReservation* const PlayerRes = ExistingReservation.PartyMembers.FindByPredicate(
								[NewPlayerRes](const FPlayerReservation& ExistingPlayerRes)
							{
								return NewPlayerRes.UniqueId == ExistingPlayerRes.UniqueId;
							});

							if (PlayerRes)
							{
								// Update the validation auth strings because they may have changed with a new login 
								PlayerRes->ValidationStr = NewPlayerRes.ValidationStr;
							}
						}
					}

					if (State->CrossPlayAllowed(ReservationRequest))
					{
						SendReservationUpdates();

						// Clean up the game entities for these duplicate players
						DuplicateReservation.ExecuteIfBound(ReservationRequest);

						// Add all players back into the pending join list
						for (int32 Count = 0; Count < ReservationRequest.PartyMembers.Num(); Count++)
						{
							NewPlayerAdded(ReservationRequest.PartyMembers[Count]);
						}

						Result = EPartyReservationResult::ReservationDuplicate;
					}
					else
					{
						// Cross play restriction
						Result = EPartyReservationResult::ReservationDenied_CrossPlayRestriction;
					}
				}
				else
				{
					// Existing reservation doesn't match incoming duplicate reservation
					Result = EPartyReservationResult::IncorrectPlayerCount;
				}
			}
			else
			{
				// Existing reservation holds less players than the incoming duplicate reservation
				Result = EPartyReservationResult::IncorrectPlayerCount;
			}
		}
		else
		{
			// Check for players we already have reservations for
			// Keep track of team index for existing members - if we have members on opposing teams, reject this reservation
			bool bContainsIncompatibleExistingMembers = false;
			int32 ExistingMemberReservationTeamNum = INDEX_NONE;
			int32 TeamIdx = -1;
			int32 NumTeamPlayersWithExistingReservation = 0;
			TArray<FPlayerReservation> ExistingPartyMemberReservations;
			for (const FPlayerReservation& PartyMember : ReservationRequest.PartyMembers)
			{
				int32 MemberExistingPartyReservationIdx = State->GetExistingReservationContainingMember(PartyMember.UniqueId);
				
				if (MemberExistingPartyReservationIdx != INDEX_NONE)
				{
					ExistingPartyMemberReservations.Add(PartyMember);
					const FPartyReservation& MemberExistingPartyReservation = Reservations[MemberExistingPartyReservationIdx];		
					// is the member on a suitable team (same as previous members/we the part doesn't have a team yet)
					if (TeamIdx == -1 || TeamIdx == MemberExistingPartyReservation.TeamNum) 
					{
						TeamIdx = MemberExistingPartyReservation.TeamNum;
						NumTeamPlayersWithExistingReservation++;
					}
					else
					{
						UE_LOGF(LogPartyBeacon, Display, "APartyBeaconHost::AddPartyReservation: Found existing reservation with missmatched team for party member %ls", *PartyMember.UniqueId.ToDebugString());
						ReservationRequest.Dump();
						MemberExistingPartyReservation.Dump();
						bContainsIncompatibleExistingMembers = true;
					}
				
					
				}
				else
				{
					// Is this player in the pending join list?
					FUniqueNetIdMatcher PlayerMatch(*PartyMember.UniqueId);
					int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
					if (FoundIdx != INDEX_NONE)
					{
						UE_LOGF(LogPartyBeacon, Display, "APartyBeaconHost::AddPartyReservation: Found party member %ls in the pending player list", *PartyMember.UniqueId.ToDebugString());
						ReservationRequest.Dump();
						bContainsIncompatibleExistingMembers = true;
					}
				}
			}	
			if (TeamIdx != -1)
			{	
				if (ShouldRespectCompetitiveIntegrity())
				{
					// Will the party members not currently on the team fit in it, and fit within the server
					const int32 NumTeamMembers = GetNumPlayersOnTeam(TeamIdx);
					const int32 NumAvailableSlotsOnTeam = FMath::Max<int32>(0, (GetMaxPlayersPerTeam() - NumTeamMembers) + NumTeamPlayersWithExistingReservation);
					if ((NumAvailableSlotsOnTeam < ReservationRequest.PartyMembers.Num()) || (State->GetRemainingReservations() < (ReservationRequest.PartyMembers.Num() - NumTeamPlayersWithExistingReservation)))
					{
						bContainsIncompatibleExistingMembers = true;
					}
				}
			}
			// if this party reservation included players who already had reservations, but is not incompatible based on teams then remove those players previous reservations
			if (!bContainsIncompatibleExistingMembers)
			{
				for (const FPlayerReservation& MemberReservation : ExistingPartyMemberReservations)
				{					
					State->RemovePlayer(MemberReservation.UniqueId);
				}
			}

			if (!bContainsIncompatibleExistingMembers)
			{
				if (State->DoesReservationFit(ReservationRequest))
				{
					bool bContinue = true;
					if (ValidatePlayers.IsBound())
					{
						bContinue = ValidatePlayers.Execute(ReservationRequest.PartyMembers);
					}

					if (bContinue)
					{
						if (!ShouldRespectCompetitiveIntegrity() || State->AreTeamsAvailable(ReservationRequest))
						{
							if (State->CrossPlayAllowed(ReservationRequest))
							{
								if (State->ValidateNewPartyReservation(ReservationRequest))
								{
									if (State->AddReservation(ReservationRequest))
									{
										// Keep track of newly added players
										for (const FPlayerReservation& PartyMember : ReservationRequest.PartyMembers)
										{
											NewPlayerAdded(PartyMember);
										}

										SendReservationUpdates();

										NotifyReservationEventNextFrame(ReservationChanged);
										if (State->IsBeaconFull())
										{
											NotifyReservationEventNextFrame(ReservationsFull);
										}
										Result = EPartyReservationResult::ReservationAccepted;
									}
									else
									{
										// Something wrong with team assignment
										Result = EPartyReservationResult::IncorrectPlayerCount;
									}
								}
								else
								{
									Result = EPartyReservationResult::ReservationDenied_ValidationFailed;
								}
							}
							else
							{
								// Cross play restriction
								Result = EPartyReservationResult::ReservationDenied_CrossPlayRestriction;
							}
						}
						else
						{
							// New reservation doesn't fit within a team allocation
							Result = EPartyReservationResult::PartyLimitReached;
						}
					}
					else
					{
						// Validate players failed above
						Result = EPartyReservationResult::ReservationDenied_Banned;
					}
				}
				else
				{
					// Reservation doesn't fit (party larger than team size, or not enough space in general)
					Result = EPartyReservationResult::PartyLimitReached;
				}
			}
			else
			{
				// Reservation contains players already accounted for
				// TODO:  This might be able to be smoothed out, main concern now is if we have players in other reservations on different teams or unassigned teams, and trying to keep party together
				Result = EPartyReservationResult::ReservationDenied_ContainsExistingPlayers;
			}
		}
	}
	else
	{
		// Invalid reservation
		Result = EPartyReservationResult::ReservationInvalid;
	}

	return Result;
}

EPartyReservationResult::Type APartyBeaconHost::UpdatePartyReservation(const FPartyReservation& ReservationUpdateRequest, bool bIsRemovingMembers)
{
	if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
	{
		UE_LOGF(LogPartyBeacon, Verbose, "APartyBeaconHost::UpdatePartyReservation");
		ReservationUpdateRequest.Dump();
	}
	EPartyReservationResult::Type Result = EPartyReservationResult::GeneralError;

	if (!State || GetBeaconState() == EBeaconState::DenyRequests)
	{
		return EPartyReservationResult::ReservationDenied;
	}

	if (bIsRemovingMembers && !State->bEnableRemovalRequests)
	{
		// We are told not to accept removal requests.
		return EPartyReservationResult::ReservationDenied;
	}

	if (ReservationUpdateRequest.IsValid(bIsValidationStrRequired) || bIsRemovingMembers)
	{
		if (bIsRemovingMembers)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Removing Member");
			int32 ExistingReservationIdx = State->GetExistingReservation(ReservationUpdateRequest.PartyLeader);
			if (ExistingReservationIdx != INDEX_NONE)
			{
				TArray<FPartyReservation>& Reservations = State->GetReservations();
				FPartyReservation* ExistingReservation = &Reservations[ExistingReservationIdx];

				// Read the list of players and remove the ones that are not in this party
				TArray<FPlayerReservation> PlayersToDelete;
				for (int32 PlayerIdx = 0; PlayerIdx < ReservationUpdateRequest.PartyMembers.Num(); PlayerIdx++)
				{
					const FPlayerReservation& MarkedPlayerRes = ReservationUpdateRequest.PartyMembers[PlayerIdx];

					const int32 FormerReservationIdx = State->GetExistingReservationContainingMember(MarkedPlayerRes.UniqueId);
					if (FormerReservationIdx != INDEX_NONE && FormerReservationIdx == ExistingReservationIdx)
					{
						PlayersToDelete.Add(MarkedPlayerRes);
					}
					else
					{
						// player is not in party
						UE_LOGF(LogPartyBeacon, Log, "Skipping player %ls because they are not in this party",
							*MarkedPlayerRes.UniqueId.ToString());
					}
				}

				// Copy new player entries into existing reservation
				for (int32 PlayerIdx = 0; PlayerIdx < PlayersToDelete.Num(); PlayerIdx++)
				{
					// If we're waiting for them to connect remove their reservation so they don't block other players from joining.
					FUniqueNetIdMatcher PlayerMatch(*PlayersToDelete[PlayerIdx].UniqueId);
					int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
					if (FoundIdx != INDEX_NONE)
					{
						const FPlayerReservation& PlayerRes = PlayersToDelete[PlayerIdx];
						ExistingReservation->RemoveAllPartyMembers(PlayerRes);

						PlayerRemoved(PlayerRes);
						State->SanityCheckReservations(true);
						State->NumConsumedReservations -= 1;
					}
					// Connected players should get their own new reservation.
					else
					{
						UpdatePartyLeader(PlayersToDelete[PlayerIdx].UniqueId, PlayersToDelete[PlayerIdx].UniqueId);

						// UpdatePartyLeader should modify the reservations array by adding a new reservation and potentially invalidating ExistingReservation's address. Sanity check the index as well.
						const int32 NewReservationIdx = State->GetExistingReservation(ReservationUpdateRequest.PartyLeader);
						if (NewReservationIdx != INDEX_NONE)
						{
							ExistingReservation = &Reservations[NewReservationIdx];

							if (NewReservationIdx != ExistingReservationIdx)
							{
								UE_LOGF(LogPartyBeacon, Verbose, "%ls - Call to UpdatePartyLeader resulted in reservation index of party leader %ls changing from %d to %d.", ANSI_TO_TCHAR(__FUNCTION__), *ReservationUpdateRequest.PartyLeader->ToString(), ExistingReservationIdx, NewReservationIdx);
								ExistingReservationIdx = NewReservationIdx;
							}
						}
						// Existing reservation was removed.
						else
						{
							UE_LOGF(LogPartyBeacon, Warning, "%ls - Call to UpdatePartyLeader resulted in removal of reservation with %ls as the party leader.", ANSI_TO_TCHAR(__FUNCTION__), *ReservationUpdateRequest.PartyLeader->ToString());
							break;
						}
					}
				}

				UE_LOGF(LogPartyBeacon, Verbose, "APartyBeaconHost::UpdatePartyReservation: Removed %d players, setting NumConsumedReservations to %d", PlayersToDelete.Num(), State->NumConsumedReservations);

				// Tell any UI and/or clients that there has been a change in the reservation state
				SendReservationUpdates();

				// Tell the owner that we've received a reservation so the UI can be updated
				NotifyReservationEventNextFrame(ReservationChanged);

				State->Reservations.RemoveAll([](const FPartyReservation& Reservation)
				{
					bool bEmptyReservation = Reservation.PartyMembers.Num() == 0;
					if (bEmptyReservation)
					{
						UE_LOGF(LogPartyBeacon, Log, "Removing reservation with party leader %ls because there are no more members in it",
							*Reservation.PartyLeader.ToString());
					}
					return bEmptyReservation;
				});
				State->SanityCheckReservations(false);
				
				Result = EPartyReservationResult::ReservationAccepted;

			}
			else
			{
				// Send a not found reservation response
				Result = EPartyReservationResult::ReservationNotFound;
			}
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Verbose, "Adding Member");			
			const int32 ExistingReservationIdx = State->GetExistingReservation(ReservationUpdateRequest.PartyLeader);
			if (ExistingReservationIdx != INDEX_NONE)
			{
				// Count the number of available slots for the existing reservation's team
				TArray<FPartyReservation>& Reservations = State->GetReservations();
				FPartyReservation& ExistingReservation = Reservations[ExistingReservationIdx];
				const int32 NumTeamMembers = GetNumPlayersOnTeam(ExistingReservation.TeamNum);
				const int32 NumAvailableSlotsOnTeam = FMath::Max<int32>(0, GetMaxPlayersPerTeam() - NumTeamMembers);
				int32 NumPlayersWithExistingReservation = 0;

				// Read the list of new players and remove the ones that have existing reservation entries
				TArray<FPlayerReservation> NewPlayers;
				for (int32 PlayerIdx = 0; PlayerIdx < ReservationUpdateRequest.PartyMembers.Num(); PlayerIdx++)
				{
					const FPlayerReservation& NewPlayerRes = ReservationUpdateRequest.PartyMembers[PlayerIdx];

					const int32 FormerReservationIdx = State->GetExistingReservationContainingMember(NewPlayerRes.UniqueId);
					if (FormerReservationIdx != ExistingReservationIdx)
					{
						// player reservation doesn't exist in this reservation so add it as a new player
						NewPlayers.Add(NewPlayerRes);

						if (FormerReservationIdx != INDEX_NONE)
						{
							++NumPlayersWithExistingReservation;
						}
					}
					else
					{
						// duplicate entry for this player
						UE_LOGF(LogPartyBeacon, Log, "Skipping player %ls because they already have a reservation with this party",
							*NewPlayerRes.UniqueId.ToString());
					}
				}
				// check to see if we have space to add new reservations for the new players 
				// Not using IsBeaconFull as we may not be adding a new player in the situation where a party player who has a reservation joins the game in which case NewPlayers.Num == 0
				if (State->DoesModifiedReservationFit(ExistingReservation, NewPlayers))
				{
					// Validate that adding the new party members to this reservation entry still fits within the team size
					if (!ShouldRespectCompetitiveIntegrity() || (NewPlayers.Num() - NumPlayersWithExistingReservation) <= NumAvailableSlotsOnTeam)
					{
						bool bPlayerRemovedFromReservation = false;
						if (NewPlayers.Num() > 0)
						{
							if (State->CrossPlayAllowed(ReservationUpdateRequest))
							{
								// Copy new player entries into existing reservation
								for (int32 PlayerIdx = 0; PlayerIdx < NewPlayers.Num(); PlayerIdx++)
								{
									const FPlayerReservation& PlayerRes = NewPlayers[PlayerIdx];

									// Remove players that existed in other reservations before adding to this reservation
									if (NumPlayersWithExistingReservation > 0)
									{
										const int32 FormerReservationIdx = State->GetExistingReservationContainingMember(PlayerRes.UniqueId);
										if (FormerReservationIdx != INDEX_NONE)
										{
											FPartyReservation& FormerReservation = Reservations[FormerReservationIdx];
											UE_LOGF(LogPartyBeacon, Log, "APartyBeaconHost::UpdatePartyReservation: Removing player %ls from former reservation with leader %ls before adding to reservation with leader %ls",
												*PlayerRes.UniqueId.ToString(), *FormerReservation.PartyLeader.ToString(), *ReservationUpdateRequest.PartyLeader.ToString());
											if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
											{
												FormerReservation.Dump();
											}
											int32 NumReservationsRemoved = FormerReservation.RemoveAllPartyMembers(PlayerRes);

											State->NumConsumedReservations -= NumReservationsRemoved;
											UE_LOGF(LogPartyBeacon, Verbose, "APartyBeaconHost::UpdatePartyReservation: Removed %d players, setting NumConsumedReservations to %d", NumReservationsRemoved, State->NumConsumedReservations);

											if (NumReservationsRemoved != 0)
											{
												bPlayerRemovedFromReservation = true;
												if (FormerReservation.PartyLeader == PlayerRes.UniqueId)
												{
													UE_LOGF(LogPartyBeacon, Display, "APartyBeaconHost::UpdatePartyReservation: Leader removed, finding member to promote");
													// Try to find a new leader for party reservation that lost its leader
													bool bAnyMemberPromoted = false;
													for (int32 FormerReservationPlayerIdx = 0; FormerReservationPlayerIdx < FormerReservation.PartyMembers.Num(); FormerReservationPlayerIdx++)
													{
														FPlayerReservation& FormerReservationPlayerEntry = FormerReservation.PartyMembers[FormerReservationPlayerIdx];
														if (FormerReservationPlayerEntry.UniqueId != FormerReservation.PartyLeader &&
															FormerReservationPlayerEntry.UniqueId.IsValid() &&
															State->GetExistingReservation(FormerReservationPlayerEntry.UniqueId) == INDEX_NONE)
														{
															// Promote to party leader (for now)
															UE_LOGF(LogPartyBeacon, Display, "APartyBeaconHost::UpdatePartyReservation: Promoting member %ls to leader", *FormerReservationPlayerEntry.UniqueId.ToDebugString());
															FormerReservation.PartyLeader = FormerReservationPlayerEntry.UniqueId;
															bAnyMemberPromoted = true;
															break;
														}
													}
													if (!bAnyMemberPromoted)
													{
														UE_LOGF(LogPartyBeacon, Display, "APartyBeaconHost::UpdatePartyReservation: Failed to find a player to promote to leader");
													}
													State->SanityCheckReservations(true);
												}
											}
										}

									}
									ExistingReservation.PartyMembers.Add(PlayerRes);
									// Keep track of newly added players
									NewPlayerAdded(PlayerRes);
									State->SanityCheckReservations(true);
								}

								// Update the reservation count before sending the response
								State->NumConsumedReservations += NewPlayers.Num();
								UE_LOGF(LogPartyBeacon, Verbose, "APartyBeaconHost::UpdatePartyReservation: Added %d players, setting NumConsumedReservations to %d", NewPlayers.Num(), State->NumConsumedReservations);

								// Tell any UI and/or clients that there has been a change in the reservation state
								SendReservationUpdates();

								// Tell the owner that we've received a reservation so the UI can be updated
								NotifyReservationEventNextFrame(ReservationChanged);
								if (State->IsBeaconFull())
								{
									// If we've hit our limit, fire the delegate so the host can do the
									// next step in getting parties together
									NotifyReservationEventNextFrame(ReservationsFull);
								}

								if (bPlayerRemovedFromReservation)
								{
									State->Reservations.RemoveAll([](const FPartyReservation& Reservation)
									{
										bool bEmptyReservation = Reservation.PartyMembers.Num() == 0;
										if (bEmptyReservation)
										{
											UE_LOGF(LogPartyBeacon, Log, "Removing reservation with party leader %ls because there are no more members in it",
												*Reservation.PartyLeader.ToString());
										}
										return bEmptyReservation;
									});
									State->SanityCheckReservations(false);
								}

								Result = EPartyReservationResult::ReservationAccepted;
							}
							else
							{
								Result = EPartyReservationResult::ReservationDenied_CrossPlayRestriction;
							}
						}
						else
						{
							// Duplicate entries (or zero) so existing reservation not updated
							Result = EPartyReservationResult::ReservationDuplicate;
						}
					}
					else
					{
						// Send an invalid party size response
						Result = EPartyReservationResult::IncorrectPlayerCount;
					}
				}
				else
				{
					// Send a session full response
					Result = EPartyReservationResult::PartyLimitReached;
				}
			}
			else
			{
				// Send a not found reservation response
				Result = EPartyReservationResult::ReservationNotFound;
			}

		}
	}
	else
	{
		// Invalid reservation
		Result = EPartyReservationResult::ReservationInvalid;
	}	

	return Result;
}

EPartyReservationResult::Type APartyBeaconHost::RemovePartyReservation(const FUniqueNetIdRepl& PartyLeader)
{
	if (State && State->RemoveReservation(PartyLeader))
	{
		CancelationReceived.ExecuteIfBound(*PartyLeader);

		SendReservationUpdates();
		NotifyReservationEventNextFrame(ReservationChanged);
		return EPartyReservationResult::ReservationRequestCanceled;
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "Failed to find reservation to cancel for leader %ls:", PartyLeader.IsValid() ? *PartyLeader->ToString() : TEXT("INVALID") );
		return EPartyReservationResult::ReservationNotFound;
	}
}

void APartyBeaconHost::RegisterAuthTicket(const FUniqueNetIdRepl& InPartyMemberId, const FString& InAuthTicket)
{
	if (State)
	{
		State->RegisterAuthTicket(InPartyMemberId, InAuthTicket);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, not able to register auth ticket.",
			*GetBeaconType());
	}
}

void APartyBeaconHost::UpdatePartyLeader(const FUniqueNetIdRepl& InPartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId)
{
	if (State)
	{
		State->UpdatePartyLeader(InPartyMemberId, NewPartyLeaderId);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, not able to update party leader.",
			*GetBeaconType());
	}
}

bool APartyBeaconHost::DoesSessionMatch(const FString& SessionId) const
{
	if (State)
	{
		UWorld* World = GetWorld();
		FName SessionName = State->GetSessionName();

		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
		FNamedOnlineSession* Session = SessionInt.IsValid() ? SessionInt->GetNamedSession(SessionName) : NULL;
		if (Session && Session->SessionInfo.IsValid() && !SessionId.IsEmpty() && Session->SessionInfo->GetSessionId().ToString() == SessionId)
		{
			return true;
		}
	}

	return false;
}

void APartyBeaconHost::ProcessReservationRequest(APartyBeaconClient* Client, const FString& SessionId, const FPartyReservation& ReservationRequest)
{
	UE_LOGF(LogPartyBeacon, Verbose, "ProcessReservationRequest %ls SessionId %ls PartyLeader: %ls PartySize: %d from (%ls)", 
		Client ? *Client->GetName() : TEXT("NULL"), 
		*SessionId,
		ReservationRequest.PartyLeader.IsValid() ? *ReservationRequest.PartyLeader->ToString() : TEXT("INVALID"),
		ReservationRequest.PartyMembers.Num(),
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));
	if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
	{
		ReservationRequest.Dump();
	}

	if (Client && Client->GetUniqueId().IsValid() && Client->GetUniqueId() == ReservationRequest.PartyLeader)
	{
		EPartyReservationResult::Type Result = EPartyReservationResult::BadSessionId;
		if (DoesSessionMatch(SessionId))
		{
			Result = AddPartyReservation(ReservationRequest);
		}

		UE_LOGF(LogPartyBeacon, Verbose, "ProcessReservationRequest result: %ls", EPartyReservationResult::ToString(Result));
		if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose) &&
			(Result != EPartyReservationResult::ReservationAccepted))
		{
			DumpReservations();
		}

		Client->ClientReservationResponse(Result);
	}
	else if (Client)
	{
		Client->ClientCancelReservationResponse(EPartyReservationResult::ReservationInvalid);
	}
}

void APartyBeaconHost::ProcessReservationUpdateRequest(APartyBeaconClient* Client, const FString& SessionId, const FPartyReservation& ReservationUpdateRequest, bool bIsRemovingMember)
{
	UE_LOGF(LogPartyBeacon, Verbose, "%ls - %ls SessionId %ls PartyLeader: %ls PartySize: %d from (%ls)",
		ANSI_TO_TCHAR(__FUNCTION__), 
		Client ? *Client->GetName() : TEXT("NULL"),
		*SessionId,
		ReservationUpdateRequest.PartyLeader.IsValid() ? *ReservationUpdateRequest.PartyLeader->ToString() : TEXT("INVALID"),
		ReservationUpdateRequest.PartyMembers.Num(),
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));

	if (Client && Client->GetUniqueId().IsValid() && Client->GetUniqueId() == ReservationUpdateRequest.PartyLeader)
	{
		const FPartyReservation* EffectiveReservationUpdateRequest = &ReservationUpdateRequest;
		FPartyReservation RemappedReservationUpdateRequest;
		if (BeaconConsoleVariables::CVarRemapReservationUpdateToExistingReservation.GetValueOnGameThread())
		{
			const int32 ExistingReservationIdx = State ? State->GetExistingReservationContainingMember(ReservationUpdateRequest.PartyLeader) : INDEX_NONE;
			const FPartyReservation* ExistingReservation = ExistingReservationIdx != INDEX_NONE ? &State->GetReservations()[ExistingReservationIdx] : nullptr;
			if (ExistingReservation)
			{
				if (ExistingReservation->PartyLeader != RemappedReservationUpdateRequest.PartyLeader)
				{
					UE_LOGF(LogPartyBeacon, Log, "Remapping reservation update request leader [%ls] to leader found in existing reservation [%ls]",
						*ExistingReservation->PartyLeader.ToDebugString(),
						*RemappedReservationUpdateRequest.PartyLeader.ToDebugString());
					RemappedReservationUpdateRequest = ReservationUpdateRequest;
					RemappedReservationUpdateRequest.PartyLeader = ExistingReservation->PartyLeader;
					EffectiveReservationUpdateRequest = &RemappedReservationUpdateRequest;
				}
			}
			else
			{
				// Invalid request, member has no existing reservation to update.
				// Continue with call to UpdatePartyReservation which will return an error.
			}
		}

		EPartyReservationResult::Type Result = EPartyReservationResult::BadSessionId;
		if (DoesSessionMatch(SessionId))
		{
			Result = UpdatePartyReservation(*EffectiveReservationUpdateRequest, bIsRemovingMember);
		}

		UE_LOGF(LogPartyBeacon, Verbose, "ProcessReservationUpdateRequest result: %ls", EPartyReservationResult::ToString(Result));
		if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose) &&
			(Result != EPartyReservationResult::ReservationAccepted))
		{
			DumpReservations();
			EffectiveReservationUpdateRequest->Dump();
		}

		Client->ClientReservationResponse(Result);
	}
	else if(Client)
	{
		Client->ClientCancelReservationResponse(EPartyReservationResult::ReservationInvalid);
	}
}

void APartyBeaconHost::ProcessReservationAddOrUpdateRequest(APartyBeaconClient* Client, const FString& SessionId, const FPartyReservation& ReservationRequest)
{
	UE_LOGF(LogPartyBeacon, Verbose, "ProcessReservationAddOrUpdateRequest %ls SessionId %ls PartyLeader: %ls PartySize: %d from (%ls)",
		Client ? *Client->GetName() : TEXT("NULL"),
		*SessionId,
		ReservationRequest.PartyLeader.IsValid() ? *ReservationRequest.PartyLeader->ToString() : TEXT("INVALID"),
		ReservationRequest.PartyMembers.Num(),
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));

	if (Client && Client->GetUniqueId().IsValid() && Client->GetUniqueId() == ReservationRequest.PartyLeader)
	{
		EPartyReservationResult::Type Result = EPartyReservationResult::BadSessionId;
		if (DoesSessionMatch(SessionId))
		{
			int32 ExistingReservationIdx = State->GetExistingReservation(ReservationRequest.PartyLeader);
			if (ExistingReservationIdx != INDEX_NONE)
			{
				Result = UpdatePartyReservation(ReservationRequest, false);
			}
			else
			{
				Result = AddPartyReservation(ReservationRequest);
			}
		}

		UE_LOGF(LogPartyBeacon, Verbose, "ProcessReservationUpdateRequest result: %ls", EPartyReservationResult::ToString(Result));
		if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose) &&
			(Result != EPartyReservationResult::ReservationAccepted))
		{
			DumpReservations();
			ReservationRequest.Dump();
		}

		Client->ClientReservationResponse(Result);
	}
	else if (Client)
	{
		Client->ClientCancelReservationResponse(EPartyReservationResult::ReservationInvalid);
	}
}

bool APartyBeaconHost::HasCrossplayOptOutReservation() const
{
	if (State)
	{
		return State->HasCrossplayOptOutReservation();
	}

	return false;
}

int32 APartyBeaconHost::GetReservationPlatformCount(const FString& InPlatform, bool bIncludeMappedPlatforms) const
{
	if (State)
	{
		return State->GetReservationPlatformCount(InPlatform, bIncludeMappedPlatforms);
	}

	return 0;
}

void APartyBeaconHost::ProcessCancelReservationRequest(APartyBeaconClient* Client, const FUniqueNetIdRepl& PartyLeader)
{
	UE_LOGF(LogPartyBeacon, Verbose, "ProcessCancelReservationRequest %ls PartyLeader: %ls from (%ls)", 
		Client ? *Client->GetName() : TEXT("NULL"), 
		PartyLeader.IsValid() ? *PartyLeader->ToString() : TEXT("INVALID"),
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));

	if (Client && Client->GetUniqueId().IsValid() && Client->GetUniqueId() == PartyLeader)
	{
		EPartyReservationResult::Type Result = RemovePartyReservation(PartyLeader);
		UE_LOGF(LogPartyBeacon, Verbose, "ProcessCancelReservationRequest result: %ls", EPartyReservationResult::ToString(Result));
		if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose) &&
			(Result != EPartyReservationResult::ReservationRequestCanceled))
		{
			DumpReservations();
		}

		Client->ClientCancelReservationResponse(Result);
	}
	else if (Client)
	{
		Client->ClientCancelReservationResponse(EPartyReservationResult::ReservationInvalid);
	}
}

void APartyBeaconHost::DumpReservations() const
{
	UE_LOGF(LogPartyBeacon, Display, "Debug info for Beacon: %ls", *GetBeaconType());
	if (State)
	{
		State->DumpReservations();
	}
	UE_LOGF(LogPartyBeacon, Display, "");
}

