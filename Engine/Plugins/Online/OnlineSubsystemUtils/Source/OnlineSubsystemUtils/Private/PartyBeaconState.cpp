// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartyBeaconState.h"
#include "OnlineSubsystemTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PartyBeaconState)

DEFINE_LOG_CATEGORY(LogPartyBeacon);

namespace ETeamAssignmentMethod
{
	const FName Smallest = FName(TEXT("Smallest"));
	const FName BestFit = FName(TEXT("BestFit"));
	const FName Random = FName(TEXT("Random"));
	const FName Manual = FName(TEXT("Manual"));
}

bool FPartyReservation::IsValid(bool bIsValidationStrRequired) const
{
	bool bIsValid = false;
	if (PartyLeader.IsValid() && PartyMembers.Num() >= 1)
	{
		bIsValid = true;
		for (const FPlayerReservation& PlayerRes : PartyMembers)
		{
			if (!PlayerRes.UniqueId.IsValid())
			{
				bIsValid = false;
				break;
			}

			if (PlayerRes.Platform.IsEmpty())
			{
				bIsValid = false;
				break;
			}

			if (bIsValidationStrRequired &&
				PartyLeader == PlayerRes.UniqueId &&
				PlayerRes.ValidationStr.IsEmpty())
			{
				bIsValid = false;
				break;
			}
		}
	}

	return bIsValid;
}

void FPartyReservation::Dump() const
{
	UE_LOGF(LogPartyBeacon, Display, "Party Reservation:");
	UE_LOGF(LogPartyBeacon, Display, "  TeamNum: %d", TeamNum);
	UE_LOGF(LogPartyBeacon, Display, "  PartyLeader: %ls", *PartyLeader.ToDebugString());
	UE_LOGF(LogPartyBeacon, Display, "  PartyMembers(%d):", PartyMembers.Num());
	
	int32 PartyMemberIdx = 0;
	for (const FPlayerReservation& PartyMember : PartyMembers)
	{
		UE_LOGF(LogPartyBeacon, Display, "    Member %d", PartyMemberIdx);
		++PartyMemberIdx;
		UE_LOGF(LogPartyBeacon, Display, "      UniqueId: %ls", *PartyMember.UniqueId.ToDebugString());
		UE_LOGF(LogPartyBeacon, Display, "		Crossplay: %ls", *LexToString(PartyMember.bAllowCrossplay));
#if UE_BUILD_SHIPPING
		UE_LOGF(LogPartyBeacon, Display, "      ValidationStr: %d bytes", PartyMember.ValidationStr.Len());
#else
		UE_LOGF(LogPartyBeacon, Display, "      ValidationStr: %ls", *PartyMember.ValidationStr);
#endif
		UE_LOGF(LogPartyBeacon, Display, "      ElapsedTime: %0.2f", PartyMember.ElapsedTime);
	}
}

void FPartyReservation::DumpUniqueIdsOnly() const
{
	UE_LOGF(LogPartyBeacon, Display, "Party Reservation UniqueIds:");
	
	int32 PartyMemberIdx = 0;
	for (const FPlayerReservation& PartyMember : PartyMembers)
	{
		UE_LOGF(LogPartyBeacon, Display, "  UniqueId: %ls", *PartyMember.UniqueId.ToDebugString());
	}
}

bool FPartyReservation::CanPlayerMigrateFromReservation(const FPartyReservation& Other) const
{
	return TeamNum == Other.TeamNum;
}

int32 FPartyReservation::RemoveAllPartyMembers(const FPlayerReservation& OtherRes)
{
	int32 NumRemoved = 0;
	for(int32 Idx = PartyMembers.Num() - 1; Idx >= 0; Idx--)
	{
		if (PartyMembers[Idx].UniqueId == OtherRes.UniqueId)
		{
			NumRemoved++;
			RemovePartyMemberAtIndex(Idx);
		}
	}

	return NumRemoved;
}

void FPartyReservation::RemovePartyMemberAtIndex(int32 Idx)
{
	if (-1 < Idx && Idx < PartyMembers.Num())
	{
		RemovedPartyMembers.Add(PartyMembers[Idx]);
		PartyMembers.RemoveAtSwap(Idx);
	}
}

UPartyBeaconState::UPartyBeaconState(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	SessionName(NAME_None),
	NumConsumedReservations(0),
	MaxReservations(0),
	NumTeams(0),
	NumPlayersPerTeam(0),
	TeamAssignmentMethod(ETeamAssignmentMethod::Smallest),
	ReservedHostTeamNum(0),
	ForceTeamNum(0),
	bRestrictCrossConsole(false),
	bRespectCompetitiveIntegrity(true)
{
}

bool UPartyBeaconState::InitState(int32 InTeamCount, int32 InTeamSize, int32 InMaxReservations, FName InSessionName, int32 InForceTeamNum, bool bInEnableRemovalRequests)
{
	if (InMaxReservations > 0)
	{
		SessionName = InSessionName;
		NumTeams = InTeamCount;
		NumPlayersPerTeam = InTeamSize;
		MaxReservations = InMaxReservations;
		ForceTeamNum = InForceTeamNum;
		Reservations.Empty(MaxReservations);
		bEnableRemovalRequests = bInEnableRemovalRequests;
		InitTeamArray();
		return true;
	}

	return false;
}

void UPartyBeaconState::InitTeamArray()
{
	if (NumTeams > 1)
	{
		// Grab one for the host team
		ReservedHostTeamNum = FMath::Rand() % NumTeams;
	}
	else
	{
		// Only one team, so choose 'forced team' for everything
		ReservedHostTeamNum = ForceTeamNum;
	}

	UE_LOGF(LogPartyBeacon, Display,
		"Beacon State: team count (%d), team size (%d), host team (%d)",
		NumTeams,
		NumPlayersPerTeam,
		ReservedHostTeamNum);
}

bool UPartyBeaconState::ReconfigureTeamAndPlayerCount(int32 InNumTeams, int32 InNumPlayersPerTeam, int32 InNumReservations)
{
	bool bSuccess = false;

	//Check total existing reservations against new total maximum
	if (NumConsumedReservations <= InNumReservations)
	{
		bool bTeamError = false;
		// Check teams with reservations against new team count
		if (NumTeams > InNumTeams)
		{
			// Any team about to be removed can't have players already there
			for (int32 TeamIdx = InNumTeams; TeamIdx < NumTeams; TeamIdx++)
			{
				if (GetNumPlayersOnTeam(TeamIdx) > 0)
				{
					bTeamError = true;
					UE_LOGF(LogPartyBeacon, Warning, "Beacon has players on a team about to be removed.");
				}
			}
		}

		bool bTeamSizeError = false;
		// Check num players per team against new team size
		if (NumPlayersPerTeam > InNumPlayersPerTeam)
		{
			for (int32 TeamIdx = 0; TeamIdx<NumTeams; TeamIdx++)
			{
				if (GetNumPlayersOnTeam(TeamIdx) > InNumPlayersPerTeam)
				{
					bTeamSizeError = true;
					UE_LOGF(LogPartyBeacon, Warning, "Beacon has too many players on a team about to be resized.");
				}
			}
		}

		if (!bTeamError && !bTeamSizeError)
		{
			NumTeams = InNumTeams;
			NumPlayersPerTeam = InNumPlayersPerTeam;
			MaxReservations = InNumReservations;

			InitTeamArray();
			bSuccess = true;

			UE_LOGF(LogPartyBeacon, Display,
				"Reconfiguring to team count (%d), team size (%d)",
				NumTeams,
				NumPlayersPerTeam);
		}
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "Beacon has too many consumed reservations for this reconfiguration, ignoring request.");
	}

	return bSuccess;
}

void UPartyBeaconState::SetCompetitiveIntegrity(bool bNewCompetitiveIntegrity)
{
	bRespectCompetitiveIntegrity = bNewCompetitiveIntegrity;
}

int32 UPartyBeaconState::GetMaxAvailableTeamSize() const
{
	int32 MaxFreeSlots = 0;
	// find the largest available free slots within all the teams
	for (int32 TeamIdx = 0; TeamIdx < NumTeams; TeamIdx++)
	{
		MaxFreeSlots = FMath::Max<int32>(MaxFreeSlots, NumPlayersPerTeam - GetNumPlayersOnTeam(TeamIdx));
	}
	return MaxFreeSlots;
}

int32 UPartyBeaconState::GetNumPlayersOnTeam(int32 TeamIdx) const
{
	int32 Result = 0;
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FPartyReservation& Reservation = Reservations[ResIdx];
		if (Reservation.TeamNum == TeamIdx)
		{
			for (int32 PlayerIdx = 0; PlayerIdx < Reservation.PartyMembers.Num(); PlayerIdx++)
			{
				const FPlayerReservation& PlayerEntry = Reservation.PartyMembers[PlayerIdx];
				// only count valid player net ids
				if (PlayerEntry.UniqueId.IsValid())
				{
					// count party members in each team (includes party leader)
					Result++;
				}
			}
		}
	}
	return Result;
}

int32 UPartyBeaconState::GetTeamForCurrentPlayer(const FUniqueNetId& PlayerId) const
{
	int32 TeamNum = INDEX_NONE;
	if (PlayerId.IsValid())
	{
		for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
		{
			const FPartyReservation& Reservation = Reservations[ResIdx];
			for (int32 PlayerIdx = 0; PlayerIdx < Reservation.PartyMembers.Num(); PlayerIdx++)
			{
				// find the player id in the existing list of reservations
				if (*Reservation.PartyMembers[PlayerIdx].UniqueId == PlayerId)
				{
					TeamNum = Reservation.TeamNum;
					break;
				}
			}
		}

		UE_LOGF(LogPartyBeacon, VeryVerbose, "Assigning player %ls to team %d",
			*PlayerId.ToString(),
			TeamNum);
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Display, "Invalid player when attempting to find team assignment");
	}

	return TeamNum;
}

int32 UPartyBeaconState::GetReservationTeamCount() const
{
	TSet<int32> Teams; 
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FPartyReservation& Reservation = Reservations[ResIdx];
		if (Reservation.TeamNum != INDEX_NONE)
		{
			Teams.Add(Reservation.TeamNum);
		}
	}

	return Teams.Num();
}

int32 UPartyBeaconState::GetPlayersOnTeam(int32 TeamIndex, TArray<FUniqueNetIdRepl>& TeamMembers) const
{
	TeamMembers.Empty(NumPlayersPerTeam);
	if (TeamIndex < GetNumTeams())
	{
		for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
		{
			const FPartyReservation& Reservation = Reservations[ResIdx];
			if (Reservation.TeamNum == TeamIndex)
			{
				for (int32 PlayerIdx = 0; PlayerIdx < Reservation.PartyMembers.Num(); PlayerIdx++)
				{
					TeamMembers.Add(Reservation.PartyMembers[PlayerIdx].UniqueId);
				}
			}
		}

		return TeamMembers.Num();
	}
	else
	{
		UE_LOGF(LogPartyBeacon, Warning, "GetPlayersOnTeam: Invalid team index %d", TeamIndex);
	}
	
	return 0;
}

void UPartyBeaconState::SetTeamAssignmentMethod(FName NewAssignmentMethod)
{
	TeamAssignmentMethod = NewAssignmentMethod;
}

/**
* Helper for sorting team sizes
*/
struct FTeamBalanceInfo
{
	/** Index of team */
	int32 TeamIdx;
	/** Current size of team */
	int32 TeamSize;

	FTeamBalanceInfo(int32 InTeamIdx, int32 InTeamSize)
		: TeamIdx(InTeamIdx),
		TeamSize(InTeamSize)
	{}
};

/**
 * Sort teams by size (equal teams are randomly mixed)
 */
struct FSortTeamSizeSmallestToLargest
{
	bool operator()(const FTeamBalanceInfo& A, const FTeamBalanceInfo& B) const
	{
		if (A.TeamSize == B.TeamSize)
		{
			return (FMath::Rand() % 2) ? true : false;
		}
		else
		{
			return A.TeamSize < B.TeamSize;
		}
	}
};

int32 UPartyBeaconState::GetTeamAssignment(const FPartyReservation& Party)
{
	if (NumTeams > 1)
	{
		if (TeamAssignmentMethod == ETeamAssignmentMethod::Manual)
		{
			const int32 CurrentPlayersOnTeam = GetNumPlayersOnTeam(Party.TeamNum);
			if ((CurrentPlayersOnTeam + Party.PartyMembers.Num()) <= NumPlayersPerTeam)
			{
				// It is expected that the caller making the reservation has manually assigned the team number
				return Party.TeamNum;
			}
			else
			{
				UE_LOGF(LogPartyBeacon, Warning, "UPartyBeaconHost::GetTeamAssignment: manually assigned team %d has no room.", Party.TeamNum);
				return INDEX_NONE;
			}
		}
		else
		{
			TArray<FTeamBalanceInfo> PotentialTeamChoices;
			for (int32 TeamIdx = 0; TeamIdx < NumTeams; TeamIdx++)
			{
				const int32 CurrentPlayersOnTeam = GetNumPlayersOnTeam(TeamIdx);
				if ((CurrentPlayersOnTeam + Party.PartyMembers.Num()) <= NumPlayersPerTeam)
				{
					new (PotentialTeamChoices)FTeamBalanceInfo(TeamIdx, CurrentPlayersOnTeam);
				}
			}

			// Grab one from our list of choices
			if (PotentialTeamChoices.Num() > 0)
			{
				if (TeamAssignmentMethod == ETeamAssignmentMethod::Smallest)
				{
					PotentialTeamChoices.Sort(FSortTeamSizeSmallestToLargest());
					return PotentialTeamChoices[0].TeamIdx;
				}
				else if (TeamAssignmentMethod == ETeamAssignmentMethod::BestFit)
				{
					PotentialTeamChoices.Sort(FSortTeamSizeSmallestToLargest());
					return PotentialTeamChoices[PotentialTeamChoices.Num() - 1].TeamIdx;
				}
				else if (TeamAssignmentMethod == ETeamAssignmentMethod::Random)
				{
					int32 TeamIndex = FMath::Rand() % PotentialTeamChoices.Num();
					return PotentialTeamChoices[TeamIndex].TeamIdx;
				}
			}
			else
			{
				UE_LOGF(LogPartyBeacon, Warning, "UPartyBeaconHost::GetTeamAssignment: couldn't find an open team for party members.");
				return INDEX_NONE;
			}
		}
	}

	return ForceTeamNum;
}

void UPartyBeaconState::BestFitTeamAssignmentJiggle()
{
	if (TeamAssignmentMethod == ETeamAssignmentMethod::BestFit &&
		NumTeams > 1)
	{
		UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::BestFitTeamAssignmentJiggle NumTeams=%d", NumTeams);
		TArray<FPartyReservation*> ReservationsToJiggle;
		ReservationsToJiggle.Reserve(Reservations.Num());
		for (FPartyReservation& Reservation : Reservations)
		{
			// Only want to rejiggle reservations with existing team assignments (new reservations will still stay at -1)
			if (Reservation.TeamNum != -1)
			{
				// Remove existing team assignments so new assignments can be given
				Reservation.TeamNum = -1;
				// Add to list of reservations that need new assignments
				ReservationsToJiggle.Add(&Reservation);
			}
		}
		// Sort so that largest party reservations come first
		ReservationsToJiggle.Sort([](const FPartyReservation& A, const FPartyReservation& B)
			{
				return B.PartyMembers.Num() < A.PartyMembers.Num();
			}
		);

		// Re-add these reservations with best fit team assignments
		for (FPartyReservation* Reservation : ReservationsToJiggle)
		{
			Reservation->TeamNum = GetTeamAssignment(*Reservation);
			if (Reservation->TeamNum == -1)
			{
				UE_LOGF(LogPartyBeacon, Warning, "UPartyBeaconHost::BestFitTeamAssignmentJiggle: could not reassign to a team!");
			}
		}
		SanityCheckReservations(true);
	}
}

bool UPartyBeaconState::AreTeamsAvailable(const FPartyReservation& ReservationRequest) const
{
	int32 IncomingPartySize = ReservationRequest.PartyMembers.Num();
	for (int32 TeamIdx = 0; TeamIdx < NumTeams; TeamIdx++)
	{
		const int32 CurrentPlayersOnTeam = GetNumPlayersOnTeam(TeamIdx);
		if ((CurrentPlayersOnTeam + IncomingPartySize) <= NumPlayersPerTeam)
		{
			return true;
		}
	}
	return false;
}

bool UPartyBeaconState::HasCrossplayOptOutReservation() const
{
	for (const FPartyReservation& ExistingReservation : Reservations)
	{
		for (const FPlayerReservation& ExistingPlayer : ExistingReservation.PartyMembers)
		{
			if (!ExistingPlayer.bAllowCrossplay)
			{
				return true;
			}
		}
	}

	return false;
}

int32 UPartyBeaconState::GetReservationPlatformCount(const FString& InPlatform, bool bIncludeMappedPlatforms) const
{
	int32 PlayerCount = 0;
	for (const FPartyReservation& ExistingReservation : Reservations)
	{
		for (const FPlayerReservation& ExistingPlayer : ExistingReservation.PartyMembers)
		{
			if (ExistingPlayer.Platform == InPlatform)
			{
				PlayerCount++;
			}
			else if (bIncludeMappedPlatforms)
			{
				const FPartyBeaconCrossplayPlatformMapping* const PlatformType = PlatformTypeMapping.FindByKey(ExistingPlayer.Platform);
				if (PlatformType && PlatformType->PlatformType == InPlatform)
				{
					PlayerCount++;
				}
			}
		}
	}

	return PlayerCount;
}

bool UPartyBeaconState::CrossPlayAllowed(const FPartyReservation& ReservationRequest) const
{
	bool bCrossplayAllowed = true;

	bool bEveryoneAllowsCrossplay = true;
	TSet<FString> ExistingPlatforms;
	for (const FPartyReservation& ExistingReservation : Reservations)
	{
		for (const FPlayerReservation& ExistingPlayer : ExistingReservation.PartyMembers)
		{
			bEveryoneAllowsCrossplay &= ExistingPlayer.bAllowCrossplay;
			const FPartyBeaconCrossplayPlatformMapping* const PlatformType = PlatformTypeMapping.FindByKey(ExistingPlayer.Platform);
			ExistingPlatforms.Add(PlatformType ? PlatformType->PlatformType : OSS_PLATFORM_NAME_OTHER);
		}
	}

	if (ExistingPlatforms.Num() > 0)
	{
		bool bPartyAllowsCrossplay = true;
		TSet<FString> PartyPlatforms;
		for (const FPlayerReservation& Player : ReservationRequest.PartyMembers)
		{
			bPartyAllowsCrossplay &= Player.bAllowCrossplay;
			const FPartyBeaconCrossplayPlatformMapping* const PlatformType = PlatformTypeMapping.FindByKey(Player.Platform);
			PartyPlatforms.Add(PlatformType ? PlatformType->PlatformType : OSS_PLATFORM_NAME_OTHER);
		}

		bool bHasIncompatibleCrossplay = false;
		TMap<FString, FString> PlatformCrossplayRestrictionsMap;
		PlatformCrossplayRestrictionsMap.Reserve(PlatformCrossplayRestrictions.Num());
		for (const FString& PlatformCrossplayRestriction : PlatformCrossplayRestrictions)
		{
			FString PlatformWithRestriction;
			FString RestrictedPlatformsAgainst;
			if (PlatformCrossplayRestriction.Split(TEXT("="), &PlatformWithRestriction, &RestrictedPlatformsAgainst))
			{
				PlatformCrossplayRestrictionsMap.Emplace(MoveTemp(PlatformWithRestriction), MoveTemp(RestrictedPlatformsAgainst));
			}
		}
		for (const TPair<FString, FString>& PlatformCrossplayRestriction : PlatformCrossplayRestrictionsMap)
		{
			if (ExistingPlatforms.Contains(PlatformCrossplayRestriction.Key))
			{
				for (const FString& PartyPlatform : PartyPlatforms)
				{
					if (PlatformCrossplayRestriction.Value.Contains(PartyPlatform))
					{
						bHasIncompatibleCrossplay = true;
						break;
					}
				}
			}
			else if (PartyPlatforms.Contains(PlatformCrossplayRestriction.Key))
			{
				for (const FString& ExistingPlatform : ExistingPlatforms)
				{
					if (PlatformCrossplayRestriction.Value.Contains(ExistingPlatform))
					{
						bHasIncompatibleCrossplay = true;
						break;
					}
				}
			}
			if (bHasIncompatibleCrossplay)
			{
				break;
			}
		}

		TSet<FString> Delta = PartyPlatforms.Intersect(ExistingPlatforms);

		// The intersection of party/existing will be less if something new is added
		const bool bPartyAddsNewPlatform = (Delta.Num() != PartyPlatforms.Num());
		// The incoming party platform makeup, if anyone in that party doesn't allow crossplay, must be a super set of the platforms already present
		// Otherwise there is a foreign existing platform that the incoming party wouldn't allow 
		// (ABC) joining and existing (BC) is ok, but (ABC) joining and existing (BD) is not
		const bool bNewPartyIsStrictSuperset = (Delta.Num() == ExistingPlatforms.Num());

		// Don't cross mingle consoles if restriction is set
		const bool bCrossConsoleAllowed = (!bHasIncompatibleCrossplay) || (bHasIncompatibleCrossplay && !bRestrictCrossConsole);
		// All the existing players must be ok with cross play for the new party to join or incoming party is subset of existing platforms
		const bool bExistingPlayersOk = (!bPartyAddsNewPlatform || (bPartyAddsNewPlatform && bEveryoneAllowsCrossplay));
		// All the incoming players must be ok with cross play for the new party to join or existing platforms are subset of incoming party
		const bool bIncomingPlayersOk = (bPartyAllowsCrossplay || bNewPartyIsStrictSuperset);

		bCrossplayAllowed = bCrossConsoleAllowed && bExistingPlayersOk && bIncomingPlayersOk;

		FString ExistingStr;
		for (const FString& Existing : ExistingPlatforms)
		{
			ExistingStr += Existing + TEXT("|");
		}
		UE_LOGF(LogPartyBeacon, Verbose, "Existing: %ls", *ExistingStr);

		FString PartyPlatformStr;
		for (const FString& PartyPlatform : PartyPlatforms)
		{
			PartyPlatformStr += PartyPlatform + TEXT("|");
		}
		UE_LOGF(LogPartyBeacon, Verbose, "NewParty: %ls", *PartyPlatformStr);

		UE_LOGF(LogPartyBeacon, Log, "UPartyBeaconState::CrossPlayAllowed bEveryoneAllowsCrossplay:%ls bPartyAllowsCrossplay:%ls bCrossConsoleAllowed:%ls bExistingPlayersOk:%ls bIncomingPlayersOk:%ls bCrossPlayAllowed:%ls",
			*LexToString(bEveryoneAllowsCrossplay),
			*LexToString(bPartyAllowsCrossplay),
			*LexToString(bCrossConsoleAllowed),
			*LexToString(bExistingPlayersOk),
			*LexToString(bIncomingPlayersOk),
			*LexToString(bCrossplayAllowed));
	}

	return bCrossplayAllowed;
}

bool UPartyBeaconState::DoesReservationFit(const FPartyReservation& ReservationRequest) const
{
	const int32 IncomingPartySize = ReservationRequest.PartyMembers.Num();
	bool bPartySizeOk = true;
	if (bRespectCompetitiveIntegrity)
	{
		bPartySizeOk = (IncomingPartySize > 0) && (IncomingPartySize <= NumPlayersPerTeam);
	}
	const bool bRoomForReservation = (NumConsumedReservations + IncomingPartySize ) <= MaxReservations;

	UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::DoesReservationFit: Incoming Party Size: %d Num Players Per Team: %d NumConsumedReservations: %d MaxReservations: %d", IncomingPartySize, NumPlayersPerTeam, NumConsumedReservations, MaxReservations);

	return bPartySizeOk && bRoomForReservation;
}

bool UPartyBeaconState::DoesModifiedReservationFit(const FPartyReservation& ExistingReservation, const TArray<FPlayerReservation>& NewPlayers) const
{
	return GetRemainingReservations() - NewPlayers.Num() >= 0;
}

bool UPartyBeaconState::AddReservation(const FPartyReservation& ReservationRequest)
{
	if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
	{
		UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::AddReservation");
		ReservationRequest.DumpUniqueIdsOnly();
	}
	int32 TeamAssignment = GetTeamAssignment(ReservationRequest);
	if (TeamAssignment != INDEX_NONE)
	{
		int32 IncomingPartySize = ReservationRequest.PartyMembers.Num();

		NumConsumedReservations += IncomingPartySize;
		UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::AddReservation: Setting NumConsumedReservations to %d", NumConsumedReservations);
		int32 ResIdx = Reservations.Add(ReservationRequest);
		Reservations[ResIdx].TeamNum = TeamAssignment;
		SanityCheckReservations(false);

		// Possibly shuffle existing teams so that beacon can accommodate biggest open slots
		BestFitTeamAssignmentJiggle();
	}

	return TeamAssignment != INDEX_NONE;
}

bool UPartyBeaconState::RemoveReservation(const FUniqueNetIdRepl& PartyLeader)
{
	const int32 ExistingReservationIdx = GetExistingReservation(PartyLeader);
	if (ExistingReservationIdx != INDEX_NONE)
	{
		NumConsumedReservations -= Reservations[ExistingReservationIdx].PartyMembers.Num();
		UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::RemoveReservation: %ls, setting NumConsumedReservations to %d", *PartyLeader.ToString(), NumConsumedReservations);
		
		for (const FPlayerReservation& PlayerRes : Reservations[ExistingReservationIdx].PartyMembers)
		{
			FUniqueNetIdMatcher PlayerMatch(*PlayerRes.UniqueId);
			int32 FoundIdx = PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
			if (FoundIdx != INDEX_NONE)
			{
				PlayersPendingJoin.RemoveAtSwap(FoundIdx);
			}
		}

		Reservations.RemoveAtSwap(ExistingReservationIdx);

		SanityCheckReservations(false);

		// Possibly shuffle existing teams so that beacon can accommodate biggest open slots
		BestFitTeamAssignmentJiggle();
		return true;
	}

	return false;
}

void UPartyBeaconState::RegisterAuthTicket(const FUniqueNetIdRepl& InPartyMemberId, const FString& InAuthTicket)
{
	if (InPartyMemberId.IsValid() && !InAuthTicket.IsEmpty())
	{
		bool bFoundReservation = false;

		for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bFoundReservation; ResIdx++)
		{
			FPartyReservation& ReservationEntry = Reservations[ResIdx];

			FPlayerReservation* PlayerRes = ReservationEntry.PartyMembers.FindByPredicate(
				[InPartyMemberId](const FPlayerReservation& ExistingPlayerRes)
			{
				return InPartyMemberId == ExistingPlayerRes.UniqueId;
			});

			if (PlayerRes)
			{
				if (PlayerRes->ValidationStr.IsEmpty())
				{
					UE_LOGF(LogPartyBeacon, VeryVerbose, "Setting auth ticket for member %ls.", *InPartyMemberId.ToDebugString());
				}
				else if (PlayerRes->ValidationStr != InAuthTicket)
				{
					UE_LOGF(LogPartyBeacon, Warning, "Auth ticket changing for member %ls.", *InPartyMemberId.ToDebugString());
				}

				PlayerRes->ValidationStr = InAuthTicket;
				bFoundReservation = true;
				break;
			}
		}

		if (!bFoundReservation)
		{
			UE_LOGF(LogPartyBeacon, Warning, "Found no reservation for player %ls, while registering auth ticket.", *InPartyMemberId.ToDebugString());
		}
	}
}

void UPartyBeaconState::UpdatePartyLeader(const FUniqueNetIdRepl& InPartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId)
{
	if (InPartyMemberId.IsValid() && NewPartyLeaderId.IsValid())
	{
		int32 PartyMemberReservationIdx = GetExistingReservationContainingMember(InPartyMemberId);
		// PartyMemberReservationIdx may be invalid if we had just started kicking this player
		if (PartyMemberReservationIdx != INDEX_NONE)
		{
			int32 PartLeaderAsMemberReservationIdx = GetExistingReservationContainingMember(NewPartyLeaderId);	
			int32 NewPartyLeaderReservationIdx = GetExistingReservation(NewPartyLeaderId);
			bool Promotion = false;
			// if the leader we want to be under is in the same party as us, and we are the current leader then promote them to the leader of the existing party. Excluding cases where we are asking to be out own leader
			if (PartyMemberReservationIdx == PartLeaderAsMemberReservationIdx && (InPartyMemberId != NewPartyLeaderId))
			{
				FPartyReservation* SharedReservation = &Reservations[PartLeaderAsMemberReservationIdx];
				if (SharedReservation->PartyLeader == InPartyMemberId)
				{
					SharedReservation->PartyLeader = NewPartyLeaderId;
					Promotion = true;
				}								
			}
			if (Promotion == false)
			{
				// Get the reservation that has NewPartyLeaderId as the leader
				// May be INDEX_NONE if the target is not the leader of a reservation and is instead just a member
				if (NewPartyLeaderReservationIdx != PartyMemberReservationIdx)
				{
					FPartyReservation* PriorReservation = &Reservations[PartyMemberReservationIdx];
					FPartyReservation* DestinationReservation = (NewPartyLeaderReservationIdx != INDEX_NONE) ? &Reservations[NewPartyLeaderReservationIdx] : nullptr;

					// Verify that a migration between existing reservations can occur
					if (!DestinationReservation || DestinationReservation->CanPlayerMigrateFromReservation(*PriorReservation))
					{
						// Make a copy of their player reservation so we can move it to the new reservation
						int32 PriorPlayerReservationIdx = PriorReservation->PartyMembers.IndexOfByPredicate([&InPartyMemberId](const FPlayerReservation& ExistingPlayerRes)
						{
							return InPartyMemberId == ExistingPlayerRes.UniqueId;
						});
						if (ensure(PriorPlayerReservationIdx != INDEX_NONE))
						{
							// Make a copy of the player reservation to be inserted into a new reservation after being removed from the prior reservation
							FPlayerReservation PlayerReservation = PriorReservation->PartyMembers[PriorPlayerReservationIdx];

							// Remove player from their previous reservation and find a new place for them
							PriorReservation->RemovePartyMemberAtIndex(PriorPlayerReservationIdx);

							// If there is already a reservation that has the new party leader as a leader, join it
							// If not, create one
							if (DestinationReservation)
							{
								UE_LOGF(LogPartyBeacon, Verbose, "UpdatePartyLeader:  Moving player %ls from reservation under party leader %ls, to reservation under party leader %ls",
									*InPartyMemberId.ToString(), *PriorReservation->PartyLeader.ToString(), *NewPartyLeaderId.ToString());
								DestinationReservation->PartyMembers.Add(MoveTemp(PlayerReservation));
							}
							else
							{
								UE_LOGF(LogPartyBeacon, Verbose, "UpdatePartyLeader:  Moving player %ls from reservation under party leader %ls, to new reservation with leader %ls",
									*InPartyMemberId.ToString(), *PriorReservation->PartyLeader.ToString(), *NewPartyLeaderId.ToString());

								{
									FPartyReservation NewReservation;
									NewReservation.TeamNum = PriorReservation->TeamNum;
									NewReservation.PartyLeader = NewPartyLeaderId;
									NewReservation.PartyMembers.Add(MoveTemp(PlayerReservation));

									Reservations.Add(MoveTemp(NewReservation));
								}
								// While the index doesn't change, the Add() call above could invalidate our memory address
								PriorReservation = &Reservations[PartyMemberReservationIdx];
							}

							// If the former reservation is now empty, remove the reservation
							if (PriorReservation->PartyMembers.Num() == 0)
							{
								UE_LOGF(LogPartyBeacon, Verbose, "UpdatePartyLeader:  Removing now empty reservation that had party leader %ls", *PriorReservation->PartyLeader.ToString());
								Reservations.RemoveAtSwap(PartyMemberReservationIdx);
								PriorReservation = nullptr;
							}

							// At this point both PriorReservation and DestinationReservation are possibly invalid, don't use them below here

							SanityCheckReservations(false);
						}
						else
						{
							UE_LOGF(LogPartyBeacon, Warning, "UpdatePartyLeader:  Member %ls not found in their own reservation!", *InPartyMemberId.ToDebugString());
						}
					}
					else
					{
						UE_LOGF(LogPartyBeacon, Warning, "UpdatePartyLeader:  Unable to migrate player %ls from reservation under leader %ls to existing reservation with leader %ls",
							*InPartyMemberId.ToString(), *PriorReservation->PartyLeader.ToString(), *NewPartyLeaderId.ToString());
					}
				}
				else
				{
					UE_LOGF(LogPartyBeacon, VeryVerbose, "UpdatePartyLeader:  Player %ls already under party leader %ls, no action taken",
						*InPartyMemberId.ToString(), *NewPartyLeaderId.ToString());
				}
			}
		}
		else
		{
			UE_LOGF(LogPartyBeacon, Warning, "UpdatePartyLeader:  No reservation found for player %ls!", *InPartyMemberId.ToDebugString());
		}
	}
}

bool UPartyBeaconState::SwapTeams(const FUniqueNetIdRepl& PartyLeader, const FUniqueNetIdRepl& OtherPartyLeader)
{
	bool bSuccess = false;

	const int32 ResIdx = GetExistingReservation(PartyLeader);
	const int32 OtherResIdx = GetExistingReservation(OtherPartyLeader);

	if (ResIdx != INDEX_NONE && OtherResIdx != INDEX_NONE)
	{
		FPartyReservation& PartyRes = Reservations[ResIdx];
		FPartyReservation& OtherPartyRes = Reservations[OtherResIdx];
		if (PartyRes.TeamNum != OtherPartyRes.TeamNum)
		{
			int32 TeamSize = GetNumPlayersOnTeam(PartyRes.TeamNum);
			int32 OtherTeamSize = GetNumPlayersOnTeam(OtherPartyRes.TeamNum);

			// Will the new teams fit
			bool bValidTeamSizeA = (PartyRes.PartyMembers.Num() + (OtherTeamSize - OtherPartyRes.PartyMembers.Num())) <= NumPlayersPerTeam;
			bool bValidTeamSizeB = (OtherPartyRes.PartyMembers.Num() + (TeamSize - PartyRes.PartyMembers.Num())) <= NumPlayersPerTeam;

			if (bValidTeamSizeA && bValidTeamSizeB)
			{
				if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
				{
					UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::SwapTeams: %ls %ls", *PartyLeader.ToString(), *OtherPartyLeader.ToString());
					PartyRes.Dump();
					OtherPartyRes.Dump();
				}
				Swap(PartyRes.TeamNum, OtherPartyRes.TeamNum);
				SanityCheckReservations(false);
				bSuccess = true;
			}
		}
	}

	return bSuccess;
}

bool UPartyBeaconState::ChangeTeam(const FUniqueNetIdRepl& PartyLeader, int32 NewTeamNum)
{
	bool bSuccess = false;

	if (NewTeamNum >= 0 && NewTeamNum < NumTeams)
	{
		int32 ResIdx = GetExistingReservation(PartyLeader);
		if (ResIdx != INDEX_NONE)
		{
			FPartyReservation& PartyRes = Reservations[ResIdx];
			if (PartyRes.TeamNum != NewTeamNum)
			{
				int32 OtherTeamSize = GetNumPlayersOnTeam(NewTeamNum);
				bool bValidTeamSize = (PartyRes.PartyMembers.Num() + OtherTeamSize) <= NumPlayersPerTeam;

				if (bValidTeamSize)
				{
					PartyRes.TeamNum = NewTeamNum;
					bSuccess = true;
				}
			}
		}
	}

	return bSuccess;
}

bool UPartyBeaconState::RemovePlayer(const FUniqueNetIdRepl& PlayerId)
{
	UE_LOGF(LogPartyBeacon, VeryVerbose, "UPartyBeaconState::RemovePlayer: %ls", *PlayerId.ToDebugString());
	bool bWasRemoved = false;

	for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bWasRemoved; ResIdx++)
	{
		FPartyReservation& Reservation = Reservations[ResIdx];

		// Try to find a new leader for party reservation that lost its leader
		if ((Reservation.PartyLeader == PlayerId) && (Reservation.PartyMembers.Num() > 1))
		{
			UE_LOGF(LogPartyBeacon, Display, "UPartyBeaconState::RemovePlayer: Party leader %ls has left the party, %d members in reservation", *PlayerId.ToString(), Reservation.PartyMembers.Num());
			if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
			{
				Reservation.Dump();
			}

			bool bAnyMemberPromoted = false;
			for (int32 PlayerIdx = 0; PlayerIdx < Reservation.PartyMembers.Num(); PlayerIdx++)
			{
				FPlayerReservation& PlayerEntry = Reservation.PartyMembers[PlayerIdx];
				if (PlayerEntry.UniqueId != Reservation.PartyLeader &&
					PlayerEntry.UniqueId.IsValid() &&
					GetExistingReservation(PlayerEntry.UniqueId) == INDEX_NONE)
				{
					// Promote to party leader (for now)
					UE_LOGF(LogPartyBeacon, Display, "UPartyBeaconState::RemovePlayer: Promoting member %ls to leader", *PlayerEntry.UniqueId.ToDebugString());
					Reservation.PartyLeader = PlayerEntry.UniqueId;
					bAnyMemberPromoted = true;
					break;
				}
			}
			if (!bAnyMemberPromoted)
			{
				UE_LOGF(LogPartyBeacon, Display, "UPartyBeaconState::RemovePlayer: Failed to find a player to promote to leader");
			}

			SanityCheckReservations(true);
		}

		// find the player in an existing reservation slot
		for (int32 PlayerIdx = 0; PlayerIdx < Reservation.PartyMembers.Num(); PlayerIdx++)
		{
			FPlayerReservation& PlayerEntry = Reservation.PartyMembers[PlayerIdx];
			if (PlayerEntry.UniqueId == PlayerId)
			{
				// player removed
				Reservation.RemovePartyMemberAtIndex(PlayerIdx--);
				bWasRemoved = true;

				// free up a consumed entry
				NumConsumedReservations--;
				if (UE_LOG_ACTIVE(LogPartyBeacon, Verbose))
				{
					UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::RemovePlayer: Player found in reservation with leader %ls, setting NumConsumedReservations to %d", *Reservation.PartyLeader.ToString(), NumConsumedReservations);
					Reservation.Dump();
				}
				SanityCheckReservations(true);
			}
		}

		// remove the entire party reservation slot if no more party members
		if (Reservation.PartyMembers.Num() == 0)
		{
			UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::RemovePlayer: Empty reservation found with leader %ls, removing", *Reservation.PartyLeader.ToString());
			Reservations.RemoveAtSwap(ResIdx--);
		}
	}

	if (bWasRemoved)
	{
		// Reshuffle existing teams so that beacon can accommodate biggest open slots
		UE_LOGF(LogPartyBeacon, Verbose, "UPartyBeaconState::RemovePlayer: Player removed, calling BestFitTeamAssignmentJiggle");
		BestFitTeamAssignmentJiggle();
	}

	SanityCheckReservations(false);
	return bWasRemoved;
}

int32 UPartyBeaconState::GetExistingReservation(const FUniqueNetIdRepl& PartyLeader) const
{
	int32 Result = INDEX_NONE;
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FPartyReservation& ReservationEntry = Reservations[ResIdx];
		if (ReservationEntry.PartyLeader == PartyLeader)
		{
			Result = ResIdx;
			break;
		}
	}

	return Result;
}

int32 UPartyBeaconState::GetExistingReservationContainingMember(const FUniqueNetIdRepl& PartyMember) const
{
	int32 Result = INDEX_NONE;
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FPartyReservation& ReservationEntry = Reservations[ResIdx];
		for (const FPlayerReservation& PlayerReservation : ReservationEntry.PartyMembers)
		{
			if (PlayerReservation.UniqueId == PartyMember)
			{
				Result = ResIdx;
				break;
			}
		}
	}

	return Result;
}

bool UPartyBeaconState::UpdateMemberPlatform(const FUniqueNetIdRepl& PartyMember, const FString& PlatformName)
{
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		FPartyReservation& ReservationEntry = Reservations[ResIdx];
		for (FPlayerReservation& PlayerReservation : ReservationEntry.PartyMembers)
		{
			if (PlayerReservation.UniqueId == PartyMember)
			{
				if (!PlatformName.IsEmpty())
				{
					PlayerReservation.Platform = PlatformName;
				}
				// Return that member was updated
				return true;
			}
		}
	}

	//Return that member was not updated
	return false;
}

bool UPartyBeaconState::PlayerHasReservation(const FUniqueNetId& PlayerId) const
{
	bool bFound = false;

	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FPartyReservation& ReservationEntry = Reservations[ResIdx];
		for (int32 PlayerIdx = 0; PlayerIdx < ReservationEntry.PartyMembers.Num(); PlayerIdx++)
		{
			if (*ReservationEntry.PartyMembers[PlayerIdx].UniqueId == PlayerId)
			{
				bFound = true;
				break;
			}
		}
	}

	return bFound;
}

bool UPartyBeaconState::GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const
{
	bool bFound = false;
	OutValidation = FString();

	for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bFound; ResIdx++)
	{
		const FPartyReservation& ReservationEntry = Reservations[ResIdx];
		for (int32 PlayerIdx = 0; PlayerIdx < ReservationEntry.PartyMembers.Num(); PlayerIdx++)
		{
			if (*ReservationEntry.PartyMembers[PlayerIdx].UniqueId == PlayerId)
			{
				OutValidation = ReservationEntry.PartyMembers[PlayerIdx].ValidationStr;
				bFound = true;
				break;
			}
		}
	}

	return bFound;
}

bool UPartyBeaconState::GetPartyLeader(const FUniqueNetIdRepl& InPartyMemberId, FUniqueNetIdRepl& OutPartyLeaderId) const
{
	bool bFoundReservation = false;

	if (InPartyMemberId.IsValid())
	{
		for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bFoundReservation; ResIdx++)
		{
			const FPartyReservation& ReservationEntry = Reservations[ResIdx];

			const FPlayerReservation* PlayerRes = ReservationEntry.PartyMembers.FindByPredicate(
				[InPartyMemberId](const FPlayerReservation& ExistingPlayerRes)
			{
				return InPartyMemberId == ExistingPlayerRes.UniqueId;
			});

			if (PlayerRes)
			{
				UE_LOGF(LogPartyBeacon, Display, "Found party leader for member %ls.", *InPartyMemberId.ToDebugString());
				OutPartyLeaderId = ReservationEntry.PartyLeader;
				bFoundReservation = true;
				break;
			}
		}

		if (!bFoundReservation)
		{
			UE_LOGF(LogPartyBeacon, Warning, "Found no reservation for player %ls, while looking for party leader.", *InPartyMemberId.ToDebugString());
		}
	}

	return bFoundReservation;
}

void UPartyBeaconState::DumpReservations() const
{
	FUniqueNetIdRepl NetId;
	FPlayerReservation PlayerRes;

	UE_LOGF(LogPartyBeacon, Display, "Session that reservations are for: %ls", *SessionName.ToString());
	UE_LOGF(LogPartyBeacon, Display, "Number of teams: %d", NumTeams);
	UE_LOGF(LogPartyBeacon, Display, "Number players per team: %d", NumPlayersPerTeam);
	UE_LOGF(LogPartyBeacon, Display, "Number total reservations: %d", MaxReservations);
	UE_LOGF(LogPartyBeacon, Display, "Number consumed reservations: %d", NumConsumedReservations);
	UE_LOGF(LogPartyBeacon, Display, "Number of party reservations: %d", Reservations.Num());

	// Log each party that has a reservation
	for (int32 PartyIndex = 0; PartyIndex < Reservations.Num(); PartyIndex++)
	{
		NetId = Reservations[PartyIndex].PartyLeader;
		UE_LOGF(LogPartyBeacon, Display, "\t Party leader: %ls", *NetId->ToDebugString());
		UE_LOGF(LogPartyBeacon, Display, "\t Party team: %d", Reservations[PartyIndex].TeamNum);
		UE_LOGF(LogPartyBeacon, Display, "\t Party size: %d", Reservations[PartyIndex].PartyMembers.Num());
		// Log each member of the party
		for (int32 MemberIndex = 0; MemberIndex < Reservations[PartyIndex].PartyMembers.Num(); MemberIndex++)
		{
			PlayerRes = Reservations[PartyIndex].PartyMembers[MemberIndex];
			UE_LOGF(LogPartyBeacon, Display, "\t  Party member: %ls [%ls] Cross: %ls", *PlayerRes.UniqueId->ToString(), *PlayerRes.Platform, *LexToString(PlayerRes.bAllowCrossplay));
		}
	}
	UE_LOGF(LogPartyBeacon, Display, "");
}

void UPartyBeaconState::SanityCheckReservations(const bool bIgnoreEmptyReservations) const
{
#if !UE_BUILD_SHIPPING
	// Verify that each player is only in exactly one reservation
	TMap<FUniqueNetIdRepl, FUniqueNetIdRepl> PlayersInReservation;
	for (const FPartyReservation& Reservation : Reservations)
	{
		if (!Reservation.PartyLeader.IsValid())
		{
			DumpReservations();
			checkf(false, TEXT("Reservation does not have valid party leader!"));
		}
		if (Reservation.PartyMembers.Num() == 0 && !bIgnoreEmptyReservations)
		{
			DumpReservations();
			checkf(false, TEXT("Reservation under leader %s has no members!"), *Reservation.PartyLeader.ToString());
		}
		for (const FPlayerReservation& PlayerReservation : Reservation.PartyMembers)
		{
			if (PlayerReservation.UniqueId.IsValid())
			{
				const FUniqueNetIdRepl* const ExistingReservationLeader = PlayersInReservation.Find(PlayerReservation.UniqueId);
				if (ExistingReservationLeader != nullptr)
				{
					if (*ExistingReservationLeader == Reservation.PartyLeader)
					{
						DumpReservations();
						checkf(false, TEXT("Player %s is in reservation with leader %s multiple times!"),
							*PlayerReservation.UniqueId.ToString(), *Reservation.PartyLeader.ToString());
					}
					else
					{
						DumpReservations();
						checkf(false, TEXT("Player %s is in multiple reservations (with leader %s and %s)!"),
							*PlayerReservation.UniqueId.ToString(), *(*ExistingReservationLeader).ToString(), *Reservation.PartyLeader.ToString());
					}
				}
				PlayersInReservation.Add(PlayerReservation.UniqueId, Reservation.PartyLeader);
			}
		}
	}
#endif // !UE_BUILD_SHIPPING
}

