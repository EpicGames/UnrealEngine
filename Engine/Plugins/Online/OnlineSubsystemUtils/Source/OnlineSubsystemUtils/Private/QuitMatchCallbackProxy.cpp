// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuitMatchCallbackProxy.h"
#include "Interfaces/OnlineTurnBasedInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QuitMatchCallbackProxy)

UQuitMatchCallbackProxy::UQuitMatchCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UQuitMatchCallbackProxy::~UQuitMatchCallbackProxy()
{
}

UQuitMatchCallbackProxy* UQuitMatchCallbackProxy::QuitMatch(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, EMPMatchOutcome::Outcome Outcome, int32 TurnTimeoutInSeconds)
{
	UQuitMatchCallbackProxy* Proxy = NewObject<UQuitMatchCallbackProxy>();
	Proxy->PlayerControllerWeakPtr = PlayerController;
	Proxy->WorldContextObject = WorldContextObject;
    Proxy->MatchID = MatchID;
	Proxy->Outcome = Outcome;
	Proxy->TurnTimeoutInSeconds = TurnTimeoutInSeconds;
	return Proxy;
}

void UQuitMatchCallbackProxy::Activate()
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("QuitMatch"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerControllerWeakPtr.Get());

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr OnlineTurnBasedPtr = Helper.OnlineSub->GetTurnBasedInterface();
		if (OnlineTurnBasedPtr.IsValid())
		{
			FTurnBasedMatchPtr Match = OnlineTurnBasedPtr->GetMatchWithID(MatchID);
			if (Match.IsValid())
			{
				FQuitMatchSignature QuitMatchSignature;
				QuitMatchSignature.BindUObject(this, &UQuitMatchCallbackProxy::QuitMatchDelegate);
				Match->QuitMatch(Outcome, TurnTimeoutInSeconds, QuitMatchSignature);
                return;
			}
			else
			{
				FString Message = FString::Printf(TEXT("Match ID %s not found"), *MatchID);
				FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn Based Matches not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}

	// Fail immediately
	OnFailure.Broadcast();
}

void UQuitMatchCallbackProxy::QuitMatchDelegate(FString InMatchID, bool Succeeded)
{
	if (Succeeded)
	{
		OnSuccess.Broadcast();
	}
	else
	{
		OnFailure.Broadcast();
	}
}

