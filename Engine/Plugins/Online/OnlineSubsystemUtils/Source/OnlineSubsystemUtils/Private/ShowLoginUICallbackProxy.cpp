// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShowLoginUICallbackProxy.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "Interfaces/OnlineExternalUIInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShowLoginUICallbackProxy)

UShowLoginUICallbackProxy::UShowLoginUICallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WorldContextObject(nullptr)
{
}

UShowLoginUICallbackProxy* UShowLoginUICallbackProxy::ShowExternalLoginUI(UObject* WorldContextObject, APlayerController* InPlayerController)
{
	UShowLoginUICallbackProxy* Proxy = NewObject<UShowLoginUICallbackProxy>();
	Proxy->PlayerControllerWeakPtr = InPlayerController;
	Proxy->WorldContextObject = WorldContextObject;
	return Proxy;
}

void UShowLoginUICallbackProxy::Activate()
{
	APlayerController* MyPlayerController = PlayerControllerWeakPtr.Get();
	if (!MyPlayerController)
	{
		FFrame::KismetExecutionMessage(TEXT("A player controller must be provided in order to show the external login UI."), ELogVerbosity::Warning);
		OnFailure.Broadcast(MyPlayerController);
		return;
	}

	const FOnlineSubsystemBPCallHelper Helper(TEXT("ShowLoginUI"), WorldContextObject);

	if (Helper.OnlineSub == nullptr)
	{
		OnFailure.Broadcast(MyPlayerController);
		return;
	}

	IOnlineExternalUIPtr OnlineExternalUI = Helper.OnlineSub->GetExternalUIInterface();
	if (!OnlineExternalUI.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("External UI not supported by the current online subsystem"), ELogVerbosity::Warning);
		OnFailure.Broadcast(MyPlayerController);
		return;
	}

	const ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(MyPlayerController->Player);
	if (LocalPlayer == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Can only show login UI for local players"), ELogVerbosity::Warning);
		OnFailure.Broadcast(MyPlayerController);
		return;
	}

	const bool bWaitForDelegate = OnlineExternalUI->ShowLoginUI(LocalPlayer->GetControllerId(), false, false,
		FOnLoginUIClosedDelegate::CreateUObject(this, &UShowLoginUICallbackProxy::OnShowLoginUICompleted));

	if (!bWaitForDelegate)
	{
		FFrame::KismetExecutionMessage(TEXT("The online subsystem couldn't show its login UI"), ELogVerbosity::Log);
		OnFailure.Broadcast(MyPlayerController);
	}
}

void UShowLoginUICallbackProxy::OnShowLoginUICompleted(FUniqueNetIdPtr UniqueId, int LocalPlayerNum, const FOnlineError& Error)
{
	// Update the cached unique ID for the local player and the player state.
	APlayerController* MyPlayerController = PlayerControllerWeakPtr.Get();

	if (MyPlayerController != nullptr)
	{
		ULocalPlayer* LocalPlayer = MyPlayerController->GetLocalPlayer();
		if (LocalPlayer != nullptr)
		{
			LocalPlayer->SetCachedUniqueNetId(FUniqueNetIdRepl(UniqueId));
		}

		if (MyPlayerController->PlayerState != nullptr)
		{
			MyPlayerController->PlayerState->SetUniqueId(FUniqueNetIdRepl(UniqueId));
		}
	}

	if (UniqueId.IsValid() && UniqueId->IsValid())
	{
		OnSuccess.Broadcast(MyPlayerController);
	}
	else
	{
		OnFailure.Broadcast(MyPlayerController);
	}
}

