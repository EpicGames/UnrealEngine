// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpectatorBeaconClient.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemUtils.h"
#include "SpectatorBeaconHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpectatorBeaconClient)

#if !UE_BUILD_SHIPPING
namespace SpectatorBeaconConsoleVariables
{
	/** Time to delay delegates firing a reservation request response */
	TAutoConsoleVariable<float> CVarDelayReservationResponse(
		TEXT("spectatorbeacon.DelayReservationResponse"),
		0.0f,
		TEXT("Delay time between received response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);

	/** Time to delay delegates firing a cancel reservation request response */
	TAutoConsoleVariable<float> CVarDelayCancellationResponse(
		TEXT("spectatorbeacon.DelayCancellationResponse"),
		0.0f,
		TEXT("Delay time between received cancel response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);

	/** Time to delay delegates firing a reservation update response */
	TAutoConsoleVariable<float> CVarDelayUpdateResponse(
		TEXT("spectatorbeacon.DelayUpdateResponse"),
		0.0f,
		TEXT("Delay time between received update response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);

	/** Time to delay delegates firing a reservation full response */
	TAutoConsoleVariable<float> CVarDelayFullResponse(
		TEXT("spectatorbeacon.DelayFullResponse"),
		0.0f,
		TEXT("Delay time between received full response and notification\n")
		TEXT("Time in secs"),
		ECVF_Default);
}
#endif

/** Max time to wait for a response from the server for CancelReservation */
#define CANCEL_FAILSAFE 5.0f

ASpectatorBeaconClient::ASpectatorBeaconClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	RequestType(ESpectatorClientRequestType::NonePending),
	bPendingReservationSent(false),
	bCancelReservation(false)
{
}

void ASpectatorBeaconClient::BeginDestroy()
{
	ClearTimers(true);
	Super::BeginDestroy();
}

void ASpectatorBeaconClient::ClearTimers(bool bCallFailSafeIfNeeded)
{
	if (bCallFailSafeIfNeeded &&
		CancelRPCFailsafe.IsValid())
	{
		UE_LOGF(LogBeacon, Verbose, "Clearing timers with cancel reservation in flight.  Calling Failsafe.");

		UWorld* World = GetWorld();
		if (World)
		{
			FTimerManager& TM = World->GetTimerManager();
			TM.ClearTimer(CancelRPCFailsafe);
		}
		OnCancelledFailsafe();
	}

	UWorld* World = GetWorld();
	if (World)
	{
		if (PendingResponseTimerHandle.IsValid())
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "ClearTimers: Pending reservation response cleared.");
		}

		if (PendingCancelResponseTimerHandle.IsValid())
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "ClearTimers: Pending cancel response cleared.");
		}

		if (PendingReservationUpdateTimerHandle.IsValid())
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "ClearTimers: Pending reservation update cleared.");
		}

		if (PendingReservationFullTimerHandle.IsValid())
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "ClearTimers: Pending reservation full cleared.");
		}

		if (CancelRPCFailsafe.IsValid())
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "ClearTimers: Cancel failsafe cleared.");
		}

		FTimerManager& TM = World->GetTimerManager();
		TM.ClearTimer(PendingResponseTimerHandle);
		TM.ClearTimer(PendingCancelResponseTimerHandle);
		TM.ClearTimer(PendingReservationUpdateTimerHandle);
		TM.ClearTimer(PendingReservationFullTimerHandle);
		TM.ClearTimer(CancelRPCFailsafe);

		PendingResponseTimerHandle.Invalidate();
		PendingCancelResponseTimerHandle.Invalidate();
		PendingReservationUpdateTimerHandle.Invalidate();
		PendingReservationFullTimerHandle.Invalidate();
		CancelRPCFailsafe.Invalidate();
	}
}

bool ASpectatorBeaconClient::RequestReservation(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingSpectator, const FPlayerReservation& Spectator)
{
	bool bSuccess = false;

	FURL ConnectURL(NULL, *ConnectInfoStr, TRAVEL_Absolute);
	UE_LOGF(LogSpectatorBeacon, Verbose, "SpectatorBeaconClient RequestReservation() connectURL: %ls", *ConnectURL.ToString());
	if (InitClient(ConnectURL))
	{
		DestSessionId = InSessionId;
		PendingReservation.SpectatorId = RequestingSpectator;
		PendingReservation.Spectator = Spectator;
		bPendingReservationSent = false;
		RequestType = ESpectatorClientRequestType::ExistingSessionReservation;
		bSuccess = true;
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Warning, "RequestReservation: Failure to init client beacon with %ls.", *ConnectURL.ToString());
		RequestType = ESpectatorClientRequestType::NonePending;
	}

	if (!bSuccess)
	{
		AOnlineBeacon::OnFailure(EBeaconFailureReason::InitFailure, TEXTVIEW("beacon_spectator_init_failure"));
	}

	return bSuccess;
}

bool ASpectatorBeaconClient::RequestReservation(const FOnlineSessionSearchResult& DesiredHost, const FUniqueNetIdRepl& RequestingSpectator, const FPlayerReservation& Spectator)
{

	if (DesiredHost.IsValid())
	{
		UWorld* World = GetWorld();

		IOnlineSubsystem* OnlineSub = Online::GetSubsystem(World);
		if (OnlineSub)
		{
			IOnlineSessionPtr SessionInt = OnlineSub->GetSessionInterface();
			if (SessionInt.IsValid())
			{
				FString ConnectInfo;
				if (SessionInt->GetResolvedConnectString(DesiredHost, NAME_BeaconPort, ConnectInfo))
				{
					FString SessionId = DesiredHost.Session.SessionInfo->GetSessionId().ToString();
					return RequestReservation(ConnectInfo, SessionId, RequestingSpectator, Spectator);
				}
			}
		}
	}

	AOnlineBeacon::OnFailure(EBeaconFailureReason::ApplicationError, TEXTVIEW("beacon_spectator_request_reservation_failure"));
	
	return false;
}

void ASpectatorBeaconClient::CancelReservation()
{
	if (ensure(PendingReservation.SpectatorId.IsValid()))
	{
		bCancelReservation = true;

		// Clear out any pending response handling, only the cancel matters
		ClearTimers(false);

		if (bPendingReservationSent)
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "Sending cancel reservation request.");
			ServerCancelReservationRequest(PendingReservation.SpectatorId);

			// In case the server is loading or unresponsive (ie no host beacon)
			FTimerDelegate TimerDelegate;
			TimerDelegate.BindUObject(this, &ThisClass::OnCancelledFailsafe);

			UWorld* World = GetWorld();
			check(World);
			World->GetTimerManager().SetTimer(CancelRPCFailsafe, TimerDelegate, CANCEL_FAILSAFE, false);
		}
		else
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "Reservation request never sent, no need to send cancelation request.");
			OnCancelledComplete();
		}
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "Unable to cancel reservation request with invalid spectator.");
		OnCancelledComplete();
	}
}

void ASpectatorBeaconClient::OnConnected()
{
	if (!bCancelReservation)
	{
		if (RequestType == ESpectatorClientRequestType::ExistingSessionReservation)
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon connection established, sending join reservation request.");
			ServerReservationRequest(DestSessionId, PendingReservation);
			bPendingReservationSent = true;
		}
		else
		{
			UE_LOGF(LogSpectatorBeacon, Warning, "Failed to handle reservation request type %ls", ToString(RequestType));
			AOnlineBeacon::OnFailure(EBeaconFailureReason::TransportError, TEXTVIEW("beacon_spectator_reservation_failure"));
		}
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "Reservation request previously canceled, nothing sent.");
		OnCancelledComplete();
	}
}

void ASpectatorBeaconClient::OnCancelledFailsafe()
{
	ClientCancelReservationResponse_Implementation(ESpectatorReservationResult::ReservationRequestCanceled);
}

void ASpectatorBeaconClient::OnCancelledComplete()
{
	ReservationRequestComplete.ExecuteIfBound(ESpectatorReservationResult::ReservationRequestCanceled);
	RequestType = ESpectatorClientRequestType::NonePending;
	bCancelReservation = false;
}

void ASpectatorBeaconClient::OnFailure()
{
	ClearTimers(true);
	RequestType = ESpectatorClientRequestType::NonePending;
	Super::OnFailure();
}

/// @cond DOXYGEN_WARNINGS

bool ASpectatorBeaconClient::ServerReservationRequest_Validate(const FString& SessionId, const FSpectatorReservation& Reservation)
{
	return !SessionId.IsEmpty() && Reservation.SpectatorId.IsValid();
}

void ASpectatorBeaconClient::ServerReservationRequest_Implementation(const FString& SessionId, const FSpectatorReservation& Reservation)
{
	ASpectatorBeaconHost* BeaconHost = Cast<ASpectatorBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		PendingReservation = Reservation;
		RequestType = ESpectatorClientRequestType::ExistingSessionReservation;
		BeaconHost->ProcessReservationRequest(this, SessionId, Reservation);
	}
}

bool ASpectatorBeaconClient::ServerCancelReservationRequest_Validate(const FUniqueNetIdRepl& SpectatorId)
{
	return true;
}

void ASpectatorBeaconClient::ServerCancelReservationRequest_Implementation(const FUniqueNetIdRepl& SpectatorId)
{
	ASpectatorBeaconHost* BeaconHost = Cast<ASpectatorBeaconHost>(GetBeaconOwner());
	if (BeaconHost)
	{
		bCancelReservation = true;
		BeaconHost->ProcessCancelReservationRequest(this, SpectatorId);
	}
}

void ASpectatorBeaconClient::ClientReservationResponse_Implementation(ESpectatorReservationResult::Type ReservationResponse)
{
	if (!bCancelReservation)
	{
#if !UE_BUILD_SHIPPING
		const float Rate = SpectatorBeaconConsoleVariables::CVarDelayReservationResponse.GetValueOnGameThread();
#else
		const float Rate = 0.0f;
#endif
		if (Rate > 0.0f)
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon response received %ls, waiting %fs to notify", ESpectatorReservationResult::ToString(ReservationResponse), Rate);

			FTimerDelegate TimerDelegate;
			TimerDelegate.BindLambda([this, ReservationResponse]()
			{
				ProcessReservationResponse(ReservationResponse);
			});

			PendingResponseTimerHandle = DelayResponse(TimerDelegate, Rate);
		}
		else
		{
			ProcessReservationResponse(ReservationResponse);
		}
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon response received %ls, ignored due to cancel in progress", ESpectatorReservationResult::ToString(ReservationResponse));
		// Cancel RPC or failsafe timer will trigger the cancel
	}
}

/// @endcond

void ASpectatorBeaconClient::ProcessReservationResponse(ESpectatorReservationResult::Type ReservationResponse)
{
	if (!bCancelReservation)
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon response received %ls", ESpectatorReservationResult::ToString(ReservationResponse));
		ReservationRequestComplete.ExecuteIfBound(ReservationResponse);
		RequestType = ESpectatorClientRequestType::NonePending;
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon response received %ls, ignored due to cancel in progress", ESpectatorReservationResult::ToString(ReservationResponse));
		// Cancel RPC or failsafe timer will trigger the cancel
	}
}

/// @cond DOXYGEN_WARNINGS

void ASpectatorBeaconClient::ClientCancelReservationResponse_Implementation(ESpectatorReservationResult::Type ReservationResponse)
{
	ensure(bCancelReservation);

	// Clear out any pending response handling (including failsafe timer)
	ClearTimers(false);
#if !UE_BUILD_SHIPPING	
	const float Rate = SpectatorBeaconConsoleVariables::CVarDelayCancellationResponse.GetValueOnGameThread();
#else
	const float Rate = 0.0f;
#endif
	if (Rate > 0.0f)
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon cancellation response received %ls, waiting %fs to notify", ESpectatorReservationResult::ToString(ReservationResponse), Rate);

		FTimerDelegate TimerDelegate;
		TimerDelegate.BindLambda([this, ReservationResponse]()
		{
			ProcessCancelReservationResponse(ReservationResponse);
		});

		PendingCancelResponseTimerHandle = DelayResponse(TimerDelegate, Rate);
	}
	else
	{
		ProcessCancelReservationResponse(ReservationResponse);
	}
}

/// @endcond

void ASpectatorBeaconClient::ProcessCancelReservationResponse(ESpectatorReservationResult::Type ReservationResponse)
{
	ensure(ReservationResponse == ESpectatorReservationResult::ReservationRequestCanceled || ReservationResponse == ESpectatorReservationResult::ReservationNotFound);
	ensure(bCancelReservation);

	OnCancelledComplete();
}

/// @cond DOXYGEN_WARNINGS

void ASpectatorBeaconClient::ClientSendReservationUpdates_Implementation(int32 NumRemainingReservations)
{
	if (!bCancelReservation)
	{
#if !UE_BUILD_SHIPPING
		const float Rate = SpectatorBeaconConsoleVariables::CVarDelayUpdateResponse.GetValueOnGameThread();
#else
		const float Rate = 0.0f;
#endif
		if (Rate > 0.0f)
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon reservations remaining %d, waiting %fs to notify", NumRemainingReservations, Rate);

			FTimerDelegate TimerDelegate;
			TimerDelegate.BindLambda([this, NumRemainingReservations]()
			{
				ProcessReservationUpdate(NumRemainingReservations);
			});

			PendingReservationUpdateTimerHandle = DelayResponse(TimerDelegate, Rate);
		}
		else
		{
			ProcessReservationUpdate(NumRemainingReservations);
		}
	}
}

/// @endcond

void ASpectatorBeaconClient::ProcessReservationUpdate(int32 NumRemainingReservations)
{
	UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon reservations remaining %d", NumRemainingReservations);
	ReservationCountUpdate.ExecuteIfBound(NumRemainingReservations);
}

/// @cond DOXYGEN_WARNINGS

void ASpectatorBeaconClient::ClientSendReservationFull_Implementation()
{
	if (!bCancelReservation)
	{
#if !UE_BUILD_SHIPPING
		const float Rate = SpectatorBeaconConsoleVariables::CVarDelayFullResponse.GetValueOnGameThread();
#else
		const float Rate = 0.0f;
#endif
		if (Rate > 0.0f)
		{
			UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon reservations full, waiting %fs to notify", Rate);

			FTimerDelegate TimerDelegate;
			TimerDelegate.BindLambda([this]()
			{
				ProcessReservationFull();
			});

			PendingReservationFullTimerHandle = DelayResponse(TimerDelegate, Rate);
		}
		else
		{
			ProcessReservationFull();
		}
	}
}

/// @endcond

void ASpectatorBeaconClient::ProcessReservationFull()
{
	UE_LOGF(LogSpectatorBeacon, Verbose, "Spectator beacon reservations full");
	ReservationFull.ExecuteIfBound();
}

FTimerHandle ASpectatorBeaconClient::DelayResponse(FTimerDelegate& Delegate, float Delay)
{
	FTimerHandle TimerHandle;

	UWorld* World = GetWorld();
	if (ensure(World != nullptr))
	{
		World->GetTimerManager().SetTimer(TimerHandle, Delegate, Delay, false);
	}

	return TimerHandle;
}

