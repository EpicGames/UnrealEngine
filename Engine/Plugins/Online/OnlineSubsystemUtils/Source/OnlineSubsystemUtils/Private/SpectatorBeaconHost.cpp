// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpectatorBeaconHost.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Misc/CommandLine.h"
#include "OnlineSessionSettings.h"
#include "UObject/Package.h"
#include "Engine/NetConnection.h"
#include "OnlineSubsystemUtils.h"
#include "SpectatorBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpectatorBeaconHost)

ASpectatorBeaconHost::ASpectatorBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	State(NULL),
	bLogoutOnSessionTimeout(true),
	bIsValidationStrRequired(true)
{
	ClientBeaconActorClass = ASpectatorBeaconClient::StaticClass();
	BeaconTypeName = ClientBeaconActorClass->GetName();

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bAllowTickOnDedicatedServer = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
}

void ASpectatorBeaconHost::PostInitProperties()
{
	Super::PostInitProperties();
#if !UE_BUILD_SHIPPING
	// This value is set on the CDO as well on purpose
	bLogoutOnSessionTimeout = FParse::Param(FCommandLine::Get(), TEXT("NoTimeouts")) ? false : true;
#endif
}

bool ASpectatorBeaconHost::InitHostBeacon(int32 InMaxReservations, FName InSessionName)
{
	UE_LOGF(LogSpectatorBeacon, Verbose, "InitHostBeacon MaxSize:%d", InMaxReservations);
	if (InMaxReservations > 0)
	{
		State = NewObject<USpectatorBeaconState>(GetTransientPackage(), GetSpectatorBeaconHostClass());
		if (State->InitState(InMaxReservations, InSessionName))
		{
			return true;
		}
	}

	return false;
}

bool ASpectatorBeaconHost::InitFromBeaconState(USpectatorBeaconState* PrevState)
{
	if (!State && PrevState)
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "InitFromBeaconState MaxSize:%d", PrevState->MaxReservations);
		State = PrevState;
		return true;
	}

	return false;
}

void ASpectatorBeaconHost::Tick(float DeltaTime)
{
	if (State)
	{
		UWorld* World = GetWorld();
		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);

		if (SessionInt.IsValid())
		{
			FName SessionName = State->GetSessionName();
			TArray<FSpectatorReservation>& Reservations = State->GetReservations();

			FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
			if (Session)
			{
				TArray< FUniqueNetIdPtr > PlayersToLogout;
				for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
				{
					FSpectatorReservation& SpectatorRes = Reservations[ResIdx];

					// Determine if we have a client connection for the reservation 
					bool bIsConnectedReservation = false;
					for (int32 ClientIdx = 0; ClientIdx < ClientActors.Num(); ClientIdx++)
					{
						ASpectatorBeaconClient* Client = Cast<ASpectatorBeaconClient>(ClientActors[ClientIdx]);
						if (Client)
						{
							const FSpectatorReservation& ClientRes = Client->GetPendingReservation();
							if (ClientRes.SpectatorId == SpectatorRes.SpectatorId)
							{
								bIsConnectedReservation = true;
								break;
							}
						}
						else
						{
							UE_LOGF(LogSpectatorBeacon, Error, "Missing SpectatorBeaconClient found in ClientActors array");
							ClientActors.RemoveAtSwap(ClientIdx);
							ClientIdx--;
						}
					}

					if (bIsConnectedReservation)
					{
						FPlayerReservation& PlayerEntry = SpectatorRes.Spectator;
						PlayerEntry.ElapsedTime = 0.0f;
					}
					// Once a client beacon disconnects, update the elapsed time since they were found as a registrant in the game session
					else
					{
						FPlayerReservation& PlayerEntry = SpectatorRes.Spectator;

						// Determine if the player is the owner of the session	
						const bool bIsSessionOwner = Session->OwningUserId.IsValid() && (*Session->OwningUserId == *PlayerEntry.UniqueId);

						// Determine if the player member is registered in the game session
						if (SessionInt->IsPlayerInSession(SessionName, *PlayerEntry.UniqueId) ||
							// Never timeout the session owner
							bIsSessionOwner)
						{
							FUniqueNetIdMatcher PlayerMatch(*PlayerEntry.UniqueId);
							int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
							if (FoundIdx != INDEX_NONE)
							{
								UE_LOGF(LogSpectatorBeacon, Display, "Beacon (%ls): pending player %ls found in session (%ls), removing.",
									*GetName(),
									*(PlayerEntry.UniqueId.ToDebugString()),
									*SessionName.ToString());

								// reset elapsed time since found
								PlayerEntry.ElapsedTime = 0.0f;
								// also remove from pending join list
								State->PlayersPendingJoin.RemoveAtSwap(FoundIdx);
							}
						}
						else
						{
							// update elapsed time
							PlayerEntry.ElapsedTime += DeltaTime;

							if (bLogoutOnSessionTimeout)
							{
								// if the player is pending it's initial join then check against TravelSessionTimeoutSecs instead
								FUniqueNetIdMatcher PlayerMatch(*PlayerEntry.UniqueId);
								const int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
								const bool bIsPlayerPendingJoin = FoundIdx != INDEX_NONE;
								// if the timeout has been exceeded then add to list of players 
								// that need to be logged out from the beacon
								if ((bIsPlayerPendingJoin && PlayerEntry.ElapsedTime > TravelSessionTimeoutSecs) ||
									(!bIsPlayerPendingJoin && PlayerEntry.ElapsedTime > SessionTimeoutSecs))
								{
									PlayersToLogout.AddUnique(PlayerEntry.UniqueId.GetUniqueNetId());
								}
							}
						}
					}
				}

				if (bLogoutOnSessionTimeout)
				{
					// Logout any players that timed out
					for (int32 LogoutIdx = 0; LogoutIdx < PlayersToLogout.Num(); LogoutIdx++)
					{
						bool bFound = false;
						const FUniqueNetIdPtr& UniqueId = PlayersToLogout[LogoutIdx];
						float ElapsedSessionTime = 0.f;
						for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
						{
							const FSpectatorReservation& SpectatorRes = Reservations[ResIdx];
							const FPlayerReservation& PlayerEntry = SpectatorRes.Spectator;
							if (*PlayerEntry.UniqueId == *UniqueId)
							{
								ElapsedSessionTime = PlayerEntry.ElapsedTime;
								bFound = true;
								break;
							}

							if (bFound)
							{
								break;
							}
						}

						UE_LOGF(LogSpectatorBeacon, Display, "Beacon (%ls): pending player logout due to timeout for %ls, elapsed time = %0.3f, removing",
							*GetName(),
							*UniqueId->ToDebugString(),
							ElapsedSessionTime);
						// Also remove from pending join list
						State->PlayersPendingJoin.RemoveSingleSwap(UniqueId);
						// let the beacon handle the logout and notifications/delegates
						FUniqueNetIdRepl RemovedId(UniqueId);
						HandlePlayerLogout(RemovedId);
					}
				}
			}
		}
	}
}

void ASpectatorBeaconHost::SendReservationUpdates()
{
	if (State && ClientActors.Num() > 0)
	{
		int32 NumRemaining = State->GetRemainingReservations();
		int32 MaxReservations = State->GetMaxReservations();
		if (NumRemaining < MaxReservations)
		{
			if (NumRemaining > 0)
			{
				UE_LOGF(LogSpectatorBeacon, Verbose, "Sending reservation update %d", NumRemaining);
				for (AOnlineBeaconClient* ClientActor : ClientActors)
				{
					ASpectatorBeaconClient* SpectatorBeaconClient = Cast<ASpectatorBeaconClient>(ClientActor);
					if (SpectatorBeaconClient)
					{
						SpectatorBeaconClient->ClientSendReservationUpdates(NumRemaining);
					}
				}
			}
			else
			{
				UE_LOGF(LogSpectatorBeacon, Verbose, "Sending reservation full");
				for (AOnlineBeaconClient* ClientActor : ClientActors)
				{
					ASpectatorBeaconClient* SpectatorBeaconClient = Cast<ASpectatorBeaconClient>(ClientActor);
					if (SpectatorBeaconClient)
					{
						SpectatorBeaconClient->ClientSendReservationFull();
					}
				}
			}
		}
	}
}

void ASpectatorBeaconHost::NewPlayerAdded(const FPlayerReservation& NewPlayer)
{
	if (NewPlayer.UniqueId.IsValid())
	{
		if (State)
		{
			FUniqueNetIdMatcher PlayerMatch(*NewPlayer.UniqueId);
			int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
			if (FoundIdx == INDEX_NONE)
			{
				UE_LOGF(LogSpectatorBeacon, Verbose, "Beacon adding pending player %ls", *NewPlayer.UniqueId.ToDebugString());
				State->PlayersPendingJoin.Add(NewPlayer.UniqueId.GetUniqueNetId());
				OnNewPlayerAdded().ExecuteIfBound(NewPlayer);
			}
		}
		else
		{
			UE_LOGF(LogSpectatorBeacon, Warning, "Beacon skipping PlayersPendingJoin for beacon with no state!");
		}
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Warning, "Beacon skipping PlayersPendingJoin for invalid player!");
	}
}

void ASpectatorBeaconHost::NotifyReservationEventNextFrame(FOnReservationUpdate& ReservationEvent)
{
	UWorld* World = GetWorld();
	check(World);

	// Calling this on next tick to protect against re-entrance
	World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateLambda([ReservationEvent]() { ReservationEvent.ExecuteIfBound(); }));
}

void ASpectatorBeaconHost::HandlePlayerLogout(const FUniqueNetIdRepl& PlayerId)
{
	if (PlayerId.IsValid())
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "HandlePlayerLogout %ls", *PlayerId->ToDebugString());

		if (State)
		{
			if (State->RemovePlayer(PlayerId))
			{
				SendReservationUpdates();
				NotifyReservationEventNextFrame(ReservationChanged);
			}
		}
	}
}

bool ASpectatorBeaconHost::PlayerHasReservation(const FUniqueNetId& PlayerId) const
{
	bool bHasReservation = false;
	if (State)
	{
		bHasReservation = State->PlayerHasReservation(PlayerId);
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, no reservations.",
			*GetBeaconType());
	}

	return bHasReservation;
}

bool ASpectatorBeaconHost::GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const
{
	OutValidation.Empty();

	bool bHasValidation = false;
	if (State)
	{
		bHasValidation = State->GetPlayerValidation(PlayerId, OutValidation);
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, no validation.",
			*GetBeaconType());
	}

	return bHasValidation;
}

ESpectatorReservationResult::Type ASpectatorBeaconHost::AddSpectatorReservation(const FSpectatorReservation& ReservationRequest)
{
	ESpectatorReservationResult::Type Result = ESpectatorReservationResult::GeneralError;

	if (!State || GetBeaconState() == EBeaconState::DenyRequests)
	{
		return ESpectatorReservationResult::ReservationDenied;
	}

	if (ReservationRequest.IsValid(bIsValidationStrRequired))
	{
		TArray<FSpectatorReservation>& Reservations = State->GetReservations();
		const int32 ExistingReservationIdx = State->GetExistingReservation(ReservationRequest.SpectatorId);
		if (ExistingReservationIdx != INDEX_NONE)
		{
			FSpectatorReservation& ExistingReservation = Reservations[ExistingReservationIdx];
			const FPlayerReservation& NewPlayerRes = ReservationRequest.Spectator;
			if (!NewPlayerRes.ValidationStr.IsEmpty())
			{
				FPlayerReservation& PlayerRes = ExistingReservation.Spectator;
				// Update the validation auth strings because they may have changed with a new login 
				PlayerRes.ValidationStr = NewPlayerRes.ValidationStr;

			}

			if (State->CrossPlayAllowed(ReservationRequest))
			{
				SendReservationUpdates();

				// Clean up the game entities for these duplicate players
				DuplicateReservation.ExecuteIfBound(ReservationRequest);

				NewPlayerAdded(ReservationRequest.Spectator);
				Result = ESpectatorReservationResult::ReservationDuplicate;
			}
			else
			{
				// Cross play restriction
				Result = ESpectatorReservationResult::ReservationDenied_CrossPlayRestriction;
			}
		}
		else
		{
			// Check for players we already have reservations for
			// Keep track of team index for existing members - if we have members on opposing teams, reject this reservation
			bool bContainsExistingMembers = false;
			const FPlayerReservation& Spectator = ReservationRequest.Spectator;
			{
				int32 MemberExistingSpectatorReservationIdx = State->GetExistingReservation(Spectator.UniqueId);
				if (MemberExistingSpectatorReservationIdx != INDEX_NONE)
				{
					const FSpectatorReservation& MemberExistingSpectatorReservation = Reservations[MemberExistingSpectatorReservationIdx];
					UE_LOGF(LogSpectatorBeacon, Display, "ASpectatorBeaconHost::AddSpectatorReservation: Found existing reservation for spectator %ls", *Spectator.UniqueId.ToDebugString());
					ReservationRequest.Dump();
					MemberExistingSpectatorReservation.Dump();
					bContainsExistingMembers = true;
				}
				else
				{
					// Is this player in the pending join list?
					FUniqueNetIdMatcher PlayerMatch(*Spectator.UniqueId);
					int32 FoundIdx = State->PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
					if (FoundIdx != INDEX_NONE)
					{
						UE_LOGF(LogSpectatorBeacon, Display, "ASpectatorBeaconHost::AddSpectatorReservation: Found spectator %ls in the pending player list", *Spectator.UniqueId.ToDebugString());
						ReservationRequest.Dump();
						bContainsExistingMembers = true;
					}
				}
			}

			if (!bContainsExistingMembers)
			{
				if (State->DoesReservationFit(ReservationRequest))
				{
					bool bContinue = true;
					if (ValidatePlayers.IsBound())
					{
						bContinue = ValidatePlayers.Execute(ReservationRequest.Spectator);
					}

					if (bContinue)
					{						
						if (State->CrossPlayAllowed(ReservationRequest))
						{
							if (State->AddReservation(ReservationRequest))
							{
								// Keep track of newly added players
								NewPlayerAdded(Spectator);

								SendReservationUpdates();

								NotifyReservationEventNextFrame(ReservationChanged);
								if (State->IsBeaconFull())
								{
									NotifyReservationEventNextFrame(ReservationsFull);
								}
								Result = ESpectatorReservationResult::ReservationAccepted;
							}
							else
							{
								// Something wrong with team assignment
								Result = ESpectatorReservationResult::IncorrectPlayerCount;
							}
						}
						else
						{
							// Cross play restriction
							Result = ESpectatorReservationResult::ReservationDenied_CrossPlayRestriction;
						}
					
					}
					else
					{
						// Validate players failed above
						Result = ESpectatorReservationResult::ReservationDenied_Banned;
					}
				}
				else
				{
					// Reservation doesn't fit (not enough space in general)
					Result = ESpectatorReservationResult::SpectatorLimitReached;
				}
			}
			else
			{
				// Reservation contains players already accounted for
				Result = ESpectatorReservationResult::ReservationDenied_ContainsExistingPlayers;
			}
		}
	}
	else
	{
		// Invalid reservation
		Result = ESpectatorReservationResult::ReservationInvalid;
	}

	return Result;
}

ESpectatorReservationResult::Type ASpectatorBeaconHost::RemoveSpectatorReservation(const FUniqueNetIdRepl& Spectator)
{
	if (State && State->RemoveReservation(Spectator))
	{
		CancelationReceived.ExecuteIfBound(*Spectator);

		SendReservationUpdates();
		NotifyReservationEventNextFrame(ReservationChanged);
		return ESpectatorReservationResult::ReservationRequestCanceled;
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Warning, "Failed to find reservation to cancel for spectator %ls:", Spectator.IsValid() ? *Spectator->ToString() : TEXT("INVALID"));
		return ESpectatorReservationResult::ReservationNotFound;
	}
}

void ASpectatorBeaconHost::RegisterAuthTicket(const FUniqueNetIdRepl& InSpectatorId, const FString& InAuthTicket)
{
	if (State)
	{
		State->RegisterAuthTicket(InSpectatorId, InAuthTicket);
	}
	else
	{
		UE_LOGF(LogSpectatorBeacon, Warning,
			"Beacon (%ls) hasn't been initialized yet, not able to register auth ticket.",
			*GetBeaconType());
	}
}

bool ASpectatorBeaconHost::DoesSessionMatch(const FString& SessionId) const
{
	if (State)
	{
		UWorld* World = GetWorld();
		FName SessionName = State->GetSessionName();

		IOnlineSessionPtr SessionInt = Online::GetSessionInterface(World);
		FNamedOnlineSession* Session = SessionInt.IsValid() ? SessionInt->GetNamedSession(SessionName) : NULL;
		if (Session && Session->SessionInfo.IsValid() && !SessionId.IsEmpty() && Session->SessionInfo->GetSessionId().ToString() == SessionId)
		{
			return true;
		}
	}

	return false;
}

void ASpectatorBeaconHost::ProcessReservationRequest(ASpectatorBeaconClient* Client, const FString& SessionId, const FSpectatorReservation& ReservationRequest)
{
	UE_LOGF(LogSpectatorBeacon, Verbose, "ProcessReservationRequest %ls SessionId %ls Spectator: %ls from (%ls)",
		Client ? *Client->GetName() : TEXT("NULL"),
		*SessionId,
		ReservationRequest.SpectatorId.IsValid() ? *ReservationRequest.SpectatorId->ToString() : TEXT("INVALID"),
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));
	if (UE_LOG_ACTIVE(LogSpectatorBeacon, Verbose))
	{
		ReservationRequest.Dump();
	}

	if (Client)
	{
		ESpectatorReservationResult::Type Result = ESpectatorReservationResult::BadSessionId;
		if (DoesSessionMatch(SessionId))
		{
			Result = AddSpectatorReservation(ReservationRequest);
		}

		UE_LOGF(LogSpectatorBeacon, Verbose, "ProcessReservationRequest result: %ls", ESpectatorReservationResult::ToString(Result));
		if (UE_LOG_ACTIVE(LogSpectatorBeacon, Verbose) &&
			(Result != ESpectatorReservationResult::ReservationAccepted))
		{
			DumpReservations();
		}

		Client->ClientReservationResponse(Result);
	}
}

void ASpectatorBeaconHost::ProcessCancelReservationRequest(ASpectatorBeaconClient* Client, const FUniqueNetIdRepl& Spectator)
{
	UE_LOGF(LogSpectatorBeacon, Verbose, "ProcessCancelReservationRequest %ls Spectator: %ls from (%ls)",
		Client ? *Client->GetName() : TEXT("NULL"),
		Spectator.IsValid() ? *Spectator->ToString() : TEXT("INVALID"),
		Client ? *Client->GetNetConnection()->LowLevelDescribe() : TEXT("NULL"));

	if (Client)
	{
		ESpectatorReservationResult::Type Result = RemoveSpectatorReservation(Spectator);
		UE_LOGF(LogSpectatorBeacon, Verbose, "ProcessCancelReservationRequest result: %ls", ESpectatorReservationResult::ToString(Result));
		if (UE_LOG_ACTIVE(LogSpectatorBeacon, Verbose) &&
			(Result != ESpectatorReservationResult::ReservationRequestCanceled))
		{
			DumpReservations();
		}

		Client->ClientCancelReservationResponse(Result);
	}
}

void ASpectatorBeaconHost::DumpReservations() const
{
	UE_LOGF(LogSpectatorBeacon, Display, "Debug info for Beacon: %ls", *GetBeaconType());
	if (State)
	{
		State->DumpReservations();
	}
	UE_LOGF(LogSpectatorBeacon, Display, "");
}

