// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpectatorBeaconState.h"
#include "OnlineSubsystemTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpectatorBeaconState)

DEFINE_LOG_CATEGORY(LogSpectatorBeacon);

bool FSpectatorReservation::IsValid(bool bIsValidationStrRequired) const
{
	bool bIsValid = false;
	if (SpectatorId.IsValid())
	{
		bIsValid = true;
		if (!Spectator.UniqueId.IsValid())
		{
			bIsValid = false;
		}
		else if (Spectator.Platform.IsEmpty())
		{
			bIsValid = false;
		}
		else if (bIsValidationStrRequired &&
			SpectatorId == Spectator.UniqueId &&
			Spectator.ValidationStr.IsEmpty())
		{
			bIsValid = false;
		}
	}

	return bIsValid;
}

void FSpectatorReservation::Dump() const
{
	UE_LOGF(LogSpectatorBeacon, Display, "Spectator Reservation:");
	UE_LOGF(LogSpectatorBeacon, Display, "  Spectator: %ls", *SpectatorId.ToDebugString());
	UE_LOGF(LogSpectatorBeacon, Display, "  UniqueId: %ls", *Spectator.UniqueId.ToDebugString());
	UE_LOGF(LogSpectatorBeacon, Display, "	Crossplay: %ls", *LexToString(Spectator.bAllowCrossplay));
#if UE_BUILD_SHIPPING
		UE_LOGF(LogSpectatorBeacon, Display, "  ValidationStr: %d bytes", Spectator.ValidationStr.Len());
#else
		UE_LOGF(LogSpectatorBeacon, Display, "  ValidationStr: %ls", *Spectator.ValidationStr);
#endif
		UE_LOGF(LogSpectatorBeacon, Display, "  ElapsedTime: %0.2f", Spectator.ElapsedTime);
}

USpectatorBeaconState::USpectatorBeaconState(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	SessionName(NAME_None),
	NumConsumedReservations(0),
	MaxReservations(0),
	bRestrictCrossConsole(false)
{
}

bool USpectatorBeaconState::InitState(int32 InMaxReservations, FName InSessionName)
{
	if (InMaxReservations > 0)
	{
		SessionName = InSessionName;
		MaxReservations = InMaxReservations;
		Reservations.Empty(MaxReservations);
		return true;
	}

	return false;
}

bool USpectatorBeaconState::HasCrossplayOptOutReservation() const
{
	for (const FSpectatorReservation& ExistingReservation : Reservations)
	{
		const FPlayerReservation& ExistingPlayer = ExistingReservation.Spectator;
		if (!ExistingPlayer.bAllowCrossplay)
		{
			return true;
		}
	}

	return false;
}

int32 USpectatorBeaconState::GetReservationPlatformCount(const FString& InPlatform) const
{
	int32 PlayerCount = 0;
	for (const FSpectatorReservation& ExistingReservation : Reservations)
	{
		const FPlayerReservation& ExistingPlayer = ExistingReservation.Spectator;
		if (ExistingPlayer.Platform == InPlatform)
		{
			PlayerCount++;
		}
	}

	return PlayerCount;
}

bool USpectatorBeaconState::CrossPlayAllowed(const FSpectatorReservation& ReservationRequest) const
{
	// Since this player is a spectator, it won't be playing, so allow crossplay.
	return true;
}

bool USpectatorBeaconState::DoesReservationFit(const FSpectatorReservation& ReservationRequest) const
{
	const bool bRoomForReservation = (NumConsumedReservations + 1) <= MaxReservations;

	UE_LOGF(LogSpectatorBeacon, Verbose, "USpectatorBeaconState::DoesReservationFit: NumConsumedReservations: %d MaxReservations: %d", NumConsumedReservations, MaxReservations);

	return bRoomForReservation;
}

bool USpectatorBeaconState::AddReservation(const FSpectatorReservation& ReservationRequest)
{
	if (UE_LOG_ACTIVE(LogSpectatorBeacon, Verbose))
	{
		UE_LOGF(LogSpectatorBeacon, Verbose, "USpectatorBeaconState::AddReservation");
		ReservationRequest.Dump();
	}
	
	NumConsumedReservations += 1;
	UE_LOGF(LogSpectatorBeacon, Verbose, "UsPECTATORBeaconState::AddReservation: Setting NumConsumedReservations to %d", NumConsumedReservations);
	int32 ResIdx = Reservations.Add(ReservationRequest);
	SanityCheckReservations(false);

	return true;
}

bool USpectatorBeaconState::RemoveReservation(const FUniqueNetIdRepl& Spectator)
{
	const int32 ExistingReservationIdx = GetExistingReservation(Spectator);
	if (ExistingReservationIdx != INDEX_NONE)
	{
		NumConsumedReservations -= 1;
		UE_LOGF(LogSpectatorBeacon, Verbose, "USpectatorBeaconState::RemoveReservation: %ls, setting NumConsumedReservations to %d", *Spectator.ToString(), NumConsumedReservations);

		const FPlayerReservation& PlayerRes = Reservations[ExistingReservationIdx].Spectator;
		FUniqueNetIdMatcher PlayerMatch(*PlayerRes.UniqueId);
		int32 FoundIdx = PlayersPendingJoin.IndexOfByPredicate(PlayerMatch);
		if (FoundIdx != INDEX_NONE)
		{
			PlayersPendingJoin.RemoveAtSwap(FoundIdx);
		}

		Reservations.RemoveAtSwap(ExistingReservationIdx);

		SanityCheckReservations(false);
		return true;
	}

	return false;
}

void USpectatorBeaconState::RegisterAuthTicket(const FUniqueNetIdRepl& InSpectatorId, const FString& InAuthTicket)
{
	if (InSpectatorId.IsValid() && !InAuthTicket.IsEmpty())
	{
		bool bFoundReservation = false;

		for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bFoundReservation; ResIdx++)
		{
			FSpectatorReservation& ReservationEntry = Reservations[ResIdx];

			FPlayerReservation& PlayerRes = ReservationEntry.Spectator;

			if (InSpectatorId == PlayerRes.UniqueId)
			{
				if (PlayerRes.ValidationStr.IsEmpty())
				{
					UE_LOGF(LogSpectatorBeacon, VeryVerbose, "Setting auth ticket for spectator %ls.", *InSpectatorId.ToDebugString());
				}
				else if (PlayerRes.ValidationStr != InAuthTicket)
				{
					UE_LOGF(LogSpectatorBeacon, Warning, "Auth ticket changing for spectator %ls.", *InSpectatorId.ToDebugString());
				}

				PlayerRes.ValidationStr = InAuthTicket;
				bFoundReservation = true;
				break;
			}
		}

		if (!bFoundReservation)
		{
			UE_LOGF(LogSpectatorBeacon, Warning, "Found no reservation for player %ls, while registering auth ticket.", *InSpectatorId.ToDebugString());
		}
	}
}


bool USpectatorBeaconState::RemovePlayer(const FUniqueNetIdRepl& PlayerId)
{
	UE_LOGF(LogSpectatorBeacon, VeryVerbose, "USpectatrBeaconState::RemovePlayer: %ls", *PlayerId.ToDebugString());
	bool bWasRemoved = false;

	for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bWasRemoved; ResIdx++)
	{
		FSpectatorReservation& Reservation = Reservations[ResIdx];
		FPlayerReservation& PlayerEntry = Reservation.Spectator;
		if (PlayerEntry.UniqueId == PlayerId)
		{
			bWasRemoved = true;

			// free up a consumed entry
			NumConsumedReservations--;
			if (UE_LOG_ACTIVE(LogSpectatorBeacon, Verbose))
			{
				UE_LOGF(LogSpectatorBeacon, Verbose, "USpectatorBeaconState::RemovePlayer: Player found in reservation with id %ls, setting NumConsumedReservations to %d", *Reservation.SpectatorId.ToString(), NumConsumedReservations);
				Reservation.Dump();
			}
			SanityCheckReservations(true);
			UE_LOGF(LogSpectatorBeacon, Verbose, "USpectatorBeaconState::RemovePlayer: Empty reservation found with spectator %ls, removing", *Reservation.SpectatorId.ToString());
			Reservations.RemoveAtSwap(ResIdx--);
		}
	}

	SanityCheckReservations(false);
	return bWasRemoved;
}

int32 USpectatorBeaconState::GetExistingReservation(const FUniqueNetIdRepl& Spectator) const
{
	int32 Result = INDEX_NONE;
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FSpectatorReservation& ReservationEntry = Reservations[ResIdx];
		if (ReservationEntry.SpectatorId == Spectator)
		{
			Result = ResIdx;
			break;
		}
	}

	return Result;
}

bool USpectatorBeaconState::UpdateMemberPlatform(const FUniqueNetIdRepl& Spectator, const FString& PlatformName)
{
	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		FSpectatorReservation& ReservationEntry = Reservations[ResIdx];
		FPlayerReservation& PlayerReservation = ReservationEntry.Spectator;
		if (PlayerReservation.UniqueId == Spectator)
		{
			if (!PlatformName.IsEmpty())
			{
				PlayerReservation.Platform = PlatformName;
			}
			// Return that member was updated
			return true;
		}
	}

	//Return that member was not updated
	return false;
}

bool USpectatorBeaconState::PlayerHasReservation(const FUniqueNetId& PlayerId) const
{
	bool bFound = false;

	for (int32 ResIdx = 0; ResIdx < Reservations.Num(); ResIdx++)
	{
		const FSpectatorReservation& ReservationEntry = Reservations[ResIdx];
		if (*ReservationEntry.Spectator.UniqueId == PlayerId)
		{
			bFound = true;
			break;
		}
	}

	return bFound;
}

bool USpectatorBeaconState::GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const
{
	bool bFound = false;
	OutValidation = FString();

	for (int32 ResIdx = 0; ResIdx < Reservations.Num() && !bFound; ResIdx++)
	{
		const FSpectatorReservation& ReservationEntry = Reservations[ResIdx];
		if (*ReservationEntry.Spectator.UniqueId == PlayerId)
		{
			OutValidation = ReservationEntry.Spectator.ValidationStr;
			bFound = true;
			break;
		}
	}

	return bFound;
}

void USpectatorBeaconState::DumpReservations() const
{
	FUniqueNetIdRepl NetId;
	FPlayerReservation PlayerRes;

	UE_LOGF(LogSpectatorBeacon, Display, "Session that reservations are for: %ls", *SessionName.ToString());
	UE_LOGF(LogSpectatorBeacon, Display, "Number total reservations: %d", MaxReservations);
	UE_LOGF(LogSpectatorBeacon, Display, "Number consumed reservations: %d", NumConsumedReservations);
	UE_LOGF(LogSpectatorBeacon, Display, "Number of spectator reservations: %d", Reservations.Num());

	// Log each spectator that has a reservation
	for (int32 ResIndex = 0; ResIndex < Reservations.Num(); ResIndex++)
	{
		NetId = Reservations[ResIndex].SpectatorId;
		UE_LOGF(LogSpectatorBeacon, Display, "\t Spectator: %ls", *NetId->ToDebugString());
		PlayerRes = Reservations[ResIndex].Spectator;
		UE_LOGF(LogSpectatorBeacon, Display, "\t  Member: %ls [%ls] Cross: %ls", *PlayerRes.UniqueId->ToString(), *PlayerRes.Platform, *LexToString(PlayerRes.bAllowCrossplay));
	}
	UE_LOGF(LogSpectatorBeacon, Display, "");
}

void USpectatorBeaconState::SanityCheckReservations(const bool bIgnoreEmptyReservations) const
{
#if !UE_BUILD_SHIPPING
	// Verify that each player is only in exactly one reservation
	TMap<FUniqueNetIdRepl, FUniqueNetIdRepl> PlayersInReservation;
	for (const FSpectatorReservation& Reservation : Reservations)
	{
		if (!Reservation.SpectatorId.IsValid())
		{
			DumpReservations();
			checkf(false, TEXT("Reservation does not have valid spectator!"));
		}
		const FPlayerReservation& PlayerReservation = Reservation.Spectator;
		if (PlayerReservation.UniqueId.IsValid())
		{
			const FUniqueNetIdRepl* const ExistingReservationLeader = PlayersInReservation.Find(PlayerReservation.UniqueId);
			if (ExistingReservationLeader != nullptr)
			{
				DumpReservations();
				checkf(false, TEXT("Player %s is in multiple reservations!"), *PlayerReservation.UniqueId.ToString());
			}
			PlayersInReservation.Add(PlayerReservation.UniqueId, Reservation.SpectatorId);
		}
	}
#endif // !UE_BUILD_SHIPPING
}

