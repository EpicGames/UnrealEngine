// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBeaconClient)

ATestBeaconClient::ATestBeaconClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

void ATestBeaconClient::OnFailure()
{
#if !UE_BUILD_SHIPPING
	UE_LOGF(LogBeacon, Verbose, "Test beacon connection failure, handling connection timeout.");
#endif
	Super::OnFailure();
}

/// @cond DOXYGEN_WARNINGS

void ATestBeaconClient::ClientPing_Implementation()
{
#if !UE_BUILD_SHIPPING
	UE_LOGF(LogBeacon, Log, "Ping");
	ServerPong();
#endif
}

bool ATestBeaconClient::ServerPong_Validate()
{
#if !UE_BUILD_SHIPPING
	return true;
#else
	return false;
#endif
}

void ATestBeaconClient::ServerPong_Implementation()
{
#if !UE_BUILD_SHIPPING
	UE_LOGF(LogBeacon, Log, "Pong");
	ClientPing();
#endif
}

/// @endcond
