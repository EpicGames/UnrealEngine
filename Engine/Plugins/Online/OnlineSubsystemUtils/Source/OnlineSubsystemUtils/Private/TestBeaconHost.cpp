// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestBeaconHost.h"
#include "TestBeaconClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestBeaconHost)

ATestBeaconHost::ATestBeaconHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	ClientBeaconActorClass = ATestBeaconClient::StaticClass();
	BeaconTypeName = ClientBeaconActorClass->GetName();
}

bool ATestBeaconHost::Init()
{
#if !UE_BUILD_SHIPPING
	UE_LOGF(LogBeacon, Verbose, "Init");
#endif
	return true;
}

void ATestBeaconHost::OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection)
{
#if !UE_BUILD_SHIPPING
	Super::OnClientConnected(NewClientActor, ClientConnection);

	ATestBeaconClient* BeaconClient = Cast<ATestBeaconClient>(NewClientActor);
	if (BeaconClient != NULL)
	{
		BeaconClient->ClientPing();
	}
#endif
}

AOnlineBeaconClient* ATestBeaconHost::SpawnBeaconActor(UNetConnection* ClientConnection)
{	
#if !UE_BUILD_SHIPPING
	return Super::SpawnBeaconActor(ClientConnection);
#else
	return NULL;
#endif
}

