// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "Algo/RemoveIf.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Misc/AutomationTest.h"
#include "Misc/Base64.h"
#include "Net/DataChannel.h"
#include "OnlineIdentityErrors.h"
#include "OnlineBeaconUnitTestClient.h"
#include "OnlineBeaconUnitTestHost.h"
#include "OnlineBeaconUnitTestHostObject.h"
#include "OnlineSubsystemTypes.h"
#include "Tests/OnlineBeaconUnitTestUtils.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestBasicHandshakeSuccess,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.BasicHandshakeSuccess",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestBasicHandshakeSuccess::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Successful handshake.
	// Disconnect initiated by client.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 2);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(HostUserBeaconClient->GetUniqueId() == *UserId);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client (not a control message).
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Check that client actor on the host cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient == nullptr);

		// Check that encryption delegates were not fired.
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.CallbackCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.CallbackCount == 0);
		// NetworkEncryptionFailure is always fired on client / host when no encryption data is provided.
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 2);
	}

	// Successful handshake.
	// Disconnect initiated by server.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Handshake complete, channel is open.

		// Disconnect the client.
		// The connection on the host side is now closed, but the hosts client object has not yet been notified.
		BeaconHostObject->DisconnectClient(HostUserBeaconClient);
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) != nullptr);

		// Check that client and host client object cleaned up.
		// Make sure host client object cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);

		// Make sure client cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);

		// Check that encryption delegates were not fired.
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.CallbackCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.CallbackCount == 0);
		// NetworkEncryptionFailure is always fired on client / host when no encryption data is provided.
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 2);
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestBasicHandshakeClientTimeout,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.BasicHandshakeClientTimeout",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestBasicHandshakeClientTimeout::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Timeout after client sends NMT_Hello
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from sending packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::RecvEnabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconClient, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}

	// Timeout after client sends NMT_BeaconJoin
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from sending packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::RecvEnabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconClient, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}

	// Timeout after client sends NMT_BeaconNetGUIDAck
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from sending packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::RecvEnabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconClient, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestBasicHandshakeHostTimeout,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.BasicHandshakeHostTimeout",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestBasicHandshakeHostTimeout::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Timeout after host sends NMT_BeaconWelcome
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from handling packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::Disabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconHost, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilTimeoutElapsed(*Prerequisites, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconHost, false));

		// Unblock the client from handling packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::Default));

		// Try to continue the handshake after the host has cleaned up the client state.
		// The host will not see the clients control message since it has closed the connection.
		// The client will be in an invalid state due to receiving the close packet.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}

	// Timeout after host sends NMT_BeaconAssignGUID
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from handling packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::Disabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconHost, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilTimeoutElapsed(*Prerequisites, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconHost, false));

		// Unblock the client from handling packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::Default));

		// Try to continue the handshake after the host has cleaned up the client state.
		// The host will not see the clients control message since it has closed the connection.
		// The client will be in an invalid state due to receiving the close packet.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestEncryptedHandshakeSuccess,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.EncryptedHandshakeSuccess",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestEncryptedHandshakeSuccess::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	// Check that encryption is allowed.
	UTEST_TRUE_EXPR(CVarNetAllowEncryption.GetValueOnGameThread() != 0);

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Todo: Remove external dependency on AESGCMHandlerComponent.
	Prerequisites->GetConfig().Encryption.bEnabled = true;
	Prerequisites->GetConfig().Encryption.NetDriverEncryptionComponentName = TEXT("AESGCMHandlerComponent");

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	// Valid encryption key.
	const FString EncryptionIdentitfier = TEXT("test");
	const FString Base64EncryptionKey = TEXT("IYaVIE38d6J9VfbPULuSMfn3/axig797U8DVJyRm1/c=");
	TArray<uint8> DecodedEncryptionKey;
	UTEST_TRUE_EXPR(FBase64::Decode(Base64EncryptionKey, DecodedEncryptionKey));

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Successful handshake.
	// OnReceivedNetworkEncryptionToken and OnReceivedNetworkEncryptionAck respond immediately.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.
		// 
		// Future work: The beacon encryption handshake doesn't currently work as the encryption structure intends.
		// 
		// Intended flow: The encryption data is fetched using net delegate
		// Notes:
		//  The encryption data is fetched using net delegate
		// Sequence:
		//  1. Client sends encryption identifier.
		//  2. Host finds the key for the identifier and sends NMT_EncryptionAck, NMT_BeaconWelcome. - calls FNetDelegates::OnReceivedNetworkEncryptionToken
		//  3. Client finds key for identifier and enables encryption. - calls FNetDelegates::OnReceivedNetworkEncryptionAck.
		//
		// Current flow:
		// Notes:
		//  The encryption data is stored on the OnlineBeaconClient class
		//  The intended sequence can be forced by clearing the encryption data on the OnlineBeaconClient object after sending the initial hello packet.
		// Sequence:
		//  1. Client sets encryption key on the connection.
		//  2. Client sends encryption identifier.
		//  3. Host finds the key for the identifier and sends NMT_EncryptionAck, NMT_BeaconWelcome. - calls FNetDelegates::OnReceivedNetworkEncryptionToken
		//  4. Client uses stored key for identifier and enables encryption. - FNetDelegates::OnReceivedNetworkEncryptionAck is bypassed.

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		// Host config.
		Prerequisites->GetConfig().Encryption.Host.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Host.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Host.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client config.
		Prerequisites->GetConfig().Encryption.Client.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Client.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Client.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Identifier = EncryptionIdentitfier;

		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		BeaconClient->SetEncryptionData(Prerequisites->GetConfig().Encryption.Client.EncryptionData);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_EncryptionAck, NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Cleanup successful handshake.
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
	}

	// Successful handshake.
	// Force the use of OnReceivedNetworkEncryptionAck.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		// Host config.
		Prerequisites->GetConfig().Encryption.Host.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Host.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Host.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client config.
		Prerequisites->GetConfig().Encryption.Client.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Client.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Client.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Identifier = EncryptionIdentitfier;

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		BeaconClient->SetEncryptionData(Prerequisites->GetConfig().Encryption.Client.EncryptionData);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Clearing the stored encryption info forces the use of OnReceivedNetworkEncryptionAck on the client.
		BeaconClient->SetEncryptionData(FEncryptionData());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.CallbackCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_EncryptionAck, NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.CallbackCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Cleanup successful handshake.
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
	}

	// Successful handshake.
	// Force the use of OnReceivedNetworkEncryptionAck.
	// Make both OnReceivedNetworkEncryptionToken and OnReceivedNetworkEncryptionAck delay callback by one frame.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		// Host config.
		Prerequisites->GetConfig().Encryption.Host.bDelayDelegate = true;
		Prerequisites->GetConfig().Encryption.Host.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Host.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client config.
		Prerequisites->GetConfig().Encryption.Client.bDelayDelegate = true;
		Prerequisites->GetConfig().Encryption.Client.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Client.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Identifier = EncryptionIdentitfier;

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		BeaconClient->SetEncryptionData(Prerequisites->GetConfig().Encryption.Client.EncryptionData);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);

		// Clearing the stored encryption info forces the use of OnReceivedNetworkEncryptionAck on the client.
		BeaconClient->SetEncryptionData(FEncryptionData());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.CallbackCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Tick to allow the host to handle OnReceivedNetworkEncryptionToken.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.CallbackCount == 1);

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_EncryptionAck, NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.CallbackCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		// Tick to allow the client to handle OnReceivedNetworkEncryptionAck.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.CallbackCount == 1);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Cleanup successful handshake.
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
	}

	// Successful handshake.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns AllowConnection.
	// 4. Host allows the connection.
	// 4a. BeaconEncryptionFailureAction returns AllowConnection.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.
		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::AllowConnection;

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 2);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 0);
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Cleanup successful handshake.
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestEncryptedHandshakeFailed,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.EncryptedHandshakeFailed",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestEncryptedHandshakeFailed::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	// Check that encryption is allowed.
	UTEST_TRUE_EXPR(CVarNetAllowEncryption.GetValueOnGameThread() != 0);

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Todo: Remove external dependency on AESGCMHandlerComponent.
	Prerequisites->GetConfig().Encryption.bEnabled = true;
	Prerequisites->GetConfig().Encryption.NetDriverEncryptionComponentName = TEXT("AESGCMHandlerComponent");

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	// Valid encryption key.
	const FString EncryptionIdentitfier = TEXT("test");
	const FString Base64EncryptionKey = TEXT("IYaVIE38d6J9VfbPULuSMfn3/axig797U8DVJyRm1/c=");
	TArray<uint8> DecodedEncryptionKey;
	UTEST_TRUE_EXPR(FBase64::Decode(Base64EncryptionKey, DecodedEncryptionKey));

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Failed handshake. OnReceivedNetworkEncryptionToken.
	// 1. Netdriver requires encryption
	// 2. Encryption data provided.
	// 3. Client initializes connection.
	// 4. Host fails the connection.
	// 4a. FNetDelegates::OnReceivedNetworkEncryptionToken returns EEncryptionResponse::Failure
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		// Host config.
		Prerequisites->GetConfig().Encryption.Host.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Host.Response = EEncryptionResponse::Failure;
		Prerequisites->GetConfig().Encryption.Host.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client config.
		Prerequisites->GetConfig().Encryption.Client.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Client.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Client.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		BeaconClient->SetEncryptionData(Prerequisites->GetConfig().Encryption.Client.EncryptionData);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
	}

	// Failed handshake. OnReceivedNetworkEncryptionAck.
	// 1. Netdriver requires encryption
	// 2. Encryption data provided.
	// 3. Client initializes connection.
	// 4. Host allows the connection.
	// 5. Client fails the connection.
	// 5a. FNetDelegates::OnReceivedNetworkEncryptionAck returns EEncryptionResponse::Failure
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		// Host config.
		Prerequisites->GetConfig().Encryption.Host.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Host.Response = EEncryptionResponse::Success;
		Prerequisites->GetConfig().Encryption.Host.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Host.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client config.
		Prerequisites->GetConfig().Encryption.Client.bDelayDelegate = false;
		Prerequisites->GetConfig().Encryption.Client.Response = EEncryptionResponse::Failure;
		Prerequisites->GetConfig().Encryption.Client.ErrorMsg.Empty();
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Key = DecodedEncryptionKey;
		Prerequisites->GetConfig().Encryption.Client.EncryptionData.Identifier = EncryptionIdentitfier;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		BeaconClient->SetEncryptionData(Prerequisites->GetConfig().Encryption.Client.EncryptionData);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);

		// Clearing the stored encryption info forces the use of OnReceivedNetworkEncryptionAck on the client.
		BeaconClient->SetEncryptionData(FEncryptionData());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.CallbackCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetStats().Encryption = BeaconUnitTest::FTestStats::FEncryption();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_EncryptionAck }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionAck.CallbackCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 0);
	}

	// Failed handshake. BeaconEncryptionFailureAction.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns AllowConnection.
	// 4. Host disallows the connection.
	// 4a. BeaconEncryptionFailureAction returns Default.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::AllowConnection;

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		
		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);

		// Handshake testing.

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::Default;

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 2);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
	}

	// Failed handshake. BeaconEncryptionFailureAction.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns AllowConnection.
	// 4. Host disallows the connection.
	// 4a. BeaconEncryptionFailureAction returns RejectConnection.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::AllowConnection;

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);

		// Handshake testing.

		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 2);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
	}

	// Failed handshake. BeaconEncryptionFailureAction.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns Default.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.
		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::Default;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);
	}

	// Failed handshake. BeaconEncryptionFailureAction.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns AllowConnection.
	// 4. BeaconEncryptionFailureAction is unbound.
	// 5. Host disallows the connection.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.
		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::AllowConnection;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);

		Prerequisites->UnbindNetEncryptionDelegates();
		ON_SCOPE_EXIT{ Prerequisites->BindNetEncryptionDelegates(); };

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
	}

	// Failed handshake. BeaconEncryptionFailureAction.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns RejectConnection.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.
		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::RejectConnection;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}

	// Failed handshake. BeaconEncryptionFailureAction.
	// 1. Netdriver requires encryption
	// 2. No encryption data provided.
	// 3. Client initializes connection.
	// 3a. BeaconEncryptionFailureAction returns AllowConnection.
	// 4. Host disallows the connection.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Setup encryption.
		Prerequisites->GetConfig().Encryption.FailureAction = EEncryptionFailureAction::AllowConnection;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Wait until connection is initialized
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnectionInitialized(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);

		Prerequisites->UnbindNetEncryptionDelegates();
		ON_SCOPE_EXIT{ Prerequisites->BindNetEncryptionDelegates(); };

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionToken.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Encryption.NetworkEncryptionFailure.InvokeCount == 1);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestAuthenticatedHandshakeSuccess,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.AuthenticatedHandshakeSuccess",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestAuthenticatedHandshakeSuccess::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().Auth.bEnabled = true;
	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Host handshake calls StartVerifyAuthentication (callback signature with LoginOptions) and expects callback delegate to be called to complete authentication.
	// Test callback delegate fired with no delay is working.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Auth.Result = FOnlineError::Success();

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Login }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(HostUserBeaconClient->GetUniqueId() == *UserId);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Check that client actor on the host cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient == nullptr);
	}

	// Host handshake calls StartVerifyAuthentication (callback signature with LoginOptions) and expects callback delegate to be called to complete authentication.
	// Test callback delegate fired with a single frame delay is working.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Auth.bEnabled = true;
		Prerequisites->GetConfig().Auth.bDelayDelegate = true;
		Prerequisites->GetConfig().Auth.Result = FOnlineError::Success();

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Login }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(HostUserBeaconClient->GetUniqueId() == *UserId);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Check that client actor on the host cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient == nullptr);
	}

	// Auth and verify that a user supplied VerifyJoinForBeaconType method allows the connection to proceed.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Auth.bEnabled = true;
		Prerequisites->GetConfig().Auth.Result = FOnlineError::Success();

		Prerequisites->GetConfig().Auth.Verify.bEnabled = true;
		Prerequisites->GetConfig().Auth.Verify.bResult = true;

		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Login }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconAssignGUID }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconNetGUIDAck }));
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		UTEST_TRUE_EXPR(HostUserBeaconClient->GetUniqueId() == *UserId);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Receive OnConnected RPC on the client.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		// Handshake complete, channel is open.

		// Disconnect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;

		// Check that client actor on the host cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1; }));
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient == nullptr);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestAuthenticatedHandshakeFailure,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.AuthenticatedHandshakeFailure",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestAuthenticatedHandshakeFailure::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().Auth.bEnabled = true;
	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Host handshake calls StartVerifyAuthentication (callback signature with LoginOptions) and expects callback delegate to be called to complete authentication.
	// Test callback delegate fired with no delay is working.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Auth.bEnabled = true;
		Prerequisites->GetConfig().Auth.Result = OnlineIdentity::Errors::InvalidCreds();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Login }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
	}

	// Host handshake calls StartVerifyAuthentication (callback signature with LoginOptions) and expects callback delegate to be called to complete authentication.
	// Test callback delegate fired with a single frame delay is working.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Auth.bEnabled = true;
		Prerequisites->GetConfig().Auth.bDelayDelegate = true;
		Prerequisites->GetConfig().Auth.Result = OnlineIdentity::Errors::InvalidCreds();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Login }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
	}

	// Auth and verify that a user supplied VerifyJoinForBeaconType method does not allow the connection to proceed.
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		Prerequisites->GetConfig().Auth.bEnabled = true;
		Prerequisites->GetConfig().Auth.Result = FOnlineError::Success();

		Prerequisites->GetConfig().Auth.Verify.bEnabled = true;
		Prerequisites->GetConfig().Auth.Verify.bResult = false;

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Login }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		BeaconClientNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Netspeed, NMT_BeaconJoin }));
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Failure }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestAuthenticatedHandshakeClientTimeout,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.AuthenticatedHandshakeClientTimeout",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestAuthenticatedHandshakeClientTimeout::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().Auth.bEnabled = true;
	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Timeout after client sends NMT_Login
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from sending packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::RecvEnabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconClient, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilDisconnected(*Prerequisites, BeaconClient, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestAuthenticatedHandshakeHostTimeout,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.AuthenticatedHandshakeHostTimeout",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestAuthenticatedHandshakeHostTimeout::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().Auth.bEnabled = true;
	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Host setup.
	TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
	TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
	ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
	ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
	BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
	BeaconHost->RegisterHost(BeaconHostObject);
	UTEST_TRUE_EXPR(BeaconHost->InitHost());
	BeaconHost->PauseBeaconRequests(false);

	TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
	UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
	UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

	// Timeout after host sends NMT_Challenge
	{
		BeaconHostNetStats->ReceivedControlMessages.Empty();
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages.IsEmpty());
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Hello }));
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Block the client from handling packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::Disabled));

		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconHost, true));
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilTimeoutElapsed(*Prerequisites, BeaconUnitTest::ETickFlags::SleepTickTime));
		UTEST_TRUE_EXPR(BeaconUnitTest::SetTimeoutsEnabled(BeaconHost, false));

		// Unblock the client from handling packets.
		UTEST_TRUE_EXPR(BeaconUnitTest::SetSocketFlags(BeaconClient, BeaconUnitTest::ESocketFlags::Default));

		// Try to continue the handshake after the host has cleaned up the client state.
		// The host will not see the clients control message since it has closed the connection.
		// The client will be in an invalid state due to receiving the close packet.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_Challenge }));
		UTEST_TRUE_EXPR(BeaconHostNetStats->ReceivedControlMessages.IsEmpty());
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestGarbageCollection,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.GarbageCollection",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestGarbageCollection::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Successful handshake.
	// Garbage collect host beacon.
	{
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Host setup.
		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;
		TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
		TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
		ON_SCOPE_EXIT{ if (BeaconHost) BeaconHost->DestroyBeacon(); };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
		BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
		BeaconHost->RegisterHost(BeaconHostObject);
		UTEST_TRUE_EXPR(BeaconHost->InitHost());
		BeaconHost->PauseBeaconRequests(false);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) { BeaconClient->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Handshake complete, channel is open.

		// Garbage collect the host.
		BeaconHost->DestroyBeacon();
		BeaconHost = nullptr;
		BeaconHostObject = nullptr;
		GEngine->ForceGarbageCollection(true);
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);

		// Check that client and host client object cleaned up.
		// Make sure host client object cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);

		// Make sure client cleaned up.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);
	}

	// Successful handshake.
	// Garbage collect client beacon.
	{
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Host setup.
		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;
		TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
		TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
		ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
		BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
		BeaconHost->RegisterHost(BeaconHostObject);
		UTEST_TRUE_EXPR(BeaconHost->InitHost());
		BeaconHost->PauseBeaconRequests(false);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) BeaconClient->DestroyBeacon(); };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilConnected(*Prerequisites, BeaconClient));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Handshake complete, channel is open.

		// Garbage collect the client.
		BeaconClient->DestroyBeacon();
		BeaconClient = nullptr;
		UTEST_TRUE_EXPR(GEngine != nullptr);
		GEngine->ForceGarbageCollection(true);

		// Tick for client connection to send close.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		// Tick for host connection to see client close.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		// Tick for host object to be notified of closure.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
	}

	// Garbage collect host beacon during handshake.
	{
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Host setup.
		TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
		TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
		ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
		BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
		BeaconHost->RegisterHost(BeaconHostObject);
		UTEST_TRUE_EXPR(BeaconHost->InitHost());
		BeaconHost->PauseBeaconRequests(false);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) BeaconClient->DestroyBeacon(); };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClient->GetNetConnection() != nullptr);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Handshake testing.

		// Wait until the client receives a control message from the host beacon.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntilControlMessageReceived(*Prerequisites, *BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);
		UTEST_TRUE_EXPR(BeaconClientNetStats->ReceivedControlMessages == TArray<uint8>({ NMT_BeaconWelcome }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Garbage collect the host.
		BeaconHost->DestroyBeacon();
		BeaconHost = nullptr;
		UTEST_TRUE_EXPR(GEngine != nullptr);
		GEngine->ForceGarbageCollection(true);

		// Tick for host connection to send close.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		// Tick for client connection to see client close.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
	}

	// Garbage collect client beacon during RPC callback.
	{
		// Reset config and stats.
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Host setup.
		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;
		TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
		TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
		ON_SCOPE_EXIT{ if (BeaconHost) { BeaconHost->DestroyBeacon(); } };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
		BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
		BeaconHost->RegisterHost(BeaconHostObject);
		UTEST_TRUE_EXPR(BeaconHost->InitHost());
		BeaconHost->PauseBeaconRequests(false);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconHostNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconHost, &BeaconHostNetStats));
		UTEST_TRUE_EXPR(BeaconHostNetStats.IsValid());

		// Client setup.

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) BeaconClient->DestroyBeacon(); };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));

		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid);
		UTEST_TRUE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(BeaconClient->GetConnectionState() == EBeaconConnectionState::Pending);

		TSharedPtr<BeaconUnitTest::FNetworkStats> BeaconClientNetStats;
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeaconNetDriver(*Prerequisites, BeaconClient, &BeaconClientNetStats));
		UTEST_TRUE_EXPR(BeaconClientNetStats.IsValid());

		// Setup test - garbage collect the client during its OnConnected RPC from the server.
		Prerequisites->GetConfig().Client.OnConnected.Callback = [&BeaconClient]()
		{
			// Garbage collect the client.
			BeaconClient->DestroyBeacon();
			BeaconClient = nullptr;
		};
		ON_SCOPE_EXIT{ Prerequisites->GetConfig().Client.OnConnected.Callback.Reset(); };

		// Handshake testing.

		UTEST_TRUE_EXPR(BeaconUnitTest::TickUntil(*Prerequisites, [&Prerequisites](){ return Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1; }));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
		HostUserBeaconClient = BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId);
		UTEST_TRUE_EXPR(HostUserBeaconClient != nullptr);
		BeaconClientNetStats->ReceivedControlMessages.Empty();
		BeaconHostNetStats->ReceivedControlMessages.Empty();

		// Handshake complete, channel is open.

		// Beacon destroyed during RPC - garbage collect now.
		UTEST_TRUE_EXPR(GEngine != nullptr);
		GEngine->ForceGarbageCollection(true);

		// Tick for host connection to see client close and for delayed GC to run.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));

		// Tick for host object to be notified of closure.
		UTEST_TRUE_EXPR(BeaconUnitTest::TickOnce(*Prerequisites));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 1);
		UTEST_TRUE_EXPR(BeaconUnitTest::GetBeaconClientForUser(BeaconHostObject, UserId) == nullptr);
	}
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOnlineBeaconTestInvalidSocket,
	"System.Engine.Online.OnlineSubsystemUtils.OnlineBeacon.InvalidSocket",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOnlineBeaconTestInvalidSocket::RunTest(const FString& Parameters)
{
	TSharedPtr<BeaconUnitTest::FTestPrerequisites> Prerequisites = BeaconUnitTest::FTestPrerequisites::TryCreate();
	UTEST_TRUE_EXPR(Prerequisites.IsValid());

	Prerequisites->GetConfig().NetDriver.ServerListenPort = 9999;

	// Fail all netdriver initializations.
	Prerequisites->GetConfig().NetDriver.bFailInit = true;

	// Snapshot config to restore before running a test section.
	const BeaconUnitTest::FTestConfig BaseConfig = Prerequisites->GetConfig();

	FUniqueNetIdStringRef UserId = FUniqueNetIdString::Create(TEXT("User"), TEXT("UnitTest"));

	// Beacon host fails init.
	{
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		// Host setup.
		TObjectPtr<AOnlineBeaconClient> HostUserBeaconClient;
		TObjectPtr<AOnlineBeaconUnitTestHostObject> BeaconHostObject = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHostObject>(AOnlineBeaconUnitTestHostObject::StaticClass());
		TObjectPtr<AOnlineBeaconUnitTestHost> BeaconHost = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestHost>(AOnlineBeaconUnitTestHost::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconHostObject) { BeaconHostObject->Destroy(); } };
		ON_SCOPE_EXIT{ if (BeaconHost) BeaconHost->DestroyBeacon(); };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconHost));
		BeaconHost->ListenPort = Prerequisites->GetConfig().NetDriver.ServerListenPort;
		BeaconHost->RegisterHost(BeaconHostObject);
		UTEST_FALSE_EXPR(BeaconHost->InitHost());
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
	}

	// Beacon client fails init.
	{
		Prerequisites->GetConfig() = BaseConfig;
		Prerequisites->GetStats() = BeaconUnitTest::FTestStats();

		TObjectPtr<AOnlineBeaconUnitTestClient> BeaconClient = Prerequisites->GetWorld()->SpawnActor<AOnlineBeaconUnitTestClient>(AOnlineBeaconUnitTestClient::StaticClass());
		ON_SCOPE_EXIT{ if (BeaconClient) BeaconClient->DestroyBeacon(); };
		UTEST_TRUE_EXPR(BeaconUnitTest::ConfigureBeacon(*Prerequisites, BeaconClient));
		UTEST_FALSE_EXPR(BeaconUnitTest::InitClientForUser(*Prerequisites, BeaconClient, UserId));
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Client.OnFailure.InvokeCount == 1);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().Host.OnFailure.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.OnClientConnected.InvokeCount == 0);
		UTEST_TRUE_EXPR(Prerequisites->GetStats().HostObject.NotifyClientDisconnected.InvokeCount == 0);
	}

	return true;
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
