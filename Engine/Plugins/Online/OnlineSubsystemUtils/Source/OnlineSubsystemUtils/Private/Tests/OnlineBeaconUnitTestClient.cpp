// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconUnitTestClient.h"
#include "Tests/OnlineBeaconUnitTestUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconUnitTestClient)

AOnlineBeaconUnitTestClient::AOnlineBeaconUnitTestClient(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

void AOnlineBeaconUnitTestClient::OnConnected()
{
	if (BeaconUnitTest::FTestStats* TestStats = BeaconUnitTest::FTestPrerequisites::GetActiveTestStats())
	{
		++TestStats->Client.OnConnected.InvokeCount;
	}

	Super::OnConnected();


	if (const BeaconUnitTest::FTestConfig* TestConfig = BeaconUnitTest::FTestPrerequisites::GetActiveTestConfig())
	{
		if (TestConfig->Client.OnConnected.Callback)
		{
			TestConfig->Client.OnConnected.Callback();
		}
	}
}

void AOnlineBeaconUnitTestClient::OnFailure()
{
	if (BeaconUnitTest::FTestStats* TestStats = BeaconUnitTest::FTestPrerequisites::GetActiveTestStats())
	{
		++TestStats->Client.OnFailure.InvokeCount;
	}

	Super::OnFailure();
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
