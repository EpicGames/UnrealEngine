// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconClient.h"

#include "OnlineBeaconUnitTestClient.generated.h"

UCLASS(transient, notplaceable)
class AOnlineBeaconUnitTestClient : public AOnlineBeaconClient
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

	//~ Begin AOnlineBeaconClient Interface
	virtual void OnConnected() override;
	//~ End AOnlineBeaconClient Interface

	//~ Begin OnlineBeacon Interface
	virtual void OnFailure() override;
	//~ End OnlineBeacon Interface

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
};
