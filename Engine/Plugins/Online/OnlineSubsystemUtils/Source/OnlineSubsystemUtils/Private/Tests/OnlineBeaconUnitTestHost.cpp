// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconUnitTestHost.h"

#include "Online/CoreOnline.h"
#include "Tests/OnlineBeaconUnitTestUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconUnitTestHost)

AOnlineBeaconUnitTestHost::AOnlineBeaconUnitTestHost(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

bool AOnlineBeaconUnitTestHost::StartVerifyAuthentication(const FUniqueNetId& PlayerId, const FString& LoginOptions, const FString& AuthenticationToken, const FOnAuthenticationVerificationCompleteDelegate& OnComplete)
{
	const BeaconUnitTest::FTestConfig* TestConfig = BeaconUnitTest::FTestPrerequisites::GetActiveTestConfig();
	if (TestConfig == nullptr)
	{
		return false;
	}

	if (!TestConfig->Auth.bEnabled)
	{
		return Super::StartVerifyAuthentication(PlayerId, LoginOptions, AuthenticationToken, OnComplete);
	}

	if (TestConfig->Auth.bDelayDelegate)
	{
		return BeaconUnitTest::SetTimerForNextFrame(GetWorld(), GFrameCounter, [this, OnComplete]()
		{
			const BeaconUnitTest::FTestConfig* TestConfig = BeaconUnitTest::FTestPrerequisites::GetActiveTestConfig();
			OnComplete.ExecuteIfBound(TestConfig ? TestConfig->Auth.Result : FOnlineError());
		});
	}
	else
	{
		OnComplete.ExecuteIfBound(TestConfig->Auth.Result);
		return true;
	}
}

bool AOnlineBeaconUnitTestHost::VerifyJoinForBeaconType(const FUniqueNetId& PlayerId, const FString& BeaconType)
{
	const BeaconUnitTest::FTestConfig* TestConfig = BeaconUnitTest::FTestPrerequisites::GetActiveTestConfig();
	if (TestConfig == nullptr)
	{
		return false;
	}

	if (!TestConfig->Auth.Verify.bEnabled)
	{
		return Super::VerifyJoinForBeaconType(PlayerId, BeaconType);
	}

	return TestConfig->Auth.Verify.bResult;
}

void AOnlineBeaconUnitTestHost::OnFailure()
{
	if (BeaconUnitTest::FTestStats* TestStats = BeaconUnitTest::FTestPrerequisites::GetActiveTestStats())
	{
		++TestStats->Host.OnFailure.InvokeCount;
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Super::OnFailure();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
