// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHost.h"

#include "OnlineBeaconUnitTestHost.generated.h"

UCLASS(transient, notplaceable)
class AOnlineBeaconUnitTestHost : public AOnlineBeaconHost
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

public:

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

	//~ Begin AOnlineBeaconHost Interface
	virtual bool StartVerifyAuthentication(const FUniqueNetId& PlayerId, const FString& LoginOptions, const FString& AuthenticationToken, const FOnAuthenticationVerificationCompleteDelegate& OnComplete) override;
	virtual bool VerifyJoinForBeaconType(const FUniqueNetId& PlayerId, const FString& BeaconType) override;
	//~ End AOnlineBeaconHost Interface

	//~ Begin OnlineBeacon Interface
	virtual void OnFailure() override;
	//~ End OnlineBeacon Interface

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
};
