// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconUnitTestHostObject.h"

#include "Engine/NetConnection.h"
#include "OnlineBeaconUnitTestClient.h"
#include "Tests/OnlineBeaconUnitTestUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconUnitTestHostObject)

AOnlineBeaconUnitTestHostObject::AOnlineBeaconUnitTestHostObject(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	ClientBeaconActorClass = AOnlineBeaconUnitTestClient::StaticClass();
	BeaconTypeName = ClientBeaconActorClass->GetName();
}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

void AOnlineBeaconUnitTestHostObject::OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection)
{
	if (BeaconUnitTest::FTestStats* TestStats = BeaconUnitTest::FTestPrerequisites::GetActiveTestStats())
	{
		++TestStats->HostObject.OnClientConnected.InvokeCount;
	}

	Super::OnClientConnected(NewClientActor, ClientConnection);
}

void AOnlineBeaconUnitTestHostObject::NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor)
{
	if (BeaconUnitTest::FTestStats* TestStats = BeaconUnitTest::FTestPrerequisites::GetActiveTestStats())
	{
		++TestStats->HostObject.NotifyClientDisconnected.InvokeCount;
	}

	Super::NotifyClientDisconnected(LeavingClientActor);
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
