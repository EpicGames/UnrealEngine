// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconHostObject.h"

#include "OnlineBeaconUnitTestHostObject.generated.h"

UCLASS(transient, notplaceable)
class AOnlineBeaconUnitTestHostObject : public AOnlineBeaconHostObject
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

public:
	//~ Begin AOnlineBeaconHostObject Interface
	virtual void OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection) override;
	virtual void NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor) override;
	//~ End AOnlineBeaconHostObject Interface

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
};
