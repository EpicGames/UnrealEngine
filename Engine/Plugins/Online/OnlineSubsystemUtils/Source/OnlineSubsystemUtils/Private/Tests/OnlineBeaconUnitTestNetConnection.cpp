// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconUnitTestNetConnection.h"

#include "Tests/OnlineBeaconUnitTestUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconUnitTestNetConnection)

UOnlineBeaconUnitTestNetConnection::UOnlineBeaconUnitTestNetConnection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

void UOnlineBeaconUnitTestNetConnection::InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	DisableAddressResolution();

	Super::InitBase(InDriver, InSocket, InURL, InState, InMaxPacket, InPacketOverhead);
}

float UOnlineBeaconUnitTestNetConnection::GetTimeoutValue()
{
	check(Driver);
	if (Driver->bNoTimeouts)
	{
		return UE_MAX_FLT;
	}

	const bool bUseShortTimeout = (GetConnectionState() != USOCK_Pending) && (bPendingDestroy || (OwningActor && OwningActor->UseShortConnectTimeout()));
	return bUseShortTimeout ? Driver->ConnectionTimeout : Driver->InitialConnectTimeout;
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
