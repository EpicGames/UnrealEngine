// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IpConnection.h"

#include "OnlineBeaconUnitTestNetConnection.generated.h"

UCLASS(Transient, notplaceable, Config=Engine)
class UOnlineBeaconUnitTestNetConnection : public UIpConnection
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

public:
//~ Begin NetConnection Interface
	virtual void InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead) override;
	virtual float GetTimeoutValue() override;
//~ End NetConnection Interface

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
};
