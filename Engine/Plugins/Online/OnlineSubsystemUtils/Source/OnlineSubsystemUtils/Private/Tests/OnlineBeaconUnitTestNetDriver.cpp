// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconUnitTestNetDriver.h"

#include "OnlineBeaconUnitTestNetConnection.h"
#include "OnlineBeaconUnitTestSocketSubsystem.h"
#include "Tests/OnlineBeaconUnitTestUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBeaconUnitTestNetDriver)

UOnlineBeaconUnitTestNetDriver::UOnlineBeaconUnitTestNetDriver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

void UOnlineBeaconUnitTestNetDriver::PostInitProperties()
{
	Super::PostInitProperties();
	NetConnectionClassName = UOnlineBeaconUnitTestNetConnection::StaticClass()->GetPathName();
}

bool UOnlineBeaconUnitTestNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	if (Super::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		const BeaconUnitTest::FTestConfig* TestConfig = BeaconUnitTest::FTestPrerequisites::GetActiveTestConfig();
		if (TestConfig == nullptr || TestConfig->NetDriver.bFailInit)
		{
			return false;
		}
	}

	return true;
}

void UOnlineBeaconUnitTestNetDriver::TickDispatch(float DeltaTime)
{
	Super::TickDispatch(DeltaTime);
}

ISocketSubsystem* UOnlineBeaconUnitTestNetDriver::GetSocketSubsystem()
{
	return ISocketSubsystem::Get(BeaconUnitTest::SocketSubsystemName);
}

bool UOnlineBeaconUnitTestNetDriver::IsEncryptionRequired() const
{
	const BeaconUnitTest::FTestConfig* TestConfig = BeaconUnitTest::FTestPrerequisites::GetActiveTestConfig();
	return TestConfig ? TestConfig->Encryption.bEnabled : false;
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
