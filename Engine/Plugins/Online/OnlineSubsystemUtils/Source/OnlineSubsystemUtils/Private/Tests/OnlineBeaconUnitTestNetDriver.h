// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IpNetDriver.h"

#include "OnlineBeaconUnitTestNetDriver.generated.h"

UCLASS(Transient, notplaceable)
class UOnlineBeaconUnitTestNetDriver : public UIpNetDriver
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

public:
	//~ Begin UObject Interface.
	virtual void PostInitProperties() override;
	//~ End UObject Interface.

	//~ Begin UNetDriver Interface.
	virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	virtual void TickDispatch(float DeltaTime) override;
	virtual ISocketSubsystem* GetSocketSubsystem() override;
	virtual bool IsEncryptionRequired() const override;
	//~ End UNetDriver Interface

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
};
