// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineBeaconUnitTestSocketSubsystem.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "EngineLogs.h"
#include "Modules/ModuleManager.h"
#include "SocketSubsystemModule.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogUnitTestSocketSubsystem, Log, All);

void FInternetAddrBeaconUnitTest::SetIp(uint32 InAddr)
{
}


void FInternetAddrBeaconUnitTest::SetIp(const TCHAR* InAddr, bool& bIsValid)
{
}

void FInternetAddrBeaconUnitTest::GetIp(uint32& OutAddr) const
{
	OutAddr = 0;
}

void FInternetAddrBeaconUnitTest::SetPort(int32 InPort)
{
	Port = InPort;
}


int32 FInternetAddrBeaconUnitTest::GetPort() const
{
	return Port;
}

void FInternetAddrBeaconUnitTest::SetRawIp(const TArray<uint8>& RawAddr)
{
}

TArray<uint8> FInternetAddrBeaconUnitTest::GetRawIp() const
{
	return TArray<uint8>();
}

void FInternetAddrBeaconUnitTest::SetAnyAddress()
{
}

void FInternetAddrBeaconUnitTest::SetBroadcastAddress()
{
}

void FInternetAddrBeaconUnitTest::SetLoopbackAddress()
{
}

FString FInternetAddrBeaconUnitTest::ToString(bool bAppendPort) const
{
	return FString::Printf(TEXT("OnlineBeaconUnitTestINetAddr:%d"), GetPort());
}

bool FInternetAddrBeaconUnitTest::operator==(const FInternetAddr& Other) const
{
	return Other.GetPort() == GetPort();
}

uint32 FInternetAddrBeaconUnitTest::GetTypeHash() const
{
	return ::GetTypeHash(Port);
}

bool FInternetAddrBeaconUnitTest::IsValid() const
{
	return Port != InvalidPort;
}

TSharedRef<FInternetAddr> FInternetAddrBeaconUnitTest::Clone() const
{
	TSharedRef<FInternetAddr> OutClone = MakeShared<FInternetAddrBeaconUnitTest>();
	OutClone->SetPort(GetPort());
	return OutClone;
}

FSocketBeaconUnitTest::FSocketBeaconUnitTest(ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol, const TSharedRef<FOnlineBeaconUnitTestSocketSubsystem>& InSubsystem)
	: FSocket(InSocketType, InSocketDescription, InSocketProtocol)
	, WeakSocketSubsystem(InSubsystem)
{
}

FSocketBeaconUnitTest::~FSocketBeaconUnitTest()
{
}

bool FSocketBeaconUnitTest::Shutdown(ESocketShutdownMode Mode)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::Close()
{
	return true;
}

bool FSocketBeaconUnitTest::Bind(const FInternetAddr& Addr)
{
	TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin();
	if (!SocketSubsystem.IsValid())
	{
		return false;
	}

	const FInternetAddrBeaconUnitTest& UnitTestAddr = static_cast<const FInternetAddrBeaconUnitTest&>(Addr);
	LocalAddress = SocketSubsystem->BindSocket(this, UnitTestAddr);
	return LocalAddress.IsValid();
}

bool FSocketBeaconUnitTest::Connect(const FInternetAddr& Addr)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::Listen(int32 MaxBacklog)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::HasPendingData(uint32& PendingDataSize)
{
	if (FUnitTestNetworkPacket* Packet = ReceiveBuffer.Peek())
	{
		PendingDataSize = Packet->Data.Num();
		return true;
	}

	return false;
}

class FSocket* FSocketBeaconUnitTest::Accept(const FString& InSocketDescription)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return nullptr;
}

class FSocket* FSocketBeaconUnitTest::Accept(FInternetAddr& OutAddr, const FString& InSocketDescription)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return nullptr;
}

bool FSocketBeaconUnitTest::SendTo(const uint8* Data, int32 Count, int32& OutBytesSent, const FInternetAddr& Destination)
{
	OutBytesSent = 0;

	TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin();
	if (!SocketSubsystem.IsValid())
	{
		return false;
	}

	if (Count > FUnitTestNetworkPacket::MaxPacketSize)
	{
		UE_LOGF(LogUnitTestSocketSubsystem, Warning, "[%s] Unable to send data, data over maximum size. Amount=[%d/%d] DestinationAddress = (%ls)", __FUNCTION__, Count, FUnitTestNetworkPacket::MaxPacketSize, *Destination.ToString(true));
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EMSGSIZE);
		return false;
	}

	if (Count < 0)
	{
		UE_LOGF(LogUnitTestSocketSubsystem, Warning, "[%s] Unable to send data, data invalid. Amount=[%d/%d] DestinationAddress = (%ls)", __FUNCTION__, Count, FUnitTestNetworkPacket::MaxPacketSize, *Destination.ToString(true));
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	if (!Destination.IsValid())
	{
		UE_LOGF(LogUnitTestSocketSubsystem, Warning, "[%s] Unable to send data, invalid destination address. DestinationAddress = (%ls)", __FUNCTION__, *Destination.ToString(true));
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EADDRNOTAVAIL);
		return false;
	}

	const FInternetAddrBeaconUnitTest& DestAddress = static_cast<const FInternetAddrBeaconUnitTest&>(Destination);
	FUnitTestNetworkPacket SendPacket;
	SendPacket.FromAddr = LocalAddress;
	SendPacket.ToAddr = DestAddress;
	SendPacket.Data.SetNum(Count);
	FMemory::Memcpy(SendPacket.Data.GetData(), Data, Count);
	OutBytesSent = Count;

	UE_LOGF(LogUnitTestSocketSubsystem, VeryVerbose, "[%s] Outbound message queued. Socket: %p, FromPort: %d, ToPort: %d", __FUNCTION__, this, LocalAddress.GetPort(), Destination.GetPort());

	SendBuffer.Enqueue(MoveTemp(SendPacket));
	return true;
}

bool FSocketBeaconUnitTest::Send(const uint8* Data, int32 Count, int32& BytesSent)
{
	BytesSent = 0;
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags)
{
	BytesRead = 0;

	TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin();
	if (!SocketSubsystem.IsValid())
	{
		return false;
	}

	if (BufferSize < 0)
	{
		UE_LOGF(LogUnitTestSocketSubsystem, Error, "[%s] Unable to receive data, receiving buffer was invalid. BufferSize = (%d)", __FUNCTION__, BufferSize);

		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	if (Flags != ESocketReceiveFlags::None)
	{
		// We do not support peaking / blocking until a packet comes
		UE_LOGF(LogUnitTestSocketSubsystem, Error, "[%s] Socket receive flags (%d) are not supported", __FUNCTION__, int32(Flags));

		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
		return false;
	}

	const FUnitTestNetworkPacket* ReceivePacket = ReceiveBuffer.Peek();
	if (ReceivePacket == nullptr)
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EWOULDBLOCK);
		return false;
	}

	if (!EnumHasAnyFlags(UnitTestFlags, BeaconUnitTest::ESocketFlags::RecvEnabled))
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EWOULDBLOCK);
		return false;
	}

	UE_LOGF(LogUnitTestSocketSubsystem, VeryVerbose, "[%s] Inbound message received. Socket: %p, FromPort: %d, ToPort: %d", __FUNCTION__, this, ReceivePacket->FromAddr.GetPort(), LocalAddress.GetPort());

	FInternetAddrBeaconUnitTest& SourceAddress = static_cast<FInternetAddrBeaconUnitTest&>(Source);
	SourceAddress = ReceivePacket->FromAddr;
	BytesRead = FMath::Min(BufferSize, ReceivePacket->Data.Num());
	FMemory::Memcpy(Data, ReceivePacket->Data.GetData(), BytesRead);
	ReceiveBuffer.Pop();

	return true;
}

bool FSocketBeaconUnitTest::Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags)
{
	BytesRead = 0;
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

ESocketConnectionState FSocketBeaconUnitTest::GetConnectionState()
{
	return ESocketConnectionState::SCS_NotConnected;
}

void FSocketBeaconUnitTest::GetAddress(FInternetAddr& OutAddr)
{
	OutAddr = LocalAddress;
}

bool FSocketBeaconUnitTest::GetPeerAddress(FInternetAddr& OutAddr)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::SetNonBlocking(bool bIsNonBlocking)
{
	return true;
}

bool FSocketBeaconUnitTest::SetBroadcast(bool bAllowBroadcast)
{
	return true;
}

bool FSocketBeaconUnitTest::SetNoDelay(bool bIsNoDelay)
{
	return true;
}

bool FSocketBeaconUnitTest::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::LeaveMulticastGroup(const FInternetAddr& GroupAddress)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::SetMulticastLoopback(bool bLoopback)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::SetMulticastTtl(uint8 TimeToLive)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::SetMulticastInterface(const FInternetAddr& InterfaceAddress)
{
	if (TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin())
	{
		SocketSubsystem->SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	}
	return false;
}

bool FSocketBeaconUnitTest::SetReuseAddr(bool bAllowReuse)
{
	return true;
}

bool FSocketBeaconUnitTest::SetLinger(bool bShouldLinger, int32 Timeout)
{
	return true;
}

bool FSocketBeaconUnitTest::SetRecvErr(bool bUseErrorQueue)
{
	return true;
}

bool FSocketBeaconUnitTest::SetSendBufferSize(int32 Size, int32& NewSize)
{
	return true;
}

bool FSocketBeaconUnitTest::SetReceiveBufferSize(int32 Size, int32& NewSize)
{
	return true;
}

int32 FSocketBeaconUnitTest::GetPortNo()
{
	return LocalAddress.GetPort();
}

void FSocketBeaconUnitTest::SetUnitTestFlags(BeaconUnitTest::ESocketFlags InUnitTestFlags)
{
	UnitTestFlags = InUnitTestFlags;
}

void FSocketBeaconUnitTest::FlushSendBuffer()
{
	TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem = WeakSocketSubsystem.Pin();
	if (!SocketSubsystem.IsValid())
	{
		return;
	}

	if (EnumHasAnyFlags(UnitTestFlags, BeaconUnitTest::ESocketFlags::SendEnabled) && !SendBuffer.IsEmpty())
	{
		UE_LOGF(LogUnitTestSocketSubsystem, VeryVerbose, "[%s] Flushing send buffer. Socket: %p, FromPort: %d", __FUNCTION__, this, LocalAddress.GetPort());

		while (FUnitTestNetworkPacket* SendPacket = SendBuffer.Peek())
		{
			SocketSubsystem->DispatchTestPacket(MoveTemp(*SendPacket));
			SendBuffer.Pop();
		}
	}
}

FOnlineBeaconUnitTestSocketSubsystem* FOnlineBeaconUnitTestSocketSubsystem::Singleton = nullptr;

FOnlineBeaconUnitTestSocketSubsystem::FOnlineBeaconUnitTestSocketSubsystem()
	: LastSocketError(SE_NO_ERROR)
{
}

FOnlineBeaconUnitTestSocketSubsystem::~FOnlineBeaconUnitTestSocketSubsystem()
{
}

FOnlineBeaconUnitTestSocketSubsystem* FOnlineBeaconUnitTestSocketSubsystem::Get()
{
	return Singleton;
}

bool FOnlineBeaconUnitTestSocketSubsystem::Init(FString& Error)
{
	FSocketSubsystemModule& SocketSubsystem = FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets");
	SocketSubsystem.RegisterSocketSubsystem(BeaconUnitTest::SocketSubsystemName, this, false);

	ensure(Singleton == nullptr);
	Singleton = this;

	return true;
}

void FOnlineBeaconUnitTestSocketSubsystem::Shutdown()
{
	Singleton = nullptr;

	if (FSocketSubsystemModule* SocketSubsystem = FModuleManager::GetModulePtr<FSocketSubsystemModule>("Sockets"))
	{
		SocketSubsystem->UnregisterSocketSubsystem(BeaconUnitTest::SocketSubsystemName);
	}
}

FSocket* FOnlineBeaconUnitTestSocketSubsystem::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	if (!ensure(SocketType == NAME_DGram))
	{
		return nullptr;
	}

	FSocketInfo& SocketInfo = Sockets.Emplace_GetRef(FSocketInfo{MakeShared<FSocketBeaconUnitTest>(SOCKTYPE_Datagram, SocketDescription, ProtocolType, AsShared()), FInternetAddrBeaconUnitTest(), false});
	return SocketInfo.Socket.Get();
}

void FOnlineBeaconUnitTestSocketSubsystem::DestroySocket(class FSocket* Socket)
{
	DestroySocket(Socket, true);
}

FAddressInfoResult FOnlineBeaconUnitTestSocketSubsystem::GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName,
	EAddressInfoFlags QueryFlags,
	const FName ProtocolTypeName,
	ESocketType SocketType)
{
	FAddressInfoResult AddrData = FAddressInfoResult(HostName, ServiceName);
	AddrData.QueryHostName = HostName;
	AddrData.ReturnCode = SE_NO_ERROR;

	const int32 PortValue = (AddrData.QueryServiceName.IsNumeric()) ? FCString::Atoi(ServiceName) : -1;
	TSharedRef<FInternetAddrBeaconUnitTest> NewAddress = MakeShared<FInternetAddrBeaconUnitTest>();
	NewAddress->SetPort(PortValue);
	AddrData.Results.Add(FAddressInfoResultData(NewAddress, 0, NAME_None, SocketType));

	return AddrData;
}

TSharedPtr<FInternetAddr> FOnlineBeaconUnitTestSocketSubsystem::GetAddressFromString(const FString& IPAddress)
{
	return nullptr;
}

class FResolveInfo* FOnlineBeaconUnitTestSocketSubsystem::GetHostByName(const ANSICHAR* HostName)
{
	return nullptr;
}

bool FOnlineBeaconUnitTestSocketSubsystem::RequiresChatDataBeSeparate()
{
	return false;
}

bool FOnlineBeaconUnitTestSocketSubsystem::RequiresEncryptedPackets()
{
	return false;
}

bool FOnlineBeaconUnitTestSocketSubsystem::GetHostName(FString& HostName)
{
	return false;
}

TSharedRef<FInternetAddr> FOnlineBeaconUnitTestSocketSubsystem::CreateInternetAddr()
{
	return MakeShared<FInternetAddrBeaconUnitTest>();
}

bool FOnlineBeaconUnitTestSocketSubsystem::HasNetworkDevice()
{
	return true;
}

const TCHAR* FOnlineBeaconUnitTestSocketSubsystem::GetSocketAPIName() const
{
	return TEXT("OnlineBeaconUnitTest");
}

ESocketErrors FOnlineBeaconUnitTestSocketSubsystem::GetLastErrorCode()
{
	return TranslateErrorCode(LastSocketError);
}

ESocketErrors FOnlineBeaconUnitTestSocketSubsystem::TranslateErrorCode(int32 Code)
{
	return static_cast<ESocketErrors>(Code);
}

bool FOnlineBeaconUnitTestSocketSubsystem::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr> >& OutAdresses)
{
	OutAdresses.Emplace(MakeShared<FInternetAddrBeaconUnitTest>());
	return true;
}

TArray<TSharedRef<FInternetAddr>> FOnlineBeaconUnitTestSocketSubsystem::GetLocalBindAddresses()
{
	return TArray<TSharedRef<FInternetAddr>>({MakeShared<FInternetAddrBeaconUnitTest>()});
}

bool FOnlineBeaconUnitTestSocketSubsystem::IsSocketWaitSupported() const
{
	return false;
}

void FOnlineBeaconUnitTestSocketSubsystem::SetLastSocketError(const ESocketErrors NewSocketError)
{
	LastSocketError = NewSocketError;
}

void FOnlineBeaconUnitTestSocketSubsystem::DispatchTestPacket(FUnitTestNetworkPacket&& Packet)
{
	if (FSocketInfo* SocketInfo = FindUnitTestSocketInfo(Packet.ToAddr))
	{
		UE_LOGF(LogUnitTestSocketSubsystem, VeryVerbose, "[%s] Dispatching packet to socket. ToSocket: %p, FromPort: %d, ToPort: %d", __FUNCTION__, SocketInfo->Socket.Get(), Packet.FromAddr.GetPort(), Packet.ToAddr.GetPort());
		SocketInfo->Socket->ReceiveBuffer.Enqueue(MoveTemp(Packet));
	}
	else
	{
		UE_LOGF(LogUnitTestSocketSubsystem, VeryVerbose, "[%s] Failed to find socket for destination. FromPort: %d, ToPort: %d", __FUNCTION__, Packet.FromAddr.GetPort(), Packet.ToAddr.GetPort());
	}
}

void FOnlineBeaconUnitTestSocketSubsystem::FlushSendBuffers()
{
	TArray<FSocketInfo> SocketsReadyForDestroy;

	for (FSocketInfo& SocketInfo : Sockets)
	{
		if (SocketInfo.BoundAddress.IsValid())
		{
			SocketInfo.Socket->FlushSendBuffer();
		}
		
		if (SocketInfo.bDestroyPendingFlush)
		{
			SocketsReadyForDestroy.Add(SocketInfo);
		}
	}

	for (FSocketInfo& SocketInfo : SocketsReadyForDestroy)
	{
		DestroySocket(SocketInfo.Socket.Get(), false);
	}
}

FInternetAddrBeaconUnitTest FOnlineBeaconUnitTestSocketSubsystem::BindSocket(FSocketBeaconUnitTest* Socket, const FInternetAddrBeaconUnitTest& RequestedAddress)
{
	FInternetAddrBeaconUnitTest ResolvedAddress = RequestedAddress;

	// Lookup the internal pointer for bookkeeping.
	FSocketInfo* SocketInfo = FindUnitTestSocketInfo(Socket);
	if (SocketInfo == nullptr)
	{
		return FInternetAddrBeaconUnitTest();
	}

	// Check whether an ephemeral port should be assigned.
	if (!ResolvedAddress.IsValid())
	{
		ResolvedAddress.SetPort(AllocateEphemeralPort());
	}

	UE_LOGF(LogUnitTestSocketSubsystem, Verbose, "[%s] Binding socket to requested address. Socket: %p, Port: %d", __FUNCTION__, Socket, RequestedAddress.GetPort());

	// Unbind current socket if already bound.
	UnbindSocket(*SocketInfo);

	// Remove previous use of address discarding any messages pending flush to prevent tests interfering with each other.
	if (FSocketInfo* ExistingBind = FindUnitTestSocketInfo(ResolvedAddress))
	{
		UE_LOGF(LogUnitTestSocketSubsystem, Verbose, "[%s] Unbinding existing socket from requested address. Socket: %p, Port: %d", __FUNCTION__, ExistingBind->Socket.Get(), RequestedAddress.GetPort());
		UnbindSocket(*ExistingBind);
	}

	SocketInfo->BoundAddress = ResolvedAddress;
	return ResolvedAddress;
}

int32 FOnlineBeaconUnitTestSocketSubsystem::AllocateEphemeralPort()
{
	return NextEphemeralPort++;
}

void FOnlineBeaconUnitTestSocketSubsystem::DestroySocket(class FSocket* Socket, bool bFlushTransmit)
{
	if (bFlushTransmit)
	{
		if (FSocketInfo* SocketInfo = FindUnitTestSocketInfo(Socket))
		{
			const bool bHasMessagesPendingTransmit = !SocketInfo->Socket->SendBuffer.IsEmpty() && EnumHasAnyFlags(SocketInfo->Socket->UnitTestFlags, BeaconUnitTest::ESocketFlags::SendEnabled);
			if (bHasMessagesPendingTransmit)
			{
				UE_LOGF(LogUnitTestSocketSubsystem, Verbose, "[%s] Setting socket for pending destroy due to buffered outbound messages. Socket: %p, Port: %d", __FUNCTION__, Socket, Socket->GetPortNo());
				SocketInfo->bDestroyPendingFlush = true;
			}
			else
			{
				DestroySocket(Socket, false);
			}
		}
	}
	else
	{
		UE_LOGF(LogUnitTestSocketSubsystem, Verbose, "[%s] Destroying socket. Socket: %p, Port: %d", __FUNCTION__, Socket, Socket->GetPortNo());

		Sockets.SetNum(Algo::RemoveIf(Sockets, [Socket](const FSocketInfo& SocketInfo)
		{
			return SocketInfo.Socket.Get() == Socket;
		}));
	}
}

void FOnlineBeaconUnitTestSocketSubsystem::UnbindSocket(FSocketInfo& SocketInfo)
{
	SocketInfo.Socket->SendBuffer.Empty();
	SocketInfo.BoundAddress = FInternetAddrBeaconUnitTest();
}

FOnlineBeaconUnitTestSocketSubsystem::FSocketInfo* FOnlineBeaconUnitTestSocketSubsystem::FindUnitTestSocketInfo(const FSocket* Socket)
{
	return Algo::FindByPredicate(Sockets, [Socket](const FSocketInfo& SocketInfo)
	{
		return SocketInfo.Socket.Get() == Socket;
	});
}

const FOnlineBeaconUnitTestSocketSubsystem::FSocketInfo* FOnlineBeaconUnitTestSocketSubsystem::FindUnitTestSocketInfo(const FSocket* Socket) const
{
	return const_cast<FOnlineBeaconUnitTestSocketSubsystem*>(this)->FindUnitTestSocketInfo(Socket);
}

FOnlineBeaconUnitTestSocketSubsystem::FSocketInfo* FOnlineBeaconUnitTestSocketSubsystem::FindUnitTestSocketInfo(const FInternetAddrBeaconUnitTest& Address)
{
	return Algo::FindByPredicate(Sockets, [&Address](const FSocketInfo& SocketInfo)
	{
		return SocketInfo.BoundAddress == Address;
	});
}

const FOnlineBeaconUnitTestSocketSubsystem::FSocketInfo* FOnlineBeaconUnitTestSocketSubsystem::FindUnitTestSocketInfo(const FInternetAddrBeaconUnitTest& Address) const
{
	return const_cast<FOnlineBeaconUnitTestSocketSubsystem*>(this)->FindUnitTestSocketInfo(Address);
}

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
