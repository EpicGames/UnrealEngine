// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Tests/OnlineBeaconUnitTestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

class FOnlineBeaconUnitTestSocketSubsystem;

class FInternetAddrBeaconUnitTest : public FInternetAddr
{
public:

	FInternetAddrBeaconUnitTest()
	{
	}

	//~ Begin FInternetAddr Interface
	virtual void SetIp(uint32 InAddr) override;
	virtual void SetIp(const TCHAR* InAddr, bool& bIsValid) override;
	virtual void GetIp(uint32& OutAddr) const override;
	virtual void SetPort(int32 InPort) override;
	virtual int32 GetPort() const override;
	virtual void SetRawIp(const TArray<uint8>& RawAddr) override;
	virtual TArray<uint8> GetRawIp() const override;
	virtual void SetAnyAddress() override;
	virtual void SetBroadcastAddress() override;
	virtual void SetLoopbackAddress() override;
	virtual FString ToString(bool bAppendPort) const override;
	virtual bool operator==(const FInternetAddr& Other) const override;
	virtual uint32 GetTypeHash() const override;
	virtual bool IsValid() const override;
	virtual TSharedRef<FInternetAddr> Clone() const override;
	//~ End FInternetAddr Interface

	bool operator==(const FInternetAddrBeaconUnitTest& Other) const
	{
		return Other.Port == Port;
	}
	bool operator!=(const FInternetAddrBeaconUnitTest& Other) const
	{
		return !(FInternetAddrBeaconUnitTest::operator==(Other));
	}

	friend uint32 GetTypeHash(const FInternetAddrBeaconUnitTest& A)
	{
		return A.GetTypeHash();
	}

private:
	static constexpr int32 InvalidPort = 0;
	int32 Port = InvalidPort;
};

struct FUnitTestNetworkPacket
{
	static constexpr int32 MaxPacketSize = 65535;

	FInternetAddrBeaconUnitTest FromAddr;
	FInternetAddrBeaconUnitTest ToAddr;
	TArray<uint8> Data;
};

class FSocketBeaconUnitTest : public FSocket
{
public:
	FSocketBeaconUnitTest(ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol, const TSharedRef<FOnlineBeaconUnitTestSocketSubsystem>& InSubsystem);
	virtual ~FSocketBeaconUnitTest();

	//~ Begin FSocket Interface
	virtual bool Shutdown(ESocketShutdownMode Mode) override;
	virtual bool Close() override;
	virtual bool Bind(const FInternetAddr& Addr) override;
	virtual bool Connect(const FInternetAddr& Addr) override;
	virtual bool Listen(int32 MaxBacklog) override;
	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) override;
	virtual bool HasPendingData(uint32& PendingDataSize) override;
	virtual class FSocket* Accept(const FString& InSocketDescription) override;
	virtual class FSocket* Accept(FInternetAddr& OutAddr, const FString& InSocketDescription) override;
	virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination);
	virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent);
	virtual bool RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None);
	virtual bool Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None);
	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) override;
	virtual ESocketConnectionState GetConnectionState() override;
	virtual void GetAddress(FInternetAddr& OutAddr) override;
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) override;
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) override;
	virtual bool SetBroadcast(bool bAllowBroadcast = true) override;
	virtual bool SetNoDelay(bool bIsNoDelay = true) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool SetMulticastLoopback(bool bLoopback) override;
	virtual bool SetMulticastTtl(uint8 TimeToLive) override;
	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) override;
	virtual bool SetReuseAddr(bool bAllowReuse = true) override;
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) override;
	virtual bool SetRecvErr(bool bUseErrorQueue = true) override;
	virtual bool SetSendBufferSize(int32 Size, int32& NewSize) override;
	virtual bool SetReceiveBufferSize(int32 Size, int32& NewSize) override;
	virtual int32 GetPortNo() override;
	//~ End FSocket Interface

	void SetUnitTestFlags(BeaconUnitTest::ESocketFlags UnitTestFlags);
	void FlushSendBuffer();

	/** Reference to our subsystem */
	TWeakPtr<FOnlineBeaconUnitTestSocketSubsystem> WeakSocketSubsystem;

	/** Our local address; session/port will be invalid when not bound */
	FInternetAddrBeaconUnitTest LocalAddress;

	BeaconUnitTest::ESocketFlags UnitTestFlags = BeaconUnitTest::ESocketFlags::Default;

	TQueue<FUnitTestNetworkPacket> SendBuffer;
	TQueue<FUnitTestNetworkPacket> ReceiveBuffer;
};

class FOnlineBeaconUnitTestSocketSubsystem : public ISocketSubsystem, public TSharedFromThis<FOnlineBeaconUnitTestSocketSubsystem>
{
public:
	FOnlineBeaconUnitTestSocketSubsystem();
	virtual ~FOnlineBeaconUnitTestSocketSubsystem();

	static FOnlineBeaconUnitTestSocketSubsystem* Get();

	//~ Begin ISocketSubsystem Interface
	virtual bool Init(FString& Error) override;
	virtual void Shutdown() override;
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolName) override;
	virtual void DestroySocket(class FSocket* Socket) override;
	virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr,
		EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		const FName ProtocolTypeName = NAME_None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) override;
	virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& IPAddress) override;
	virtual class FResolveInfo* GetHostByName(const ANSICHAR* HostName) override;
	virtual bool RequiresChatDataBeSeparate() override;
	virtual bool RequiresEncryptedPackets() override;
	virtual bool GetHostName(FString& HostName) override;
	virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;
	virtual bool HasNetworkDevice() override;
	virtual const TCHAR* GetSocketAPIName() const override;
	virtual ESocketErrors GetLastErrorCode() override;
	virtual ESocketErrors TranslateErrorCode(int32 Code) override;
	virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr> >& OutAdresses) override;
	virtual TArray<TSharedRef<FInternetAddr>> GetLocalBindAddresses() override;
	virtual bool IsSocketWaitSupported() const override;
	//~ End ISocketSubsystem Interface

	void SetLastSocketError(const ESocketErrors NewSocketError);

	void DispatchTestPacket(FUnitTestNetworkPacket&& Packet);

	void FlushSendBuffers();

	/*
	 * Bind the socket to the requested test address. If the port is unset an ephemeral port will be bound.
	 * Once bound, the socket will be flushed for pending IO when Flush is called.
	 * Returns the resolved address.
	 */
	FInternetAddrBeaconUnitTest BindSocket(FSocketBeaconUnitTest* Socket, const FInternetAddrBeaconUnitTest& RequestedAddress);

private:

	struct FSocketInfo
	{
		TSharedPtr<FSocketBeaconUnitTest> Socket;
		FInternetAddrBeaconUnitTest BoundAddress;
		bool bDestroyPendingFlush = false;
	};

	int32 AllocateEphemeralPort();
	void DestroySocket(class FSocket* Socket, bool bFlushTransmit);
	void UnbindSocket(FSocketInfo& SocketInfo);

	FSocketInfo* FindUnitTestSocketInfo(const FSocket* Socket);
	const FSocketInfo* FindUnitTestSocketInfo(const FSocket* Socket) const;
	FSocketInfo* FindUnitTestSocketInfo(const FInternetAddrBeaconUnitTest& Address);
	const FSocketInfo* FindUnitTestSocketInfo(const FInternetAddrBeaconUnitTest& Address) const;

	static FOnlineBeaconUnitTestSocketSubsystem* Singleton;
	ESocketErrors LastSocketError;
	TArray<FSocketInfo> Sockets;

	// Start assignment outside of normal port range.
	static constexpr int32 EphemeralPortStart = 65536;
	int32 NextEphemeralPort = EphemeralPortStart;
};

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
