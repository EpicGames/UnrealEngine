// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/OnlineBeaconUnitTestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Engine/Engine.h"
#include "Engine/NetConnection.h"
#include "Engine/World.h"
#include "GameFramework/OnlineReplStructs.h"
#include "GameFramework/WorldSettings.h"
#include "IpNetDriver.h"
#include "Misc/ConfigCacheIni.h"
#include "Online/CoreOnline.h"
#include "OnlineBeaconClient.h"
#include "OnlineBeaconHost.h"
#include "OnlineBeaconHostObject.h"
#include "OnlineBeaconUnitTestNetDriver.h"
#include "OnlineBeaconUnitTestSocketSubsystem.h"
#include "PacketHandler.h"
#include "Tests/AutomationEditorCommon.h"
#include "TimerManager.h"

namespace BeaconUnitTest
{
namespace Private
{

class FNetworkStatsImpl : public FNetworkStats, public FNetworkNotify
{
public:
	FNetworkStatsImpl(const TObjectPtr<UNetDriver>& NetDriver)
		: WeakNetDriver(NetDriver)
		, PreviousNotifyHandler(nullptr)
	{
		if (NetDriver)
		{
			PreviousNotifyHandler = NetDriver->Notify;
			NetDriver->Notify = this;
		}
	}

	virtual ~FNetworkStatsImpl()
	{
		if (UNetDriver* NetDriver = WeakNetDriver.Get())
		{
			if (NetDriver->Notify == this)
			{
				NetDriver->Notify = PreviousNotifyHandler;
			}
			else
			{
				NetDriver->Notify = nullptr;
			}
		}
	}

	virtual EAcceptConnection::Type NotifyAcceptingConnection()
	{
		return PreviousNotifyHandler ? PreviousNotifyHandler->NotifyAcceptingConnection() : EAcceptConnection::Ignore;
	}

	virtual void NotifyAcceptedConnection(class UNetConnection* Connection)
	{
		if (PreviousNotifyHandler)
		{
			PreviousNotifyHandler->NotifyAcceptedConnection(Connection);
		}
	}

	virtual bool NotifyAcceptingChannel(class UChannel* Channel)
	{
		return PreviousNotifyHandler ? PreviousNotifyHandler->NotifyAcceptingChannel(Channel) : false;
	}

	virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, class FInBunch& Bunch)
	{
		ReceivedControlMessages.Add(MessageType);

		if (PreviousNotifyHandler)
		{
			PreviousNotifyHandler->NotifyControlMessage(Connection, MessageType, Bunch);
		}
	}

	TWeakObjectPtr<UNetDriver> WeakNetDriver;
	FNetworkNotify* PreviousNotifyHandler = nullptr;
};

class FTestPrerequisitesImpl final : public FTestPrerequisites
{
public:
	static TSharedPtr<FTestPrerequisitesImpl> TryCreate()
	{
		// Verify that the global net delegates are not in use before using them for the test.
		if (!ensure(!FNetDelegates::OnReceivedNetworkEncryptionToken.IsBound()))
		{
			return nullptr;
		}
		if (!ensure(!FNetDelegates::OnReceivedNetworkEncryptionAck.IsBound()))
		{
			return nullptr;
		}
		if (!ensure(!FNetDelegates::OnReceivedNetworkEncryptionFailure.IsBound()))
		{
			return nullptr;
		}

		if (!ensure(GConfig))
		{
			return nullptr;
		}

		if (!ensure(GEngine))
		{
			return nullptr;
		}

		TSharedRef<FTestPrerequisitesImpl> Prerequisites = MakeShared<FTestPrerequisitesImpl>();
		ActiveTestPrerequisites = Prerequisites;
		return Prerequisites;
	}

	static TSharedPtr<FTestPrerequisitesImpl> Get()
	{
		return ActiveTestPrerequisites.Pin();
	}

	static FTestStats* GetActiveTestStats()
	{
		TSharedPtr<FTestPrerequisitesImpl> Prerequisites = ActiveTestPrerequisites.Pin();
		return Prerequisites ? &Prerequisites->GetStats() : nullptr;
	}

	static const FTestConfig* GetActiveTestConfig()
	{
		TSharedPtr<FTestPrerequisitesImpl> Prerequisites = ActiveTestPrerequisites.Pin();
		return Prerequisites ? &Prerequisites->GetConfig() : nullptr;
	}

	FTestPrerequisitesImpl()
		: FTestPrerequisites()
	{
		// Create and register mock socket subsystem.
		FString InitError;
		SocketSubsystem = MakeShared<FOnlineBeaconUnitTestSocketSubsystem>();
		ensure(SocketSubsystem->Init(InitError));

		// Initialize world.
		World = FAutomationEditorCommonUtils::CreateNewMap();
		World->InitializeActorsForPlay(FURL());
		if (World->GetWorldSettings())
		{
			World->GetWorldSettings()->NotifyBeginPlay();
		}

		// Bind delegates for handling network encryption.
		BindNetEncryptionDelegates();

		// Bind delegate to flush all outbound network messages to their target connections at the end of world tick.
		NetworkFlushDelegateHandle = FWorldDelegates::OnWorldTickEnd.AddRaw(this, &FTestPrerequisitesImpl::NetworkFlush);

#if !NO_LOGGING
		// Disable logging for some categories since the tests will cause warnings / errors to be logged for some failure testing.
		StoredLogBeaconVerbosity = LogBeacon.GetVerbosity();
		LogBeacon.SetVerbosity(ELogVerbosity::NoLogging);
		StoredLogNetVerbosity = LogNet.GetVerbosity();
		LogNet.SetVerbosity(ELogVerbosity::NoLogging);
#endif

		// Setup encrpytion component. Store previous value to restore when the test has completed.
		GConfig->GetString(TEXT("PacketHandlerComponents"), TEXT("EncryptionComponent"), StoredEncryptionComponentName, GEngineIni);
		GConfig->SetString(TEXT("PacketHandlerComponents"), TEXT("EncryptionComponent"), *Config.Encryption.NetDriverEncryptionComponentName, GEngineIni);

		// Install netdriver definition for the test driver.
		const bool bDefinitionExists = Algo::FindBy(GEngine->NetDriverDefinitions, BeaconUnitTest::NetDriverDefinitionName, [](const FNetDriverDefinition& Definition)->const FName& { return Definition.DefName; }) != nullptr;
		if (ensure(!bDefinitionExists))
		{
			FNetDriverDefinition TestBeaconNetdriverDefinition;
			TestBeaconNetdriverDefinition.DefName = BeaconUnitTest::NetDriverDefinitionName;
			TestBeaconNetdriverDefinition.DriverClassName = FName(UOnlineBeaconUnitTestNetDriver::StaticClass()->GetPathName());
			GEngine->NetDriverDefinitions.Add(TestBeaconNetdriverDefinition);
		}

		// Install Iris netdriver config for the test driver.
		const bool bIrisConfigExists = Algo::FindBy(GEngine->IrisNetDriverConfigs, BeaconUnitTest::NetDriverDefinitionName, [](const FIrisNetDriverConfig& IrisConfig)->const FName& { return IrisConfig.NetDriverDefinition; }) != nullptr;
		if (ensure(!bIrisConfigExists))
		{
			FIrisNetDriverConfig TestBeaconNetDriverIrisConfig;
			TestBeaconNetDriverIrisConfig.NetDriverDefinition = BeaconUnitTest::NetDriverDefinitionName;
			TestBeaconNetDriverIrisConfig.bCanUseIris = true;
			GEngine->IrisNetDriverConfigs.Add(TestBeaconNetDriverIrisConfig);
		}
	}

	virtual ~FTestPrerequisitesImpl()
	{
		World->EndPlay(EEndPlayReason::EndPlayInEditor);

		// Remove unit test net driver definition.
		GEngine->NetDriverDefinitions.SetNum(Algo::RemoveIf(GEngine->NetDriverDefinitions, [&](const FNetDriverDefinition& Definition) -> bool { return Definition.DefName == BeaconUnitTest::NetDriverDefinitionName; }));

		// Remove unit test net driver Iris config.
		GEngine->IrisNetDriverConfigs.SetNum(Algo::RemoveIf(GEngine->IrisNetDriverConfigs, [&](const FIrisNetDriverConfig& IrisConfig) -> bool { return IrisConfig.NetDriverDefinition == BeaconUnitTest::NetDriverDefinitionName; }));

		FWorldDelegates::OnWorldTickEnd.Remove(NetworkFlushDelegateHandle);
		UnbindNetEncryptionDelegates();
		World = nullptr;

#if !NO_LOGGING
		// Restore logging.
		LogBeacon.SetVerbosity(StoredLogBeaconVerbosity);
		LogNet.SetVerbosity(StoredLogNetVerbosity);
#endif

		// Restore encryption config.
		GConfig->SetString(TEXT("PacketHandlerComponents"), TEXT("EncryptionComponent"), *StoredEncryptionComponentName, GEngineIni);

		// Shutdown and unregister mock socket subsystem.
		SocketSubsystem->Shutdown();
	}

	virtual void BindNetEncryptionDelegates() override
	{
		FNetDelegates::OnReceivedNetworkEncryptionToken.BindRaw(this, &FTestPrerequisitesImpl::ReceivedNetworkEncryptionToken);
		FNetDelegates::OnReceivedNetworkEncryptionAck.BindRaw(this, &FTestPrerequisitesImpl::ReceivedNetworkEncryptionAck);
		FNetDelegates::OnReceivedNetworkEncryptionFailure.BindRaw(this, &FTestPrerequisitesImpl::ReceivedNetworkEncryptionFailure);
	}

	virtual void UnbindNetEncryptionDelegates() override
	{
		FNetDelegates::OnReceivedNetworkEncryptionToken.Unbind();
		FNetDelegates::OnReceivedNetworkEncryptionAck.Unbind();
		FNetDelegates::OnReceivedNetworkEncryptionFailure.Unbind();
	}

	void NetworkFlush(UWorld*, ELevelTick, float)
	{
		SocketSubsystem->FlushSendBuffers();
	}

	void ReceivedNetworkEncryptionToken(const FString& EncryptionToken, const FOnEncryptionKeyResponse& Delegate)
	{
		++Stats.Encryption.NetworkEncryptionToken.InvokeCount;

		FEncryptionKeyResponse Response;
		Response.Response = Config.Encryption.Host.Response;
		Response.ErrorMsg = Config.Encryption.Host.ErrorMsg;
		Response.EncryptionData = Config.Encryption.Host.EncryptionData;

		if (Config.Encryption.Host.bDelayDelegate)
		{
			SetTimerForNextFrame(World, GFrameCounter, [this, Delegate, Response]()
			{
				++Stats.Encryption.NetworkEncryptionToken.CallbackCount;
				Delegate.ExecuteIfBound(Response);
			});
		}
		else
		{
			++Stats.Encryption.NetworkEncryptionToken.CallbackCount;
			Delegate.ExecuteIfBound(Response);
		}
	}

	void ReceivedNetworkEncryptionAck(const FOnEncryptionKeyResponse& Delegate)
	{
		++Stats.Encryption.NetworkEncryptionAck.InvokeCount;

		FEncryptionKeyResponse Response;
		Response.Response = Config.Encryption.Client.Response;
		Response.ErrorMsg = Config.Encryption.Client.ErrorMsg;
		Response.EncryptionData = Config.Encryption.Client.EncryptionData;

		if (Config.Encryption.Client.bDelayDelegate)
		{
			SetTimerForNextFrame(World, GFrameCounter, [this, Delegate, Response]()
			{
				++Stats.Encryption.NetworkEncryptionAck.CallbackCount;
				Delegate.ExecuteIfBound(Response);
			});
		}
		else
		{
			++Stats.Encryption.NetworkEncryptionAck.CallbackCount;
			Delegate.ExecuteIfBound(Response);
		}
	}

	EEncryptionFailureAction ReceivedNetworkEncryptionFailure(UNetConnection* Connection)
	{
		++Stats.Encryption.NetworkEncryptionFailure.InvokeCount;
		return Config.Encryption.FailureAction;
	}

	TSharedPtr<FOnlineBeaconUnitTestSocketSubsystem> SocketSubsystem;
	FDelegateHandle NetworkFlushDelegateHandle;

#if !NO_LOGGING
	ELogVerbosity::Type StoredLogBeaconVerbosity = ELogVerbosity::NoLogging;
	ELogVerbosity::Type StoredLogNetVerbosity = ELogVerbosity::NoLogging;
#endif

	FString StoredEncryptionComponentName;

	static TWeakPtr<FTestPrerequisitesImpl> ActiveTestPrerequisites;
};

TWeakPtr<FTestPrerequisitesImpl> FTestPrerequisitesImpl::ActiveTestPrerequisites;

//--------------------------------------------------------------------------------------------------
// Hacky accessor to private fields - Allowed due to a loophole in the c++ standard regarding ignoring access checks for template specialization.

// OnlineBeaconHost access.

template<typename ClassType, typename ClassType::MemberType Member>
struct TBeaconHostAccessShim
{
	friend typename ClassType::MemberType UnitTestGetBeaconHostMember(ClassType)
	{
		return Member;
	}
};
struct FBeaconHostClientActorsTag
{ 
  typedef TArray<TObjectPtr<AOnlineBeaconClient>> AOnlineBeaconHostObject::*MemberType;
  friend MemberType UnitTestGetBeaconHostMember(FBeaconHostClientActorsTag);
};
template struct TBeaconHostAccessShim<FBeaconHostClientActorsTag, &AOnlineBeaconHostObject::ClientActors>;

struct FBeaconHostAuthRequiredTag
{ 
  typedef bool AOnlineBeaconHost::*MemberType;
  friend MemberType UnitTestGetBeaconHostMember(FBeaconHostAuthRequiredTag);
};
template struct TBeaconHostAccessShim<FBeaconHostAuthRequiredTag, &AOnlineBeaconHost::bAuthRequired>;

//--------------------------------------------------------------------------------------------------

// OnlineBeacon access.

template<typename ClassType, typename ClassType::MemberType Member>
struct TBeaconAccessShim
{
	friend typename ClassType::MemberType UnitTestGetBeaconMember(ClassType)
	{
		return Member;
	}
};

struct FBeaconNetDriverTag
{ 
  typedef TObjectPtr<UNetDriver> AOnlineBeacon::*MemberType;
  friend MemberType UnitTestGetBeaconMember(FBeaconNetDriverTag);
};
template struct TBeaconAccessShim<FBeaconNetDriverTag, &AOnlineBeacon::NetDriver>;

struct FBeaconNetDriverDefinitionNameTag
{ 
  typedef FName AOnlineBeacon::*MemberType;
  friend MemberType UnitTestGetBeaconMember(FBeaconNetDriverDefinitionNameTag);
};
template struct TBeaconAccessShim<FBeaconNetDriverDefinitionNameTag, &AOnlineBeacon::NetDriverDefinitionName>;

/* Private */}

const FName NetDriverDefinitionName(TEXT("UnitTestBeaconNetDriver"));
const FName SocketSubsystemName(TEXT("UnitTestSocketSubsystem"));

FTestStats* FTestPrerequisites::GetActiveTestStats()
{
	return Private::FTestPrerequisitesImpl::GetActiveTestStats();
}

const FTestConfig* FTestPrerequisites::GetActiveTestConfig()
{
	return Private::FTestPrerequisitesImpl::GetActiveTestConfig();
}

TSharedPtr<FTestPrerequisites> FTestPrerequisites::TryCreate()
{
	return Private::FTestPrerequisitesImpl::TryCreate();
}

bool SetSocketFlags(AOnlineBeacon* OnlineBeacon, ESocketFlags Flags)
{
	if (OnlineBeacon == nullptr)
	{
		return false;
	}

	TObjectPtr<UNetDriver>& NetDriver = OnlineBeacon->*UnitTestGetBeaconMember(Private::FBeaconNetDriverTag());
	UIpNetDriver* IpNetDriver = Cast<UIpNetDriver>(NetDriver.Get());
	FSocketBeaconUnitTest* Socket = IpNetDriver ? reinterpret_cast<FSocketBeaconUnitTest*>(IpNetDriver->GetSocket()) : nullptr;

	if (Socket == nullptr)
	{
		return false;
	}

	Socket->SetUnitTestFlags(Flags);
	return true;
}

bool SetTimeoutsEnabled(AOnlineBeacon* OnlineBeacon, bool bEnabled)
{
	if (OnlineBeacon == nullptr)
	{
		return false;
	}

	TObjectPtr<UNetDriver>& NetDriver = OnlineBeacon->*UnitTestGetBeaconMember(Private::FBeaconNetDriverTag());
	NetDriver->bNoTimeouts = bEnabled == false;

	return true;
}

bool ConfigureBeacon(const FTestPrerequisites& Prerequisites, AOnlineBeacon* OnlineBeacon)
{
	if (OnlineBeacon == nullptr)
	{
		return false;
	}

	const BeaconUnitTest::FTestConfig& TestConfig = Prerequisites.GetConfig();

	// Set beacon to use the unit test net driver.
	FName& BeaconNetDriverDefinitionName = OnlineBeacon->*UnitTestGetBeaconMember(Private::FBeaconNetDriverDefinitionNameTag());
	BeaconNetDriverDefinitionName = NetDriverDefinitionName;

	if (AOnlineBeaconHost* BeaconHost = Cast<AOnlineBeaconHost>(OnlineBeacon))
	{
		bool& bAuthRequired = BeaconHost->*UnitTestGetBeaconHostMember(Private::FBeaconHostAuthRequiredTag());
		bAuthRequired = TestConfig.Auth.bEnabled;
	}

	return true;
}

bool ConfigureBeaconNetDriver(const FTestPrerequisites& Prerequisites, AOnlineBeacon* OnlineBeacon, TSharedPtr<FNetworkStats>* OutStats)
{
	if (OnlineBeacon == nullptr)
	{
		return false;
	}

	TObjectPtr<UNetDriver>& NetDriver = OnlineBeacon->*UnitTestGetBeaconMember(Private::FBeaconNetDriverTag());
	if (NetDriver == nullptr)
	{
		return false;
	}

	const BeaconUnitTest::FTestConfig TestConfig = Prerequisites.GetConfig();
	NetDriver->bNoTimeouts = true;
	NetDriver->InitialConnectTimeout = TestConfig.NetDriver.InitialConnectTimeout;
	NetDriver->ConnectionTimeout = TestConfig.NetDriver.ConnectionTimeout;
	NetDriver->KeepAliveTime = TestConfig.NetDriver.ConnectionTimeout;

	if (OutStats)
	{
		*OutStats = MakeShared<Private::FNetworkStatsImpl>(NetDriver);
	}

	return true;
}

bool InitClientForUser(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, const FUniqueNetIdRef& User)
{
	if (OnlineBeaconClient == nullptr)
	{
		return false;
	}

	FURL URL;
	URL.Port = Prerequisites.GetConfig().NetDriver.ServerListenPort;
	if (!OnlineBeaconClient->InitClient(URL))
	{
		return false;
	}

	UNetConnection* Connection = OnlineBeaconClient->GetNetConnection();
	if(Connection == nullptr)
	{
		return false;
	}

	Connection->PlayerId = User;
	return true;
}

AOnlineBeaconClient* GetBeaconClientForUser(AOnlineBeaconHostObject* OnlineBeaconHostObject, const FUniqueNetIdRef& User)
{
	if (OnlineBeaconHostObject == nullptr)
	{
		return nullptr;
	}

	if (!User->IsValid())
	{
		return nullptr;
	}

	TArray<TObjectPtr<AOnlineBeaconClient>>& ClientActors = OnlineBeaconHostObject->*UnitTestGetBeaconHostMember(Private::FBeaconHostClientActorsTag());
	TObjectPtr<AOnlineBeaconClient>* Result = Algo::FindByPredicate(ClientActors, [&User](const TObjectPtr<AOnlineBeaconClient>& BeaconClient){
		return BeaconClient->GetUniqueId() == User;
	});

	return Result ? Result->Get() : nullptr;
}

bool TickOnce(const FTestPrerequisites& Prerequisites, ETickFlags Flags)
{
	UWorld* World = Prerequisites.GetWorld();
	if (World == nullptr)
	{
		return false;
	}

	if (EnumHasAllFlags(Flags, ETickFlags::SleepTickTime))
	{
		FPlatformProcess::Sleep(Prerequisites.GetConfig().WorldTickRate);
	}
	World->Tick(ELevelTick::LEVELTICK_All, Prerequisites.GetConfig().WorldTickRate);
	++GFrameCounter;

	return true;
}

bool TickUntilConnectionInitialized(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, ETickFlags Flags)
{
	if (OnlineBeaconClient == nullptr)
	{
		return false;
	}

	TWeakObjectPtr<AOnlineBeaconClient> WeakOnlineBeaconClient = OnlineBeaconClient;
	return TickUntil(Prerequisites, [WeakOnlineBeaconClient]()
	{
		AOnlineBeaconClient* OnlineBeaconClient = WeakOnlineBeaconClient.Get();
		UNetConnection* BeaconConnection = OnlineBeaconClient ? OnlineBeaconClient->GetNetConnection() : nullptr;
		return BeaconConnection && BeaconConnection->Handler ? BeaconConnection->Handler->IsFullyInitialized() : false;
	}, Flags);
}

bool TickUntilControlMessageReceived(const FTestPrerequisites& Prerequisites, const FNetworkStats& Stats, ETickFlags Flags)
{
	return TickUntil(Prerequisites, [&Stats]() -> bool
	{
		return !Stats.ReceivedControlMessages.IsEmpty();
	});
}

bool TickUntilConnected(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, ETickFlags Flags)
{
	TWeakObjectPtr<AOnlineBeaconClient> WeakOnlineBeaconClient = OnlineBeaconClient;
	return TickUntil(Prerequisites, [WeakOnlineBeaconClient]()
	{
		AOnlineBeaconClient* OnlineBeaconClient = WeakOnlineBeaconClient.Get();
		return OnlineBeaconClient != nullptr && (OnlineBeaconClient->GetConnectionState() == EBeaconConnectionState::Open);
	}, Flags);
}

bool TickUntilDisconnected(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, ETickFlags Flags)
{
	TWeakObjectPtr<AOnlineBeaconClient> WeakOnlineBeaconClient = OnlineBeaconClient;
	return TickUntil(Prerequisites, [WeakOnlineBeaconClient]()
	{
		AOnlineBeaconClient* OnlineBeaconClient = WeakOnlineBeaconClient.Get();
		return OnlineBeaconClient != nullptr && (OnlineBeaconClient->GetConnectionState() == EBeaconConnectionState::Invalid || OnlineBeaconClient->GetConnectionState() == EBeaconConnectionState::Closed);
	}, Flags);
}

bool TickUntilTimeoutElapsed(const FTestPrerequisites& Prerequisite, ETickFlags Flags)
{
	const double StartTime = FPlatformTime::Seconds();
	const double ExpireTime = StartTime + Prerequisite.GetConfig().NetDriver.InitialConnectTimeout;

	// Wait for time to expire.
	return BeaconUnitTest::TickUntil(Prerequisite, [ExpireTime]() -> bool
	{
		return FPlatformTime::Seconds() > ExpireTime;
	}, BeaconUnitTest::ETickFlags::SleepTickTime);
}

/* EBeaconUnitTest */ }

#endif /* WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR */
