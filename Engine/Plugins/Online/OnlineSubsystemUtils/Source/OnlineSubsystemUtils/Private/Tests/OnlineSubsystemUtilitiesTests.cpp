// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CommandLine.h"
#include "Misc/AutomationTest.h"
#include "Engine/Engine.h"
#include "OnlineSubsystem.h"
#include "Tests/TestIdentityInterface.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(OSSUtilsTestLog, All, All);

namespace OSSUtilsTestHelper
{
	// Enums used to hold the state of the current test.
	enum type
	{
		NotStarted,
		BeginTest,
		PerformingTest,
	};

	/**
	* Get the username and password from the command line.
	* Command line arguments are -OSSIDTESTUSER="YourUser" and -OSSIDTESTPSSWD="YouPassword"
	* @param OutUsername is the referenced variable that will hold the found username.
	* @param OutPassword is the referenced variable that will hold the found password.
	*/
	bool GetUserInfo( FString& OutUsername, FString& OutPassword )
	{
		OutUsername = TEXT( "" );
		OutPassword = TEXT( "" );
		// Get the username and password from the command line
		const FString CommandLine( FCommandLine::Get() );
		FParse::Value( *CommandLine, TEXT( "OSSIDTESTUSER=" ), OutUsername );
		FParse::Value( *CommandLine, TEXT( "OSSIDTESTPSSWD=" ), OutPassword );

		if ( OutUsername.IsEmpty() || OutPassword.IsEmpty() )
		{
			UE_LOGF( OSSUtilsTestLog, Error, "Missing test username and/or password." );
			return false;
		}
		return true;
	}

	void GetOSSNamesForTesting( TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands, const FString& PostTestName = TEXT( "" ) )
	{
		TArray< FName > OSSList;
		OSSList.AddUnique(TEXT( "MCP" ) );
		OSSList.AddUnique(TEXT( "Null" ) );
		OSSList.AddUnique(TEXT( "Steam" ) );
		OSSList.AddUnique(TEXT( "Facebook" ) );
		OSSList.AddUnique(TEXT( "IOS" ) );
		OSSList.AddUnique(TEXT( "GooglePlay" ) );
		OSSList.AddUnique(TEXT( "Live" ) );
		OSSList.AddUnique(TEXT( "Thunderhead" ) );
		OSSList.AddUnique(TEXT( "Amazon" ) );

		for ( auto& OSSName : OSSList )
		{
			if (IOnlineSubsystem::Get(OSSName))
			{
				FString PostName = FString::Printf( TEXT( ".%s" ), *PostTestName );
				FString PrettyName = FString::Printf(TEXT( "%s%s" ), 
					*OSSName.ToString(), 
					PostTestName.IsEmpty() ? TEXT( "" ) : *PostName );
				OutBeautifiedNames.Add( PrettyName );
				OutTestCommands.Add( OSSName.ToString() );
			}
		}
	}
}

/**
* Struct used to hold the user account credentials and the identity interface.
*/
struct FTestIdentityInterfaceStruct
{
	UWorld* CurrentWorld;
	FOnlineAccountCredentials* AccountCredentials;
	FTestIdentityInterface* IdentityInterface;
	bool bIsLogoutTest;
	OSSUtilsTestHelper::type TestStatus;

	/**
	* Constructor for the struct.
	* @param InUsername is the valid username that can be used with the OSS service.
	* @param InPassword is the valid password for the given username and OSS service.
	* @param InSubsystem is the OSS service to be used.
	*/
	FTestIdentityInterfaceStruct( const FString& InUsername, const FString& InPassword, const FString& InAccountType, const FString& InSubsystem )
		: CurrentWorld( GEngine->GetWorld() )
		, AccountCredentials( new FOnlineAccountCredentials( InAccountType, InUsername, InPassword ) )
		, IdentityInterface( new FTestIdentityInterface( InSubsystem ) )
		, bIsLogoutTest( false )
		, TestStatus( OSSUtilsTestHelper::NotStarted )
	{
	}
};

/**
* Latent command used to run the login and logout tests.
* @param TestStruct is the name of the struct that holds the test identity information.
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER( FTestIdentityInterfaceLoginOut, FTestIdentityInterfaceStruct, TestStruct );

bool FTestIdentityInterfaceLoginOut::Update()
{
	switch ( TestStruct.TestStatus )
	{
	case OSSUtilsTestHelper::NotStarted:
		UE_LOGF( OSSUtilsTestLog, Log, "Starting the 'Login' or 'Logout' functional test." );
		TestStruct.TestStatus = OSSUtilsTestHelper::BeginTest;
		break;
	case OSSUtilsTestHelper::BeginTest:
		UE_LOGF( OSSUtilsTestLog, Log, "Attempting to %ls.", 
			TestStruct.bIsLogoutTest ? TEXT( "'Logout'" ) : TEXT( "'Login'" ) );

		TestStruct.IdentityInterface->Test( 
			TestStruct.CurrentWorld, 
			*TestStruct.AccountCredentials, 
			TestStruct.bIsLogoutTest );
		TestStruct.TestStatus = OSSUtilsTestHelper::PerformingTest;
		break;
	case OSSUtilsTestHelper::PerformingTest:
		if ( TestStruct.IdentityInterface->GetTestStatus() ) return true;
	default:
		break;
	}

	return false;
}

/**
* This Login and Logout functional test is intended to test that a user can
* login and logout from the service.
* Use -OSSIDTESTUSER='TestUser' and -OSSIDTESTPSSWD='YouPassword'
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST( FIdenitiyInterfacetFunctionalLoginLogoutTest, "System.OSS", EAutomationTestFlags::ClientContext | EAutomationTestFlags::StressFilter)

void FIdenitiyInterfacetFunctionalLoginLogoutTest::GetTests( TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands ) const
{
	OSSUtilsTestHelper::GetOSSNamesForTesting(
		OutBeautifiedNames,
		OutTestCommands,
		TEXT("Functional Tests.Log in and then log out"));
}

bool FIdenitiyInterfacetFunctionalLoginLogoutTest::RunTest( const FString& Parameters )
{
	// Get the user info that is used for logging in and out of the subsystem.
	FString Username = TEXT( "" );
	FString Password = TEXT( "" );
	if ( !OSSUtilsTestHelper::GetUserInfo( Username, Password ) ) return false;

	// Two identity structs are created because the test function will delete the object after each run.
	FTestIdentityInterfaceStruct LoginTestStruct( Username, Password, TEXT( "epic" ), Parameters );
	FTestIdentityInterfaceStruct LogoutTestStruct( Username, Password, TEXT("epic"), Parameters );
	LogoutTestStruct.bIsLogoutTest = true;

	/* The first latent command will run the login test.*/
	ADD_LATENT_AUTOMATION_COMMAND( FTestIdentityInterfaceLoginOut( LoginTestStruct ) );
	/* This second latent command will run the logout test.*/
	ADD_LATENT_AUTOMATION_COMMAND( FTestIdentityInterfaceLoginOut( LogoutTestStruct ) );

	return true;
}


IMPLEMENT_COMPLEX_AUTOMATION_TEST(FTestIdentityInterfaceLogoutWhileNotLoggedIn, "System.OSS", EAutomationTestFlags::ClientContext | EAutomationTestFlags::StressFilter)

void FTestIdentityInterfaceLogoutWhileNotLoggedIn::GetTests( TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands ) const
{
	OSSUtilsTestHelper::GetOSSNamesForTesting( 
		OutBeautifiedNames, 
		OutTestCommands, 
		TEXT( "Functional Tests.Logging out while not logged in" ) );
}

bool FTestIdentityInterfaceLogoutWhileNotLoggedIn::RunTest(const FString& Parameters)
{
	// Get the user info that is used for logging in and out of the subsystem.
	FString Username = TEXT( "" );
	FString Password = TEXT( "" );
	if ( !OSSUtilsTestHelper::GetUserInfo( Username, Password ) ) return false;

	FTestIdentityInterfaceStruct LogoutTestStruct( Username, Password, TEXT( "epic" ), Parameters );
	LogoutTestStruct.bIsLogoutTest = true;

	// Stop the test here if the user is currently logged in.
	if ( LogoutTestStruct.IdentityInterface->IsTheUserLoggedIn() )
	{
		AddError( TEXT( "Unable to test as the user is currently logged in." ) );
		return false;
	}

	/* This latent command will run the logout test.*/
	ADD_LATENT_AUTOMATION_COMMAND( FTestIdentityInterfaceLoginOut( LogoutTestStruct ) );

	return true;
}

#endif //#if WITH_DEV_AUTOMATION_TESTS
