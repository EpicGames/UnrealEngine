// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestAchievementsInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * The process of this test is:
 *
 * [Read Achievements] -> [ReadAchievementsDelegate]
 * -> [Read Achievement Descriptions] -> [Read Achievement Descriptions Delegate]
 * -> -> [Write Achievements] -> [Achievement Unlocked Delegate]
 * -> -> -> [delete self]
 */


void FTestAchievementsInterface::Test(UWorld* InWorld)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FTestAchievementsInterface::Test"));

	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(InWorld, FName(*SubsystemName));
	check(OnlineSub); 

	if (OnlineSub->GetIdentityInterface().IsValid())
	{
		UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);

		OnlineAchievements = OnlineSub->GetAchievementsInterface();
	}

	if( OnlineAchievements.IsValid() && UserId.IsValid() )
	{
		ReadAchievements();
	}
	else
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: OSS [%s] does not have a valid achievement interface or identity interface for this test to run."), *SubsystemName );
		delete this;
	}
}


void FTestAchievementsInterface::ReadAchievements()
{
	// Add our delegate and read achievements
	OnlineAchievements->QueryAchievements( *UserId.Get(), FOnQueryAchievementsCompleteDelegate::CreateRaw( this, &FTestAchievementsInterface::OnQueryAchievementsComplete ) );
}


void FTestAchievementsInterface::OnQueryAchievementsComplete(const FUniqueNetId& PlayerId, const bool bWasSuccessful)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FTestAchievementsInterface::OnQueryAchievementsComplete"));

	if (bWasSuccessful)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded Achievements"));

		TArray<FOnlineAchievement> PlayerAchievements;
		if (OnlineAchievements->GetCachedAchievements(*UserId.Get(), PlayerAchievements) != EOnlineCachedResult::Success || PlayerAchievements.Num() == 0)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: Either GetCachedAchievements() failed or number of achievements is 0"));
			delete this;
			return;
		}

		UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Number of Achievements: %d"), PlayerAchievements.Num());
		for(int32 Idx = 0; Idx < PlayerAchievements.Num(); ++Idx)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT(" Achievement %d: %s"), Idx, *PlayerAchievements[ Idx ].ToDebugString());
		}

		QueryAchievementDescriptions( PlayerId );
	}
	else
	{
		// if our test failed, be sure to clean itself up
		UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("TEST FAILED: Failed to Load Achievements"));
		delete this;
	}
}


void FTestAchievementsInterface::QueryAchievementDescriptions(const FUniqueNetId& PlayerId)
{
	// Add our delegate and read achievement descriptions
	OnlineAchievements->QueryAchievementDescriptions( PlayerId, FOnQueryAchievementsCompleteDelegate::CreateRaw( this, &FTestAchievementsInterface::OnQueryAchievementDescriptionsComplete ) );
}


void FTestAchievementsInterface::OnQueryAchievementDescriptionsComplete(const FUniqueNetId& PlayerId, const bool bWasSuccessful)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FTestAchievementsInterface::OnQueryAchievementDescriptionsComplete"));

	if (bWasSuccessful)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Loaded Achievement descriptions"));

		TArray<FOnlineAchievement> PlayerAchievements;
		if (OnlineAchievements->GetCachedAchievements(*UserId.Get(), PlayerAchievements) != EOnlineCachedResult::Success || PlayerAchievements.Num() == 0)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: Either GetCachedAchievements() failed or number of achievements is 0"));
			delete this;
			return;
		}

		for(int32 Idx = 0; Idx < PlayerAchievements.Num(); ++Idx)
		{
			FOnlineAchievementDesc Desc;
			if (OnlineAchievements->GetCachedAchievementDescription(PlayerAchievements[ Idx ].Id, Desc) != EOnlineCachedResult::Success)
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Failed to GetCachedAchievementDescription() for achievement '%s'"),
					*PlayerAchievements[ Idx ].Id
					);
				delete this;
				return;
			}
			UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT(" Descriptor for achievement '%s': %s"), *PlayerAchievements[ Idx ].Id, *Desc.ToDebugString());
		}
		
		WriteAchievements();
	}
	else
	{
		// if our test failed, be sure to clean itself up
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: Failed to Load Achievement descriptions"));
		delete this;
	}
}


void FTestAchievementsInterface::WriteAchievements()
{
	WriteObject = MakeShareable(new FOnlineAchievementsWrite());
	FOnlineAchievementsWriteRef WriteObjectRef = WriteObject.ToSharedRef();

	TArray<FOnlineAchievement> PlayerAchievements;
	if (OnlineAchievements->GetCachedAchievements(*UserId.Get(), PlayerAchievements) != EOnlineCachedResult::Success || PlayerAchievements.Num() == 0)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: Either GetCachedAchievements() failed or number of achievements is 0"));
		delete this;
		return;
	}

	if (PlayerAchievements.Num() >= 3)
	{
		// report progress on first, last and median achievements
		WriteObject->SetFloatStat(PlayerAchievements[ 0 ].Id, 1.0f);
		WriteObject->SetFloatStat(PlayerAchievements[ PlayerAchievements.Num() - 1 ].Id, 50.0f);
		WriteObject->SetFloatStat(PlayerAchievements[ PlayerAchievements.Num() / 2 ].Id, 100.0f);
	}
	else
	{
		// report progress on all achievements
		for (int32 IdxAch = 0; IdxAch < PlayerAchievements.Num(); ++IdxAch)
		{
			WriteObject->SetFloatStat(PlayerAchievements[ IdxAch ].Id, 50.0f);
		}
	}

	// Add our delegate and write some achievements to the server
	OnAchievementUnlockedDelegateHandle = OnlineAchievements->AddOnAchievementUnlockedDelegate_Handle( OnAchievementUnlockedDelegate );
	OnlineAchievements->WriteAchievements( *UserId.Get(), WriteObjectRef, OnAchievementsWrittenDelegate );
}


void FTestAchievementsInterface::OnAchievementsWritten(const FUniqueNetId& PlayerId, bool bWasSuccessful)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("FTestAchievementsInterface::OnAchievementsWritten( bWasSuccessful = %s )"), bWasSuccessful ? TEXT("true") : TEXT("false"));

	// Clear our delegate and delete this test.
	OnlineAchievements->ClearOnAchievementUnlockedDelegate_Handle(OnAchievementUnlockedDelegateHandle);

	if (bWasSuccessful && WriteObject->WriteState == EOnlineAsyncTaskState::Done)
	{
#if !UE_BUILD_SHIPPING
		OnlineAchievements->ResetAchievements(*UserId.Get());
#endif // !UE_BUILD_SHIPPING
		// TODO: spin for a while?
		UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("TEST COMPLETED SUCCESSFULLY."));
	}
	else
	{
		if (WriteObject->WriteState != EOnlineAsyncTaskState::Done)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: WriteObject->WriteState is not in state %s, but instead %s"),
				EOnlineAsyncTaskState::ToString(EOnlineAsyncTaskState::Done),
				EOnlineAsyncTaskState::ToString(WriteObject->WriteState)
				);
		}

		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("TEST FAILED: Write did not complete successfully") );
		}
	}

	delete this;
}

void FTestAchievementsInterface::OnAchievementsUnlocked(const FUniqueNetId& PlayerId, const FString& AchievementId)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Display, TEXT("Achievement Unlocked - %s"), *AchievementId);
}

#endif //WITH_DEV_AUTOMATION_TESTS
