// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "OnlineSubsystem.h"
#include "Interfaces/OnlineAchievementsInterface.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the Achievements interface
 */
 class FTestAchievementsInterface
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;

	/** The online interface to use for testing */
	IOnlineAchievementsPtr OnlineAchievements;

	/** Delegate called when we have wrote/failed to write achievements to the server */
	FOnAchievementsWrittenDelegate OnAchievementsWrittenDelegate;

	/** Delegate called when an achievement is unlocked on the server */
	FOnAchievementUnlockedDelegate OnAchievementUnlockedDelegate;

	/** The id of the player we are testing achievements for */
	FUniqueNetIdPtr UserId;

	/** Leaderboard read object */
	FOnlineAchievementsWritePtr WriteObject;

	/** OnAchievementUnlocked delegate handle */
	FDelegateHandle OnAchievementUnlockedDelegateHandle;

	/** Hidden on purpose */
	FTestAchievementsInterface()
		: SubsystemName()
	{
	}

	/**
	 * Test the OSS capacity to read achievements from a server
	 */
	void ReadAchievements();

	/**
	 * Called when the read achievements request from the server is complete
	 *
	 * @param PlayerId The player id who is responsible for this delegate being fired
	 * @param bWasSuccessful true if the server responded successfully to the request
	 */
	void OnQueryAchievementsComplete(const FUniqueNetId& PlayerId, const bool bWasSuccessful);
	
	/**
	 * Test the OSS capacity to read achievement descriptions from a server
	 */
	void QueryAchievementDescriptions(const FUniqueNetId& PlayerId);

	/**
	 * Called when the read achievement descriptions request from the server is complete
	 *
	 * @param PlayerId The player id who is responsible for this delegate being fired
	 * @param bWasSuccessful true if the server responded successfully to the request
	 */
	void OnQueryAchievementDescriptionsComplete(const FUniqueNetId& PlayerId, const bool bWasSuccessful);

	/**
	 * Test the OSS capacity to write achievementsto the server. We will write one that completes an achievement
	 */
	void WriteAchievements();

	/**
	 * Called when achievemnets have been written to the server.
	 *
	 * @param PlayerId The player id who is responsible for this delegate being fired
	 * @param bWasSuccessful true if the server responded successfully to the request
	 */
	void OnAchievementsWritten(const FUniqueNetId& PlayerId, bool bWasSuccessful);

	/**
	 * Called when an achievement is unlocked on the server
	 *
	 * @param PlayerId The player id who is responsible for this delegate being fired
	 * @param AchievementId The achievement id of the achievement which has been unlocked
	 */
	void OnAchievementsUnlocked(const FUniqueNetId& PlayerId, const FString& AchievementId);

 public:
	/**
	 * ctor which is used to create a test object as we require an OSS name, of which, we are testing
	 *
	 * @param InSubsystemName the subsystem to test
	 */
	FTestAchievementsInterface(const FString& InSubsystemName)
		: SubsystemName(InSubsystemName)
		, OnlineAchievements(NULL)
		, OnAchievementsWrittenDelegate( FOnAchievementsWrittenDelegate::CreateRaw(this, &FTestAchievementsInterface::OnAchievementsWritten))
		, OnAchievementUnlockedDelegate(FOnAchievementUnlockedDelegate::CreateRaw(this, &FTestAchievementsInterface::OnAchievementsUnlocked))
	{
	}

	/**
	 * Kicks off all of the testing process
	 */
	void Test(class UWorld* InWorld);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
