// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestCloudInterface.h"
#include "Misc/Paths.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestCloudInterface::Test(UWorld* InWorld)
{
	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(InWorld, FName(*Subsystem));
	check(OnlineSub); 

	if (OnlineSub->GetIdentityInterface().IsValid())
	{
		UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
	}
	if (UserId.IsValid())
	{
		// Cache interfaces
		UserCloud = OnlineSub->GetUserCloudInterface();
		if (UserCloud.IsValid())
		{
			// Setup delegates
			EnumerationDelegate = FOnEnumerateUserFilesCompleteDelegate::CreateRaw(this, &FTestCloudInterface::OnEnumerateUserFilesComplete);
			OnWriteUserCloudFileCompleteDelegate = FOnWriteUserFileCompleteDelegate::CreateRaw(this, &FTestCloudInterface::OnWriteUserCloudFileComplete);
			OnWriteUserFileCanceledDelegate = FOnWriteUserFileCanceledDelegate::CreateRaw(this, &FTestCloudInterface::OnCancelWriteUserCloudFileComplete);
			OnReadEnumeratedUserFilesCompleteDelegate = FOnReadUserFileCompleteDelegate::CreateRaw(this, &FTestCloudInterface::OnReadEnumeratedUserFilesComplete);
			OnDeleteEnumeratedUserFilesCompleteDelegate = FOnDeleteUserFileCompleteDelegate::CreateRaw(this, &FTestCloudInterface::OnDeleteEnumeratedUserFilesComplete);

			SharedCloud = OnlineSub->GetSharedCloudInterface();
			if (SharedCloud.IsValid())
			{
				OnWriteSharedCloudFileCompleteDelegate = FOnWriteSharedFileCompleteDelegate::CreateRaw(this, &FTestCloudInterface::OnWriteSharedCloudFileComplete);
				OnReadEnumerateSharedFileCompleteDelegate = FOnReadSharedFileCompleteDelegate::CreateRaw(this, &FTestCloudInterface::OnReadEnumeratedSharedFileCompleteDelegate);

				// Pre-generated content
				SharedCloud->GetDummySharedHandlesForTest(RandomSharedFileHandles);
			}
		}
		else
		{
			UE_LOG_ONLINE_CLOUD(Warning, TEXT("Cloud test failed.  Cloud API not supported."));
			delete this;
		}
	}
	else
	{
		UE_LOG_ONLINE_CLOUD(Warning, TEXT("Cloud test failed.  No logged in user."));
		delete this;
	}
}

bool FTestCloudInterface::Tick( float DeltaTime )
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTestCloudInterface_Tick);

	if (CurrentTestPhase != LastTestPhase)
	{
		if (!bOverallSuccess)
		{
			UE_LOG_ONLINE_CLOUD(Log, TEXT("Testing failed in phase %d"), EnumToUnderlyingType(LastTestPhase));
			CurrentTestPhase = EUserCloudTestPhase::End;
		}
		if (!SharedCloud.IsValid() && 
			CurrentTestPhase > EUserCloudTestPhase::CancelWritingFiles)
		{
			UE_LOG_ONLINE_CLOUD(Log, TEXT("Skipping shared cloud tests"));
			CurrentTestPhase = EUserCloudTestPhase::End;
		}

		switch(CurrentTestPhase)
		{
		case EUserCloudTestPhase::EnumerateFiles:
			// Enumerate all files at the beginning
			EnumerateUserFiles();
			break;
		case EUserCloudTestPhase::WriteFiles:
			// Write out N files for the user of various sizes/names and enumerate
			// @todo: make the file count configurable on a per-platform basis, since different platforms have different file count limits
			OnWriteUserCloudFileCompleteDelegateHandle = WriteNUserCloudFiles(*UserId, FString(TEXT("UserCloud.bin")), 15, false, OnWriteUserCloudFileCompleteDelegate);
			break;
		case EUserCloudTestPhase::ReadFiles:
			// Read in N files for the user and enumerate
			OnReadEnumeratedUserFilesCompleteDelegateHandle = ReadEnumeratedUserFiles(OnReadEnumeratedUserFilesCompleteDelegate);
			break;
		case EUserCloudTestPhase::DeleteFilesLocally:
			// Delete the N files for the user and enumerate (should be removed locally but remain in the cloud)
			// We run local deletion first because the opposite would mean that by the time we attempt a local deletion, EnumerateUserFiles may return an empty list
			OnDeleteEnumeratedUserFilesCompleteDelegateHandle = DeleteEnumeratedUserFiles(false, true, OnDeleteEnumeratedUserFilesCompleteDelegate);
			break;
		case EUserCloudTestPhase::DeleteFilesInCloud:
			// Forget the N files for the user and enumerate (should be removed in the cloud but remain locally)
			OnDeleteEnumeratedUserFilesCompleteDelegateHandle = DeleteEnumeratedUserFiles(true, false, OnDeleteEnumeratedUserFilesCompleteDelegate);
			break;
		case EUserCloudTestPhase::CancelWritingFiles:
			OnWriteUserCloudFileCompleteDelegateHandle = WriteNUserCloudFiles(*UserId, FString(TEXT("UserCloud.bin")), 5, true, OnWriteUserCloudFileCompleteDelegate);
			OnWriteUserFileCanceledDelegateHandle = CancelWriteNUserCloudFiles(*UserId, FString(TEXT("UserCloud.bin")), 5, OnWriteUserFileCanceledDelegate);
			break;
		case EUserCloudTestPhase::WriteSharedFiles:
			// Write out N files for the user and share with the cloud and enumerate
			OnWriteSharedCloudFileCompleteDelegateHandle = WriteNSharedCloudFiles(*UserId, FString(TEXT("SharedCloud.bin")), 15, OnWriteSharedCloudFileCompleteDelegate);
			break;
		case EUserCloudTestPhase::ReadSharedFiles:
			// Read in N files given the shared handles that this user generated
			OnReadEnumerateSharedFileCompleteDelegateHandle = ReadEnumeratedSharedFiles(false, OnReadEnumerateSharedFileCompleteDelegate);
			break;
		case EUserCloudTestPhase::DeleteSharedFilesInCloud:
			// Delete, marking all the files as only forgotten
			OnDeleteEnumeratedUserFilesCompleteDelegateHandle = DeleteEnumeratedUserFiles(true, false, OnDeleteEnumeratedUserFilesCompleteDelegate);
			break;
		case EUserCloudTestPhase::ReadRandomSharedFiles:
			// Try to read in the N random files from their shared handles
			OnReadEnumerateSharedFileCompleteDelegateHandle = ReadEnumeratedSharedFiles(true, OnReadEnumerateSharedFileCompleteDelegate);
			break;
		case EUserCloudTestPhase::DeleteSharedFilesTotally:
			// Delete, marking all the files for permanent removal
			OnDeleteEnumeratedUserFilesCompleteDelegateHandle = DeleteEnumeratedUserFiles(true, true, OnDeleteEnumeratedUserFilesCompleteDelegate);
			break;	
		case EUserCloudTestPhase::End:
			bOverallSuccess = bOverallSuccess && Cleanup();
			UE_LOG_ONLINE_CLOUD(Log, TEXT("TESTING COMPLETE Success:%s!"), bOverallSuccess ? TEXT("true") : TEXT("false"));
			delete this;
			return false;
		}
		LastTestPhase = CurrentTestPhase;
	}
	return true;
}

bool FTestCloudInterface::Cleanup()
{
	bool bSuccess = true;
	if (UserCloud.IsValid() &&
		!UserCloud->ClearFiles(*UserId))
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Failed to cleanup user files"));
		bSuccess = false;
	}
	if (SharedCloud.IsValid() &&
		!SharedCloud->ClearSharedFiles())
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Failed to cleanup shared files"));
		bSuccess = false;
	}

	return bSuccess;
}

void FTestCloudInterface::EnumerateUserFiles()
{
	EnumerationDelegateHandle = UserCloud->AddOnEnumerateUserFilesCompleteDelegate_Handle(EnumerationDelegate);
	UserCloud->EnumerateUserFiles(*UserId);
}

void FTestCloudInterface::OnEnumerateUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& InUserId)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("OnEnumerateUserFilesComplete Success:%d UserId:%s"), bWasSuccessful, *InUserId.ToDebugString());
	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	TArray<FCloudFileHeader> UserFiles;
	UserCloud->GetUserFileList(InUserId, UserFiles);
	UserCloud->ClearOnEnumerateUserFilesCompleteDelegate_Handle(EnumerationDelegateHandle);

	for (int32 Idx=0; Idx<UserFiles.Num(); Idx++)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("\t%d FileName:%s DLName:%s Hash:%s Size:%d"),
			Idx, *UserFiles[Idx].FileName, *UserFiles[Idx].DLName, *UserFiles[Idx].Hash, UserFiles[Idx].FileSize);
	}

	// Enumeration always advances the test phase
	CurrentTestPhase++;
}

void FTestCloudInterface::WriteRandomFile(TArray<uint8>& Buffer, int32 Size)
{
	Buffer.Empty(Size);
	Buffer.AddUninitialized(Size);
	for (int32 i=0; i<Size; i++)
	{
		Buffer[i] = i % 255;
	}
}

FDelegateHandle FTestCloudInterface::WriteNUserCloudFiles(const FUniqueNetId& InUserId, const FString& FileNameBase, int32 FileCount, bool bCancelWrite, FOnWriteUserFileCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Writing %d files to the cloud for user %s. bCancelWrite=%s"), FileCount, *InUserId.ToDebugString(), *LexToString(bCancelWrite));

	WriteUserCloudFileCount = FileCount;
	FDelegateHandle Result = UserCloud->AddOnWriteUserFileCompleteDelegate_Handle(Delegate);

	TArray<uint8> DummyData;
	for (int32 FileIdx=0; FileIdx<FileCount; FileIdx++)
	{
		// @todo: make the dummy data size configurable on a per-platform basis, since different platforms (e.g. PS4) have different size limits
		//WriteRandomFile(DummyData, FMath::TruncToInt(FMath::FRandRange(1024, 100*1024)));
		WriteRandomFile(DummyData, FMath::TruncToInt(FMath::FRandRange(256.f, 1024.f)));
		UserCloud->WriteUserFile(InUserId, FString::Printf(TEXT("%s%d.%s"), *FPaths::GetBaseFilename(FileNameBase), FileIdx, *FPaths::GetExtension(FileNameBase)), DummyData);
	}

	bIsFailureExpected = bCancelWrite;

	return Result;
}

FDelegateHandle FTestCloudInterface::WriteNSharedCloudFiles(const FUniqueNetId& InUserId, const FString& FileNameBase, int32 FileCount, FOnWriteSharedFileCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Writing %d files to the cloud and sharing for user %s"), FileCount, *InUserId.ToDebugString());

	FString FileName(FileNameBase);

	WriteSharedCloudFileCount = FileCount;
	FDelegateHandle Result = SharedCloud->AddOnWriteSharedFileCompleteDelegate_Handle(Delegate);

	TArray<uint8> DummyData;
	for (int32 FileIdx=0; FileIdx<FileCount; FileIdx++)
	{
		WriteRandomFile(DummyData, FMath::TruncToInt(FMath::FRandRange(1024.f, 100*1024.f)));
		SharedCloud->WriteSharedFile(InUserId, FString::Printf(TEXT("%s%d.%s"), *FPaths::GetBaseFilename(FileName), FileIdx, *FPaths::GetExtension(FileName)), DummyData);
	}

	return Result;
}

void FTestCloudInterface::OnWriteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& FileName)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Write user file complete Success:%d UserId:%s FileName:%s"), bWasSuccessful, *InUserId.ToDebugString(), *FileName);
	
	bOverallSuccess = bOverallSuccess && (bWasSuccessful || bIsFailureExpected);

	static int32 NumWrittenCount = 0;
	NumWrittenCount++;
	if (NumWrittenCount == WriteUserCloudFileCount)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Write %d User Files Complete!"), NumWrittenCount);
		UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
		UserCloud->ClearOnWriteUserFileCompleteDelegate_Handle(OnWriteUserCloudFileCompleteDelegateHandle);
		NumWrittenCount = 0;
		bIsFailureExpected = false;
		EnumerateUserFiles();
	}
}

FDelegateHandle FTestCloudInterface::CancelWriteNUserCloudFiles(const FUniqueNetId& InUserId, const FString& FileNameBase, int32 FileCount, FOnWriteUserFileCanceledDelegate& Delegate)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Cancelling %d write operations for user %s"), FileCount, *InUserId.ToDebugString());

	WriteUserFileCanceledCount = FileCount;
	FDelegateHandle Result = UserCloud->AddOnWriteUserFileCanceledDelegate_Handle(Delegate);

	for (int32 FileIdx = 0; FileIdx < FileCount; FileIdx++)
	{
		UserCloud->CancelWriteUserFile(InUserId, FString::Printf(TEXT("%s%d.%s"), *FPaths::GetBaseFilename(FileNameBase), FileIdx, *FPaths::GetExtension(FileNameBase)));
	}

	return Result;
}

void FTestCloudInterface::OnCancelWriteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& FileName)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Cancellation of user file writing complete Success:%d UserId:%s FileName:%s"), bWasSuccessful, *InUserId.ToDebugString(), *FileName);

	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	static int32 NumCancelledWriteCount = 0;
	NumCancelledWriteCount++;
	if (NumCancelledWriteCount == WriteUserFileCanceledCount)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Cancel %d User File Writings Complete!"), NumCancelledWriteCount);
		UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
		UserCloud->ClearOnWriteUserFileCanceledDelegate_Handle(OnWriteUserFileCanceledDelegateHandle);
		NumCancelledWriteCount = 0;
		// We don't enumerate user files here, the failed write file delegate will do it
	}
}

FDelegateHandle FTestCloudInterface::ReadEnumeratedUserFiles(FOnReadUserFileCompleteDelegate& Delegate)
{
	TArray<FCloudFileHeader> UserFiles;
	UserCloud->GetUserFileList(*UserId, UserFiles);

	ReadUserFileCount = UserFiles.Num();

	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Reading %d enumerated files for user %s"), ReadUserFileCount, *UserId->ToDebugString());

	FDelegateHandle Result;
	if (ReadUserFileCount > 0)
	{
		Result = UserCloud->AddOnReadUserFileCompleteDelegate_Handle(Delegate);
		for (int32 Idx=0; Idx < ReadUserFileCount; Idx++)
		{
			UE_LOG_ONLINE_CLOUD(Log, TEXT("\tFileName:%s Size:%d"), *UserFiles[Idx].FileName, UserFiles[Idx].FileSize);
			UserCloud->ReadUserFile(*UserId, UserFiles[Idx].FileName);
		}
	}
	else
	{
		EnumerateUserFiles();
	}

	return Result;
}

void FTestCloudInterface::OnReadEnumeratedUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& FileName)
{
	int32 FileSize = 0;
	char *buffer = NULL;
	if (bWasSuccessful)
	{
		TArray<uint8> FileContents;
		bWasSuccessful = UserCloud->GetFileContents(InUserId, FileName, FileContents);
		FileSize = FileContents.Num();
		buffer = (char *) FileContents.GetData();
	}

	UE_LOG_ONLINE_CLOUD(Log, TEXT("Read user file complete Success:%d UserId:%s FileName:%s Size:%d"), bWasSuccessful, *InUserId.ToDebugString(), *FileName, FileSize);
	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	static int32 NumReadCount = 0;
	NumReadCount++;
	if (NumReadCount == ReadUserFileCount)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Read %d User Files Complete!"), NumReadCount);
		UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
		UserCloud->ClearOnReadUserFileCompleteDelegate_Handle(OnReadEnumeratedUserFilesCompleteDelegateHandle);
		NumReadCount = 0;
		EnumerateUserFiles();
	}
}

FDelegateHandle FTestCloudInterface::DeleteEnumeratedUserFiles(bool bCloudDelete, bool bLocalDelete, FOnDeleteUserFileCompleteDelegate& Delegate)
{
	TArray<FCloudFileHeader> UserFiles;
	UserCloud->GetUserFileList(*UserId, UserFiles);

	DeleteUserFileCount = UserFiles.Num();

	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Deleting %d files from the cloud for user %s CLOUD: %s LOCAL: %s"), DeleteUserFileCount, *UserId->ToDebugString(), bCloudDelete ? TEXT("true") : TEXT("false"), bLocalDelete ? TEXT("true") : TEXT("false"));
	FDelegateHandle Result;
	if (DeleteUserFileCount > 0)
	{
		Result = UserCloud->AddOnDeleteUserFileCompleteDelegate_Handle(Delegate);
		for (int32 Idx=0; Idx < DeleteUserFileCount; Idx++)
		{
			UE_LOG_ONLINE_CLOUD(Log, TEXT("\tFileName:%s Size:%d"), *UserFiles[Idx].FileName, UserFiles[Idx].FileSize);
			UserCloud->DeleteUserFile(*UserId, UserFiles[Idx].FileName, bCloudDelete, bLocalDelete);
		}
	}
	else
	{
		EnumerateUserFiles();
	}

	return Result;
}

void FTestCloudInterface::OnDeleteEnumeratedUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& FileName)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Delete user file complete Success:%d UserId:%s FileName:%s"), bWasSuccessful, *InUserId.ToDebugString(), *FileName);
	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	static int32 NumDeletedCount = 0;
	NumDeletedCount++;
	if (NumDeletedCount == DeleteUserFileCount)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Delete %d User Files Complete!"), NumDeletedCount);
		UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
		UserCloud->ClearOnDeleteUserFileCompleteDelegate_Handle(OnDeleteEnumeratedUserFilesCompleteDelegateHandle);
		NumDeletedCount = 0;
		EnumerateUserFiles();
	}
}

void FTestCloudInterface::OnWriteSharedCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& FileName, const TSharedRef<FSharedContentHandle>& SharedHandle)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Write shared file complete Success:%d UserId:%s FileName:%s SharedHandle:%s"), bWasSuccessful, *InUserId.ToDebugString(), *FileName, *SharedHandle->ToDebugString());
	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	CloudFileHandles.Add(SharedHandle);

	static int32 NumWrittenCount = 0;
	NumWrittenCount++;
	if (NumWrittenCount == WriteSharedCloudFileCount)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Write %d Shared Files Complete!"), NumWrittenCount);
		UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
		SharedCloud->ClearOnWriteSharedFileCompleteDelegate_Handle(OnWriteSharedCloudFileCompleteDelegateHandle);
		NumWrittenCount = 0;
		// Enumeration will put the newly shared files into this user's list
		EnumerateUserFiles();
	}
}

FDelegateHandle FTestCloudInterface::ReadEnumeratedSharedFiles(bool bUseRandom, FOnReadSharedFileCompleteDelegate& Delegate)
{
	if (bUseRandom)
	{
		ReadSharedFileCount = RandomSharedFileHandles.Num();
	}
	else
	{
		ReadSharedFileCount = CloudFileHandles.Num();
	}
	
	UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Reading %d enumerated shared files"), ReadSharedFileCount);

	FDelegateHandle Result;
	if (ReadSharedFileCount > 0)
	{
		Result = SharedCloud->AddOnReadSharedFileCompleteDelegate_Handle(Delegate);
		for (int32 Idx=0; Idx < ReadSharedFileCount; Idx++)
		{
			if (bUseRandom)
			{
				UE_LOG_ONLINE_CLOUD(Log, TEXT("\tHandle:%s"), *RandomSharedFileHandles[Idx]->ToDebugString());
				SharedCloud->ReadSharedFile(*RandomSharedFileHandles[Idx]);
			}
			else
			{
				UE_LOG_ONLINE_CLOUD(Log, TEXT("\tHandle:%s"), *CloudFileHandles[Idx]->ToDebugString());
				SharedCloud->ReadSharedFile(*CloudFileHandles[Idx]);
			}
		}
	}
	else
	{
		EnumerateUserFiles();
	}

	return Result;
}

void FTestCloudInterface::OnReadEnumeratedSharedFileCompleteDelegate(bool bWasSuccessful, const FSharedContentHandle& SharedHandle)
{
	UE_LOG_ONLINE_CLOUD(Log, TEXT("Read shared file complete Success:%d SharedHandle:%s"), bWasSuccessful, *SharedHandle.ToDebugString());
	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	static int32 NumReadCount = 0;
	NumReadCount++;
	if (NumReadCount == ReadSharedFileCount)
	{
		UE_LOG_ONLINE_CLOUD(Log, TEXT("Read %d Shared Files Complete!"), NumReadCount);
		UE_LOG_ONLINE_CLOUD(Log, TEXT("------------------------------------------------"));
		SharedCloud->ClearOnReadSharedFileCompleteDelegate_Handle(OnReadEnumerateSharedFileCompleteDelegateHandle);
		NumReadCount = 0;
		EnumerateUserFiles();
	}
}

#endif //WITH_DEV_AUTOMATION_TESTS
