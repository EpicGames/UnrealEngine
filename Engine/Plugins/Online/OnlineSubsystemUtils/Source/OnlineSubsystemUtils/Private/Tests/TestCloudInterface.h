// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Interfaces/OnlineUserCloudInterface.h"
#include "Interfaces/OnlineSharedCloudInterface.h"
#include "Online/CoreOnlineFwd.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the friends interface
 */
 class FTestCloudInterface : public FTSTickerObjectBase
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString Subsystem;

	/** Keep track of success across all functions and callbacks */
	bool bOverallSuccess;

	/** We'll set this to true when operations are expected to fail, like in the cancelled file write case */
	bool bIsFailureExpected;

	/** Convenient access to the cloud interfaces */
	IOnlineUserCloudPtr UserCloud;
	IOnlineSharedCloudPtr SharedCloud;

	/** Delegates to various cloud functionality triggered */
	FOnEnumerateUserFilesCompleteDelegate EnumerationDelegate;
	FOnWriteUserFileCompleteDelegate OnWriteUserCloudFileCompleteDelegate;
	FOnWriteUserFileCanceledDelegate OnWriteUserFileCanceledDelegate;
	FOnReadUserFileCompleteDelegate OnReadEnumeratedUserFilesCompleteDelegate;
	FOnDeleteUserFileCompleteDelegate OnDeleteEnumeratedUserFilesCompleteDelegate;
	FOnWriteSharedFileCompleteDelegate OnWriteSharedCloudFileCompleteDelegate;
	FOnReadSharedFileCompleteDelegate OnReadEnumerateSharedFileCompleteDelegate;

	/** Handles to those delegates */
	FDelegateHandle EnumerationDelegateHandle;
	FDelegateHandle OnWriteUserCloudFileCompleteDelegateHandle;
	FDelegateHandle OnWriteUserFileCanceledDelegateHandle;
	FDelegateHandle OnReadEnumeratedUserFilesCompleteDelegateHandle;
	FDelegateHandle OnDeleteEnumeratedUserFilesCompleteDelegateHandle;
	FDelegateHandle OnWriteSharedCloudFileCompleteDelegateHandle;
	FDelegateHandle OnReadEnumerateSharedFileCompleteDelegateHandle;

	/** Logged in UserId */
	FUniqueNetIdPtr UserId;

	/** Array of shared handles generated from the user share write call */
	TArray< TSharedRef<FSharedContentHandle> > CloudFileHandles;
	/** Static array of shared handles from another user */
	TArray< TSharedRef<FSharedContentHandle> > RandomSharedFileHandles;

	/** Number of files written to be used to clear the delegate */
	int32 WriteUserCloudFileCount;
	/** Number of files written to be used to clear the delegate */
	int32 WriteSharedCloudFileCount;
	/** Number of file writing cancellations to be used to clear the delegate */
	int32 WriteUserFileCanceledCount;
	/** Number of files read to be used to clear the delegate */
	int32 ReadUserFileCount;
	/** Number of files read to be used to clear the delegate */
	int32 ReadSharedFileCount;
	/** Number of files deleted to be used to clear the delegate */
	int32 DeleteUserFileCount;

	enum class EUserCloudTestPhase
	{
		Invalid,
		EnumerateFiles,
		WriteFiles,
		ReadFiles,
		DeleteFilesLocally,
		DeleteFilesInCloud,
		CancelWritingFiles,
		WriteSharedFiles,
		ReadSharedFiles,
		DeleteSharedFilesInCloud,
		ReadRandomSharedFiles,
		DeleteSharedFilesTotally,
		End
	};

	friend EUserCloudTestPhase& operator++(EUserCloudTestPhase& TestPhase) {
		const int TestPhaseInt = int(TestPhase) + 1;
		const int EndPhaseInt = int(EUserCloudTestPhase::End);
		TestPhase = EUserCloudTestPhase((TestPhaseInt > EndPhaseInt) ? EndPhaseInt : TestPhaseInt);
		return TestPhase;
	}

	friend EUserCloudTestPhase operator++(EUserCloudTestPhase& TestPhase, int) {
		const EUserCloudTestPhase Result = TestPhase;
		++TestPhase;
		return Result;
	}

	/** Current phase of testing */
	EUserCloudTestPhase CurrentTestPhase;
	/** Last phase of testing triggered */
	EUserCloudTestPhase LastTestPhase;

	/**
	 *	Fill a buffer with random data of a given file size
	 * @param Buffer buffer to fill
	 * @param Size size to allocate to the buffer
	 */
	void WriteRandomFile(TArray<uint8>& Buffer, int32 Size);

	/**
	 *	Enumerate all files in the cloud for the logged in user
	 */
	void EnumerateUserFiles();

	/**
	 *	Write out files marked for the cloud for the given user
	 * @param UserId user to write cloud files for
	 * @param FileNameBase filename to derive generated files from (appends 0-FileCount)
	 * @param FileCount number of files to write out
	 * @param Delegate delegate to trigger for each file written
	 */
	FDelegateHandle WriteNUserCloudFiles(const FUniqueNetId& UserId, const FString& FileNameBase, int32 FileCount, bool bCancelWrite, FOnWriteUserFileCompleteDelegate& Delegate);

	/**
	 *	Write out files marked for the cloud (and shared) for the given user
	 * @param UserId user to write cloud files for
	 * @param FileNameBase filename to derive generated files from (appends 0-FileCount)
	 * @param FileCount number of files to write out
	 * @param Delegate delegate to trigger for each file written
	 */
	FDelegateHandle WriteNSharedCloudFiles(const FUniqueNetId& UserId, const FString& FileNameBase, int32 FileCount, FOnWriteSharedFileCompleteDelegate& Delegate);

	/**
	 *	Cancel cloud file write operations for the given user
	 * @param UserId user to cancel cloud file write operations for
	 * @param FileNameBase filename to derive cancelled file names from (appends 0-FileCount)
	 * @param FileCount number of files to cancel
	 * @param Delegate delegate to trigger for each file write cancelled
	 */
	FDelegateHandle CancelWriteNUserCloudFiles(const FUniqueNetId& InUserId, const FString& FileNameBase, int32 FileCount, FOnWriteUserFileCanceledDelegate& Delegate);

	/**
	 *	Read cloud files currently enumerated for the logged in user
	 * (assumes EnumerateFiles has been called)
	 * @param Delegate delegate to trigger for each file read	
	 */	
	FDelegateHandle ReadEnumeratedUserFiles(FOnReadUserFileCompleteDelegate& Delegate);

	/**
	 *	Read shared cloud files currently enumerated
	 * 
	 * @param bUseRandom use the random array of existing shared files if true, current users shared files otherwise
	 * @param Delegate delegate to trigger for each file read	
	 */	
	FDelegateHandle ReadEnumeratedSharedFiles(bool bUseRandom, FOnReadSharedFileCompleteDelegate& Delegate);

	/**
	 *	Delete cloud files currently enumerated
	 *
	 * @param bCloudDelete delete from the cloud, but not local version
	 * @param bLocalDelete delete local copy of file, but not cloud version
	 * @param Delegate delegate to trigger for each file deleted
	 */
	FDelegateHandle DeleteEnumeratedUserFiles(bool bCloudDelete, bool bLocalDelete, FOnDeleteUserFileCompleteDelegate& Delegate);

	/**
	 *	Delegate triggered when all user files have been enumerated
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 */
	void OnEnumerateUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& UserId);

	/**
	 *	Delegate triggered for each user cloud file written
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename written to cloud
	 */
	void OnWriteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file write cancellation
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName name of the file for which writing was cancelled
	 */
	void OnCancelWriteUserCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file read
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename read from cloud
	 */
	void OnReadEnumeratedUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file deleted
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename deleted from cloud
	 */
	void OnDeleteEnumeratedUserFilesComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName);

	/**
	 *	Delegate triggered for each user cloud file shared 
	 * @param bWasSuccessful did the operation complete successfully
	 * @param UserId user that triggered the operation
	 * @param FileName filename written to cloud
	 * @param SharedHandle handle given to newly shared filed
	 */
	void OnWriteSharedCloudFileComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& FileName, const TSharedRef<FSharedContentHandle>& SharedHandle);

	/**
	 *	Delegate triggered for each shared file read from the cloud
	 * @param bWasSuccessful did the operation complete successfully
	 * @param SharedHandle shared handle for the read file
	 */
	void OnReadEnumeratedSharedFileCompleteDelegate(bool bWasSuccessful, const FSharedContentHandle& SharedHandle);

	/** Call various cloud cleanup functions and test for success */
	bool Cleanup();

	/** Hidden on purpose */
	FTestCloudInterface()
		: Subsystem()
	{
	}

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestCloudInterface(const FString& InSubsystem) :
		Subsystem(InSubsystem),
		bOverallSuccess(true),
		UserCloud(NULL),
		SharedCloud(NULL),
		WriteUserCloudFileCount(0),
		WriteSharedCloudFileCount(0),
		ReadUserFileCount(0),
		ReadSharedFileCount(0),
		DeleteUserFileCount(0),
		CurrentTestPhase(EUserCloudTestPhase::EnumerateFiles),
		LastTestPhase(EUserCloudTestPhase::Invalid)
	{
	}

	~FTestCloudInterface()
	{
		UserCloud = NULL;
		SharedCloud = NULL;
	}

	// FTSTickerObjectBase

	bool Tick( float DeltaTime ) override;

	// FTestCloudInterface

	/**
	 * Kicks off all of the testing process
	 */
	void Test(class UWorld* InWorld);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
