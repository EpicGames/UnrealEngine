// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestEntitlementsInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestEntitlementsInterface::Test(UWorld* InWorld)
{
	IOnlineSubsystem* OSS = Online::GetSubsystem(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);

	IdentityOSS = OSS->GetIdentityInterface();
	if (IdentityOSS.IsValid())
	{
		UserId = IdentityOSS->GetUniquePlayerId(LocalUserIdx);
	}
	if (UserId.IsValid() && UserId->IsValid())
	{
		EntitlementsOSS = OSS->GetEntitlementsInterface();
		if (EntitlementsOSS.IsValid())
		{
			// Add delegates for the various async calls
			OnQueryEntitlementsCompleteDelegateHandle = EntitlementsOSS->AddOnQueryEntitlementsCompleteDelegate_Handle(OnQueryEntitlementsCompleteDelegate);

			// kick off next test
			StartNextTest();
		}
		else
		{
			UE_LOG_ONLINE_ENTITLEMENT(Warning, TEXT("Entitlement test failed. Failed to get entitlement service API"));

			// done with the test
			FinishTest();
		}
	}
	else
	{
		UE_LOG_ONLINE_ENTITLEMENT(Warning, TEXT("Entitlement test failed. No logged in user"));

		// done with the test
		FinishTest();
	}
}

/**
 * Step through the various tests that should be run and initiate the next one
 */
void FTestEntitlementsInterface::StartNextTest()
{
	if (bQueryEntitlements)
	{
		EntitlementsOSS->QueryEntitlements(*UserId, FString());
	}
	else
	{
		FinishTest();
	}
}

void FTestEntitlementsInterface::FinishTest()
{
	if (EntitlementsOSS.IsValid())
	{
		// Clear delegates for the entitlements async calls
		EntitlementsOSS->ClearOnQueryEntitlementsCompleteDelegate_Handle(OnQueryEntitlementsCompleteDelegateHandle);
	}

	delete this;
}

void FTestEntitlementsInterface::OnQueryEntitlementsComplete(bool bWasSuccessful, const FUniqueNetId& InUserId, const FString& Namespace, const FString& Error)
{
	UE_LOG_ONLINE_ENTITLEMENT(Log, TEXT("enumerated entitlements. UserId=%s Result=%s Error=[%s]"), 
		*InUserId.ToDebugString(), bWasSuccessful ? TEXT("true") : TEXT("false"), *Error);

	// Now check em out
	TArray<TSharedRef<FOnlineEntitlement>> Entitlements;
	EntitlementsOSS->GetAllEntitlements(InUserId, FString(), Entitlements);

	for (auto It = Entitlements.CreateConstIterator(); It; ++It)
	{
		const TSharedRef<FOnlineEntitlement> Entry = *It;
		UE_LOG_ONLINE_ENTITLEMENT(Log, TEXT("	entitlement id=%s name=%s"),
			*Entry->Id, *Entry->Name);
	}

	// kick off next test
	bQueryEntitlements = false;
	StartNextTest();
}

#endif //WITH_DEV_AUTOMATION_TESTS
