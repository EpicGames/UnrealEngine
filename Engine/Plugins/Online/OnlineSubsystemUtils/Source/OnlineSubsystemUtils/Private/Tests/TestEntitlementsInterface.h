// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineEntitlementsInterface.h"

class Error;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the Mcp account creation/management process
 */
 class FTestEntitlementsInterface
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;
	/** Online services used by these tests */
	IOnlineIdentityPtr IdentityOSS;
	IOnlineEntitlementsPtr EntitlementsOSS;
	
	/** Delegates for callbacks of each test */
	FOnQueryEntitlementsCompleteDelegate OnQueryEntitlementsCompleteDelegate;

	/** Handle to the above delegates */
	FDelegateHandle OnQueryEntitlementsCompleteDelegateHandle;

	/** toggles for whether to run each test */
	bool bQueryEntitlements;

	/** Account registered by the test that should be used for all tests */
	FUniqueNetIdPtr UserId;
	/** Local user to run tests for */
	int32 LocalUserIdx;

	/** Hidden on purpose */
	FTestEntitlementsInterface()
		: SubsystemName()
		, LocalUserIdx(0)
	{
	}

	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/**
	 * Called when the async task for enumerating entitlements has completed
	 *
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param UserId of the user who was granted entitlements in this callback
	 * @param Error string representing the error condition if any
	 */
	void OnQueryEntitlementsComplete(bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Namespace, const FString& Error);

 public:
	
	FTestEntitlementsInterface(const FString& InSubsystemName)
		: SubsystemName(InSubsystemName)
		, IdentityOSS(NULL)
		, EntitlementsOSS(NULL)
		, OnQueryEntitlementsCompleteDelegate(FOnQueryEntitlementsCompleteDelegate::CreateRaw(this, &FTestEntitlementsInterface::OnQueryEntitlementsComplete))
		, bQueryEntitlements(true)
		, LocalUserIdx(0)
	{
	}

	/**
	 * Kicks off all of the testing process
	 */
	void Test(class UWorld* InWorld);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
