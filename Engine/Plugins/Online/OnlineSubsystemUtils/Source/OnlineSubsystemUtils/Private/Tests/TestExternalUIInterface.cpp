// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestExternalUIInterface.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestExternalUIInterface::PrepareTests()
{
	// Cache interfaces
	OnlineSub = IOnlineSubsystem::Get(FName(*SubsystemName));
	check(OnlineSub);

	ExternalUI = OnlineSub->GetExternalUIInterface();
	check(ExternalUI.IsValid());

	// Define and register delegates
	ExternalUIChangeDelegate = FOnExternalUIChangeDelegate::CreateRaw(this, &FTestExternalUIInterface::OnExternalUIChange);
	ExternalUIChangeDelegateHandle = ExternalUI->AddOnExternalUIChangeDelegate_Handle(ExternalUIChangeDelegate);
}

void FTestExternalUIInterface::Test()
{
	PrepareTests();

	// Are we testing at least one of our external UI?
	if (bTestLoginUI == false && bTestFriendsUI == false && bTestInviteUI == false && bTestAchievementsUI == false && bTestWebURL == false && bTestProfileUI == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Error, TEXT("ExternalUI test -- No UIs selected to test"));
		FinishTest();
	}
	else
	{
		StartNextTest();
	}
}

void FTestExternalUIInterface::TestSendMessage(const FString& InMsgRecepient, const FString& InMessage)
{
	PrepareTests();
	bIsolatedTest = true;

	if (InMsgRecepient.IsEmpty())
	{
		UE_LOG_ONLINE_EXTERNALUI(Error, TEXT("ExternalUI TestSendMessage -- InMsgRecepient must not be empty!"));
		FinishTest();
		return;
	}

	FUniqueNetIdPtr TargetUser = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(InMsgRecepient);
	if (!TargetUser.IsValid())
	{
		UE_LOG_ONLINE_EXTERNALUI(Error, TEXT("ExternalUI TestSendMessage -- Message target user is invalid!"));
		FinishTest();
		return;
	}

	FShowSendMessageParams MessageParams;
	MessageParams.DisplayMessage = FText::FromString(InMessage);

	bool bWasOpened = ExternalUI->ShowSendMessageToUserUI(0, *TargetUser, MessageParams, FOnShowSendMessageUIClosedDelegate::CreateRaw(this, &FTestExternalUIInterface::OnSendMessageUIClosed));
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("ExternalUI TestSendMessage -- Message test opened %d"), bWasOpened);
	if (!bWasOpened)
	{
		FinishTest();
	}
}

void FTestExternalUIInterface::TestStorePage(const FString& InProductId, bool bShouldAddToCart)
{
	PrepareTests();
	bIsolatedTest = true;

	if (InProductId.IsEmpty())
	{
		UE_LOG_ONLINE_EXTERNALUI(Error, TEXT("ExternalUI TestStorePage -- Product Id must not be empty!"));
		FinishTest();
		return;
	}

	FShowStoreParams StoreParams;
	StoreParams.ProductId = InProductId;
	StoreParams.bAddToCart = bShouldAddToCart;

	bool bWasOpened = ExternalUI->ShowStoreUI(0, StoreParams, FOnShowStoreUIClosedDelegate::CreateRaw(this, &FTestExternalUIInterface::OnStoreUIClosed));
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("ExternalUI TestStorePage -- Store test opened %d"), bWasOpened);
	if (!bWasOpened)
	{
		FinishTest();
	}
}

void FTestExternalUIInterface::FinishTest()
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("FTestExternalUIInterface::FinishTest -- completed testing"));
	ExternalUI->ClearOnExternalUIChangeDelegate_Handle(ExternalUIChangeDelegateHandle);
	delete this;
}

void FTestExternalUIInterface::StartNextTest()
{
	State = ETestExternalUIInterfaceState::Type(int(State) + 1);
	bool bShowedUI = false;

	if (State >= ETestExternalUIInterfaceState::End)
	{
		// We're done. We've tested everything.
		FinishTest();
		return;
	}
	else if (State == ETestExternalUIInterfaceState::ShowLoginUI)
	{
		bShowedUI = TestLoginUI();
	}
	else if (State == ETestExternalUIInterfaceState::ShowFriendsUI)
	{
		bShowedUI = TestFriendsUI();
	}
	else if (State == ETestExternalUIInterfaceState::ShowInviteUI)
	{
		bShowedUI = TestInviteUI();
	}
	else if (State == ETestExternalUIInterfaceState::ShowAchievementsUI)
	{
		bShowedUI = TestAchievementsUI();
	}
	else if (State == ETestExternalUIInterfaceState::ShowWebURL)
	{
		bShowedUI = TestWebURL();
	}
	else if (State == ETestExternalUIInterfaceState::ShowProfileUI)
	{
		bShowedUI = TestProfileUI();
	}

	if (!bShowedUI)
	{
		// Either the test was not enabled or there was an error. Go to the next test.
		StartNextTest();
	}
}

bool FTestExternalUIInterface::TestLoginUI()
{
	if (bTestLoginUI == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestLoginUI (skipping)"));
		return false;
	}

	bool bShowingUI = ExternalUI->ShowLoginUI(0, true, false, FOnLoginUIClosedDelegate::CreateRaw(this, &FTestExternalUIInterface::OnLoginUIClosed));
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestLoginUI bShowingUI: %d"), bShowingUI);
	return bShowingUI;
}

bool FTestExternalUIInterface::TestFriendsUI()
{
	if (bTestFriendsUI == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestFriendsUI (skipping)"));
		return false;
	}

	bool bShowingUI = ExternalUI->ShowFriendsUI(0);
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestFriendsUI bShowingUI: %d"), bShowingUI);
	return bShowingUI;
}

bool FTestExternalUIInterface::TestInviteUI()
{
	if (bTestInviteUI == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestInviteUI (skipping)"));
		return false;
	}

	bool bShowingUI = ExternalUI->ShowInviteUI(0);
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestInviteUI bShowingUI: %d"), bShowingUI);
	return bShowingUI;
}


bool FTestExternalUIInterface::TestAchievementsUI()
{
	if (bTestAchievementsUI == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestAchievementsUI (skipping)"));
		return false;
	}

	bool bShowingUI = ExternalUI->ShowAchievementsUI(0);
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestAchievementsUI bShowingUI: %d"), bShowingUI);
	return bShowingUI;
}


bool FTestExternalUIInterface::TestWebURL()
{
	if (bTestWebURL == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestWebURL (skipping)"));
		return false;
	}

	bool bShowingUI = ExternalUI->ShowWebURL(
		TEXT("https://www.unrealengine.com"),
		FShowWebUrlParams(),
		FOnShowWebUrlClosedDelegate::CreateRaw(this, &FTestExternalUIInterface::OnShowWebUrlClosed));

	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestWebURL bShowingUI: %d"), bShowingUI);
	return bShowingUI;
}

bool FTestExternalUIInterface::TestProfileUI()
{
	if (bTestProfileUI == false)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestProfileUI (skipping)"));
		return false;
	}

	// Show our own profile
	FUniqueNetIdPtr UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
	bool bShowingUI = ExternalUI->ShowProfileUI(
		*UserId.Get(),
		*UserId.Get(),
		FOnProfileUIClosedDelegate::CreateRaw(this, &FTestExternalUIInterface::OnProfileUIClosed));

	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("TestProfileUI bShowingUI: %d"), bShowingUI);
	return bShowingUI;
}

void FTestExternalUIInterface::OnExternalUIChange(bool bIsOpening)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("OnExternalUIChange delegate invoked. bIsOpening = %d"), bIsOpening);

	if (bIsOpening == false)
	{
		if (bIsolatedTest)
		{
			FinishTest();
		}
		else
		{
			// The external UI is no longer active
			// Move on to the next test
			StartNextTest();
		}
	}
}

void FTestExternalUIInterface::OnLoginUIClosed(FUniqueNetIdPtr LoggedInUserId, const int LocalUserId, const FOnlineError& Error)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("Login UI closed by local user %d. Logged-in user = %s. Error = %s"), LocalUserId, *LoggedInUserId->ToString(), *Error.ToLogString());
}

void FTestExternalUIInterface::OnProfileUIClosed()
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("Profile UI closed by user."));
}

void FTestExternalUIInterface::OnShowWebUrlClosed(const FString& FinalUrl)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("Show Web Url closed with FinalUrl=%s."), *FinalUrl);
}

void FTestExternalUIInterface::OnStoreUIClosed(bool bWasPurchased)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("Store UI was closed. Purchase Made: %d"), bWasPurchased);
}

void FTestExternalUIInterface::OnSendMessageUIClosed(bool bWasSent)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("Send Message UI closed. Message sent: %d"), bWasSent);
}

#endif //WITH_DEV_AUTOMATION_TESTS
