// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineExternalUIInterface.h"

struct FOnlineError;

class IOnlineSubsystem;

#if WITH_DEV_AUTOMATION_TESTS

/** Enumeration of external UI tests */
namespace ETestExternalUIInterfaceState
{
	enum Type
	{
		Begin,
		ShowLoginUI,
		ShowFriendsUI,
		ShowInviteUI,
		ShowAchievementsUI,
		ShowWebURL,
		ShowProfileUI,
		End,
	};
}

/**
 * Class used to test the external UI interface
 */
 class FTestExternalUIInterface
 {

	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;

	/** Booleans that control which external UIs to test */
	bool bTestLoginUI;
	bool bTestFriendsUI;
	bool bTestInviteUI;
	bool bTestAchievementsUI;
	bool bTestWebURL;
	bool bTestProfileUI;

	/** The online interface to use for testing */
	IOnlineSubsystem* OnlineSub;

	/** Convenient access to the external UI interfaces */
	IOnlineExternalUIPtr ExternalUI;

	/** Delegate for external UI opening and closing */
	FOnExternalUIChangeDelegate ExternalUIChangeDelegate;

	/** ExternalUIChange delegate handle */
	FDelegateHandle ExternalUIChangeDelegateHandle;

	/** Current external UI test */
	ETestExternalUIInterfaceState::Type State;

	/** If this is an one off test (for when calling a test that requires extra information, outside the automation loop) */
	bool bIsolatedTest;

	/** Prepares the testing harness environment */
	void PrepareTests();

	/** Completes testing and cleans up after itself */
	void FinishTest();

	/** Go to the next test */
	void StartNextTest();

	/** Specific test functions */
	bool TestLoginUI();
	bool TestFriendsUI();
	bool TestInviteUI();
	bool TestAchievementsUI();
	bool TestWebURL();
	bool TestProfileUI();

	/** Delegate called when external UI is opening and closing */
	void OnExternalUIChange(bool bIsOpening);

	/** Delegate executed when the user login UI has been closed. */
	void OnLoginUIClosed(FUniqueNetIdPtr LoggedInUserId, const int LocalUserId, const FOnlineError& Error);

	/** Delegate executed when the user profile UI has been closed. */
	void OnProfileUIClosed();

	/** Delegate executed when the show web UI has been closed. */
	void OnShowWebUrlClosed(const FString& FinalUrl);

	/** Delegate executed when the store UI has been closed. */
	void OnStoreUIClosed(bool bWasPurchased);

	/** Delegate executed when the send message UI has been closed. */
	void OnSendMessageUIClosed(bool bWasSent);

 public:

	/**
	 * Constructor
	 *
	 */
	FTestExternalUIInterface(const FString& InSubsystemName, bool bInTestLoginUI, bool bInTestFriendsUI, bool bInTestInviteUI, bool bInTestAchievementsUI, bool bInTestWebURL, bool bInTestProfileUI)
		:	SubsystemName(InSubsystemName)
		,	bTestLoginUI(bInTestLoginUI)
		,	bTestFriendsUI(bInTestFriendsUI)
		,	bTestInviteUI(bInTestInviteUI)
		,	bTestAchievementsUI(bInTestAchievementsUI)
		,	bTestWebURL(bInTestWebURL)
		,	bTestProfileUI(bInTestProfileUI)
		,	OnlineSub(NULL)
		,	ExternalUI(NULL)
		,	State(ETestExternalUIInterfaceState::Begin)
		,	bIsolatedTest(false)
	{
	}

	/**
	 * Kicks off all of the testing process
	 *
	 */
	void Test();

	/**
	 * Runs a test for sending messages (not automated as this requires user input)
	 */
	void TestSendMessage(const FString& InMsgRecepient, const FString& InMessage);

	/**
	 * Runs a test for opening the store page
	 */
	void TestStorePage(const FString& InProductId, bool bShouldAddToCart);

 };

#endif //WITH_DEV_AUTOMATION_TESTS
