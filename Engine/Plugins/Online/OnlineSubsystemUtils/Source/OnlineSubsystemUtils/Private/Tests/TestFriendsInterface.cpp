// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestFriendsInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "OnlineSubsystemUtils.h"
#include "Interfaces/OnlineSharingInterface.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestFriendsInterface::Test(UWorld* InWorld, const TArray<FString>& Invites)
{
	OnlineSub = Online::GetSubsystem(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineSub != NULL &&
		OnlineSub->GetIdentityInterface().IsValid() &&
		OnlineSub->GetFriendsInterface().IsValid())
	{
		// don't affect default friends list for MCP
		FString McpSubsystemString;
		MCP_SUBSYSTEM.ToString(McpSubsystemString);
		if (SubsystemName.Equals(McpSubsystemString, ESearchCase::IgnoreCase))
		{
			FriendsListName = TEXT("TestFriends");
		}

		// Add our delegate for the async call
		OnDeleteFriendCompleteDelegate = FOnDeleteFriendCompleteDelegate::CreateRaw(this, &FTestFriendsInterface::OnDeleteFriendComplete);
		OnQueryRecentPlayersCompleteDelegate = FOnQueryRecentPlayersCompleteDelegate::CreateRaw(this, &FTestFriendsInterface::OnQueryRecentPlayersComplete);
		OnQueryBlockedPlayersCompleteDelegate = FOnQueryBlockedPlayersCompleteDelegate::CreateRaw(this, &FTestFriendsInterface::OnQueryBlockedPlayersComplete);

		OnDeleteFriendCompleteDelegateHandle        = OnlineSub->GetFriendsInterface()->AddOnDeleteFriendCompleteDelegate_Handle		(0, OnDeleteFriendCompleteDelegate);
		OnQueryRecentPlayersCompleteDelegateHandle  = OnlineSub->GetFriendsInterface()->AddOnQueryRecentPlayersCompleteDelegate_Handle	(OnQueryRecentPlayersCompleteDelegate);
		OnQueryBlockedPlayersCompleteDelegateHandle = OnlineSub->GetFriendsInterface()->AddOnQueryBlockedPlayersCompleteDelegate_Handle	(OnQueryBlockedPlayersCompleteDelegate);

		// list of pending users to send invites to
		for (int32 Idx=0; Idx < Invites.Num(); Idx++)
		{
			FUniqueNetIdPtr FriendId = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(Invites[Idx]);
			if (FriendId.IsValid())
			{
				InvitesToSend.Add(FriendId);
			}
		}

		UE_LOG_ONLINE_FRIEND(Log, TEXT("Starting friends interface tests for %s"), *SubsystemName);

		StartNextTest();
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Failed to get friends interface for %s"), *SubsystemName);
		FinishTest();
	}
}

void FTestFriendsInterface::OnRequestNewPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful)
{
	IOnlineSharingPtr SharingInt = OnlineSub->GetSharingInterface();
	if (SharingInt.IsValid())
	{
		SharingInt->ClearOnRequestNewReadPermissionsCompleteDelegate_Handle(0, OnRequestNewReadPermissionsDelegateHandle);
	}

	FOnReadFriendsListComplete Delegate = FOnReadFriendsListComplete::CreateRaw(this, &FTestFriendsInterface::OnReadFriendsComplete);
	OnlineSub->GetFriendsInterface()->ReadFriendsList(LocalUserNum, FriendsListName, Delegate);
}

void FTestFriendsInterface::StartNextTest()
{
	if (bReadFriendsList)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Reading friends list"));

		IOnlineSharingPtr SharingInt = OnlineSub->GetSharingInterface();
		if (SharingInt.IsValid())
		{
			UE_LOG_ONLINE_FRIEND(Log, TEXT("Sharing interface valid. Requesting permissions"));

			FOnRequestNewReadPermissionsCompleteDelegate Delegate = FOnRequestNewReadPermissionsCompleteDelegate::CreateRaw(this, &FTestFriendsInterface::OnRequestNewPermissionsComplete);
			OnRequestNewReadPermissionsDelegateHandle = SharingInt->AddOnRequestNewReadPermissionsCompleteDelegate_Handle(0, Delegate);
			SharingInt->RequestNewReadPermissions(0, EOnlineSharingCategory::Friends);
		}
		else
		{
			UE_LOG_ONLINE_FRIEND(Log, TEXT("Sharing interface invalid. Skipping permission request"));

			OnRequestNewPermissionsComplete(0, true);
		}
	}
	else if (bQueryRecentPlayers)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Querying recent players"));

		if (OnlineSub->GetIdentityInterface().IsValid() &&
			OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0).IsValid())
		{
			OnlineSub->GetFriendsInterface()->QueryRecentPlayers(*OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0), RecentPlayersNamespace);
		}
	}
	else if (bAcceptInvites && InvitesToAccept.Num() > 0)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Accepting invites"));

		FOnAcceptInviteComplete Delegate = FOnAcceptInviteComplete::CreateRaw(this, &FTestFriendsInterface::OnAcceptInviteComplete);
		OnlineSub->GetFriendsInterface()->AcceptInvite(0, *InvitesToAccept[0], FriendsListName, Delegate);
	}
	else if (bSendInvites && InvitesToSend.Num() > 0)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Sending invites"));

		FOnSendInviteComplete OnSendInviteCompleteDelegate = FOnSendInviteComplete::CreateRaw(this, &FTestFriendsInterface::OnSendInviteComplete);
		OnlineSub->GetFriendsInterface()->SendInvite(0, *InvitesToSend[0], FriendsListName, OnSendInviteCompleteDelegate);
	}
	else if (bDeleteFriends && FriendsToDelete.Num() > 0)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Deleting friends"));

		OnlineSub->GetFriendsInterface()->DeleteFriend(0, *FriendsToDelete[0], FriendsListName);
	}
	else if (bDeleteFriendsList)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Deleting friends list"));

		FOnDeleteFriendsListComplete Delegate = FOnDeleteFriendsListComplete::CreateRaw(this, &FTestFriendsInterface::OnDeleteFriendsListComplete);
		OnlineSub->GetFriendsInterface()->DeleteFriendsList(0, FriendsListName, Delegate);
	}
	else if (bQueryBlockedPlayers)
	{
		UE_LOG_ONLINE_FRIEND(Log, TEXT("Querying blocked players"));

		if (OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0).IsValid())
		{
			OnlineSub->GetFriendsInterface()->QueryBlockedPlayers(*OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0));
		}
		else
		{
			UE_LOG_ONLINE_FRIEND(Log, TEXT("Player unique id invalid. Skipping Test"));

			bQueryBlockedPlayers = false;
			StartNextTest();
		}
	}
	else
	{
		FinishTest();
	}
}

void FTestFriendsInterface::FinishTest()
{
	UE_LOG_ONLINE_FRIEND(Log, TEXT("All tests finished"));

	if (OnlineSub != NULL &&
		OnlineSub->GetFriendsInterface().IsValid())
	{
		// Clear delegates for the various async calls
		OnlineSub->GetFriendsInterface()->ClearOnDeleteFriendCompleteDelegate_Handle      (0, OnDeleteFriendCompleteDelegateHandle);
		OnlineSub->GetFriendsInterface()->ClearOnQueryRecentPlayersCompleteDelegate_Handle(OnQueryRecentPlayersCompleteDelegateHandle);
		OnlineSub->GetFriendsInterface()->ClearOnQueryBlockedPlayersCompleteDelegate_Handle(OnQueryBlockedPlayersCompleteDelegateHandle);
	}
	delete this;
}

void FTestFriendsInterface::OnReadFriendsComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_FRIEND(Log, TEXT("ReadFriendsList() for player (%d) was success=%d error=%s"), LocalPlayer, bWasSuccessful, *ErrorStr);

	if (bWasSuccessful)
	{
		TArray< TSharedRef<FOnlineFriend> > Friends;
		// Grab the friends data so we can print it out
		if (OnlineSub->GetFriendsInterface()->GetFriendsList(LocalPlayer, ListName, Friends))
		{
			UE_LOG_ONLINE_FRIEND(Log,
				TEXT("GetFriendsList(%d) returned %d friends"), LocalPlayer, Friends.Num());

			// Clear old entries
			InvitesToAccept.Empty();
			FriendsToDelete.Empty();
			// Log each friend's data out
			for (int32 Index = 0; Index < Friends.Num(); Index++)
			{
				const FOnlineFriend& Friend = *Friends[Index];
				const FOnlineUserPresence& Presence = Friend.GetPresence();
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t%s has unique id (%s)"), *Friend.GetDisplayName(), *Friend.GetUserId()->ToDebugString());
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t Invite status (%s)"), EInviteStatus::ToString(Friend.GetInviteStatus()));
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t Presence: %s"), *Presence.Status.StatusStr);
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t State: %s"), EOnlinePresenceState::ToString(Presence.Status.State));
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t bIsOnline (%s)"), Presence.bIsOnline ? TEXT("true") : TEXT("false"));
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t bIsPlaying (%s)"), Presence.bIsPlaying ? TEXT("true") : TEXT("false"));
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t bIsPlayingThisGame (%s)"), Presence.bIsPlayingThisGame ? TEXT("true") : TEXT("false"));
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t bIsJoinable (%s)"), Presence.bIsJoinable ? TEXT("true") : TEXT("false"));
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t\t bHasVoiceSupport (%s)"), Presence.bHasVoiceSupport ? TEXT("true") : TEXT("false"));

				// keep track of pending invites from the list and accept them
				if (Friend.GetInviteStatus() == EInviteStatus::PendingInbound)
				{
					InvitesToAccept.AddUnique(Friend.GetUserId());
				}
				// keep track of list of friends to delete
				FriendsToDelete.AddUnique(Friend.GetUserId());
			}
		}	
		else
		{
			UE_LOG_ONLINE_FRIEND(Log,
				TEXT("GetFriendsList(%d) failed"), LocalPlayer);
		}
	}
	
	// done with this part of the test
	bReadFriendsList = false;
	// kick off next test
	StartNextTest();
}

void FTestFriendsInterface::OnQueryRecentPlayersComplete(const FUniqueNetId& UserId, const FString& Namespace, bool bWasSuccessful, const FString& ErrorStr)
{
	UE_LOG_ONLINE_FRIEND(Log,
		TEXT("QueryRecentPlayers() for player (%s) was success=%d error=%s"), *UserId.ToDebugString(), bWasSuccessful, *ErrorStr);

	if (bWasSuccessful)
	{
		TArray< TSharedRef<FOnlineRecentPlayer> > Players;
		// Grab the friends data so we can print it out
		if (OnlineSub->GetFriendsInterface()->GetRecentPlayers(UserId, Namespace, Players))
		{
			UE_LOG_ONLINE_FRIEND(Log,
				TEXT("GetRecentPlayers returned %d players"), Players.Num());

			// Log each friend's data out
			for (auto RecentPlayer : Players)
			{
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t%s has unique id (%s)"), *RecentPlayer->GetDisplayName(), *RecentPlayer->GetUserId()->ToDebugString());
				UE_LOG_ONLINE_FRIEND(Log,
					TEXT("\t LastSeen (%s)"), *RecentPlayer->GetLastSeen().ToString());
			}
		}
	}

	// done with this part of the test
	bQueryRecentPlayers = false;
	// kick off next test
	StartNextTest();
}

void FTestFriendsInterface::OnAcceptInviteComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_FRIEND(Log, TEXT("AcceptInvite() for player (%d) from friend (%s) was success=%d. %s"), 
		LocalPlayer, *FriendId.ToDebugString(), bWasSuccessful, *ErrorStr);

	// done with this part of the test if no more invites to accept
	if (InvitesToAccept.Num() > 0)
	{
		InvitesToAccept.RemoveAt(0);
	}

	if (InvitesToAccept.Num() == 0)
	{
		bAcceptInvites = false;
		bReadFriendsList = true;
	}
	// kick off next test
	StartNextTest();
}

void FTestFriendsInterface::OnSendInviteComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_FRIEND(Log, TEXT("SendInvite() for player (%d) to friend (%s) was success=%d. %s"), 
		LocalPlayer, *FriendId.ToDebugString(), bWasSuccessful, *ErrorStr);

	// done with this part of the test if no more invites to send
	if (InvitesToSend.Num() > 0)
	{
		InvitesToSend.RemoveAt(0);
	}

	if (InvitesToSend.Num() == 0)
	{
		bSendInvites = false;
		bReadFriendsList = true;
	}
	// kick off next test
	StartNextTest();
}

void FTestFriendsInterface::OnDeleteFriendComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_FRIEND(Log, TEXT("DeleteFriend() for player (%d) to friend (%s) was success=%d. %s"), 
		LocalPlayer, *FriendId.ToDebugString(), bWasSuccessful, *ErrorStr);

	// done with this part of the test if no more friends to delete
	if (FriendsToDelete.Num() > 0)
	{
		FriendsToDelete.RemoveAt(0);
	}

	if (FriendsToDelete.Num() == 0)
	{
		bDeleteFriends = false;
		bReadFriendsList = bWasSuccessful;
	}
	// kick off next test
	StartNextTest();
}

void FTestFriendsInterface::OnDeleteFriendsListComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_FRIEND(Log,
		TEXT("DeleteFriendsList() for player (%d) was success=%d"), LocalPlayer, bWasSuccessful);

	// done with this part of the test
	bDeleteFriendsList = false;
	bQueryBlockedPlayers = true;
	// kick off next test
	StartNextTest();
}

void FTestFriendsInterface::OnQueryBlockedPlayersComplete(const FUniqueNetId& UserId, bool bWasSuccessful, const FString& Error)
{
	UE_LOG_ONLINE_FRIEND(Log, TEXT("QueryBlockedPlayers() for player (%s) was success=%d error=%s"), *UserId.ToDebugString(), bWasSuccessful, *Error);
	bQueryBlockedPlayers = false;

	if (bWasSuccessful)
	{
		TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayers;
		if (OnlineSub->GetFriendsInterface()->GetBlockedPlayers(UserId, BlockedPlayers))
		{
			UE_LOG_ONLINE_FRIEND(Log, TEXT("GetBlockedPlayers() (%s) returned %d blocked users"), *UserId.ToDebugString(), BlockedPlayers.Num());

			for (int32 Index = 0; Index < BlockedPlayers.Num(); Index++)
			{
				const FOnlineBlockedPlayer& BlockedPlayer = *BlockedPlayers[Index];
				UE_LOG_ONLINE_FRIEND(Log, TEXT("\t%s (%s) is blocked"), *BlockedPlayer.GetRealName(), *BlockedPlayer.GetUserId()->ToDebugString());
			}
		}
		else
		{
			UE_LOG_ONLINE_FRIEND(Log, TEXT("GetBlockedPlayers() for player %s failed"), *UserId.ToDebugString());
		}
	}

	StartNextTest();
}
#endif //WITH_DEV_AUTOMATION_TESTS
