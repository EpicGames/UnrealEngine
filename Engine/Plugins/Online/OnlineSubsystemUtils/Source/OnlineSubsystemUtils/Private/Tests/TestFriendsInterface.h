// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineFriendsInterface.h"

class IOnlineSubsystem;

#if WITH_DEV_AUTOMATION_TESTS
/**
 * Class used to test the friends interface
 */
 class FTestFriendsInterface
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineSubsystem* OnlineSub;
	/** Delegate to use for deleting a friend entry */
	FOnDeleteFriendCompleteDelegate OnDeleteFriendCompleteDelegate;
	/** Delegate to use for querying for recent players */
	FOnQueryRecentPlayersCompleteDelegate OnQueryRecentPlayersCompleteDelegate;
	/** Delegate to use for querying the blocked players list */
	FOnQueryBlockedPlayersCompleteDelegate OnQueryBlockedPlayersCompleteDelegate;

	/** Handles to the above delegates */
	FDelegateHandle OnDeleteFriendCompleteDelegateHandle;
	FDelegateHandle OnQueryRecentPlayersCompleteDelegateHandle;
	FDelegateHandle OnQueryBlockedPlayersCompleteDelegateHandle;
	FDelegateHandle OnRequestNewReadPermissionsDelegateHandle;

	/** Default name of friends list for running tests */
	FString FriendsListName;
	/** Filled in after reading friends list. Invites that are pending will be auto-accepted */
	TArray<FUniqueNetIdPtr > InvitesToAccept;
	/** List of friends to send invites to. From test options */
	TArray<FUniqueNetIdPtr > InvitesToSend;
	/** List of friends to delete */
	TArray<FUniqueNetIdPtr > FriendsToDelete;
	
	/** true to enable friends list read */
	bool bReadFriendsList;
	/** true to enable auto accept of pending invites */
	bool bAcceptInvites;
	/** true to send invites */
	bool bSendInvites;
	/** true to delete individual friends from list */
	bool bDeleteFriends;
	/** true to delete the test friends list */
	bool bDeleteFriendsList;
	/** true to query for recent players */
	bool bQueryRecentPlayers;
	/** true to query the block list */
	bool bQueryBlockedPlayers;

	FString RecentPlayersNamespace;

	/** Hidden on purpose */
	FTestFriendsInterface()
		: SubsystemName()
	{
	}

	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/**
	 * 
	 */
	void OnRequestNewPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful);

	/**
	 * Delegate used when the friends read request has completed
	 *
	 * @param LocalPlayer the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	void OnReadFriendsComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr);

	/**
	 * Delegate used when the query for recent players has completed
	 *
	 * @param UserId the id of the user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param Error string representing the error condition
	 */
	void OnQueryRecentPlayersComplete(const FUniqueNetId& UserId, const FString& Namespace, bool bWasSuccessful, const FString& ErrorStr);

	/**
	 * Delegate used when an invite accept request has completed
	 *
	 * @param LocalPlayer the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param FriendId player that invited us
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	void OnAcceptInviteComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr);

	/**
	 * Delegate used when an invite send request has completed
	 *
	 * @param LocalPlayer the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param FriendId player that was invited
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	void OnSendInviteComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr);

	/**
	 * Delegate used when an friend delete request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param FriendId player that was deleted
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	void OnDeleteFriendComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueNetId& FriendId, const FString& ListName, const FString& ErrorStr);

	/**
     * Delegate used when the friends list delete request has completed
     *
	 * @param LocalPlayer the controller number of the associated user that made the request
     * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
     */
	void OnDeleteFriendsListComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr);

	/**
	* Delegate used when the friends list delete request has completed
	*
	* @param UserId the UniqueNetId of the associated user that made the request
	* @param bWasSuccessful true if the async action completed without error, false if there was an error
	* @param ErrorStr string representing the error condition
	*/
	void OnQueryBlockedPlayersComplete(const FUniqueNetId& UserId, bool bWasSuccessful, const FString& Error);

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystemName the subsystem to test
	 */
	FTestFriendsInterface(const FString& InSubsystemName) 
		: SubsystemName(InSubsystemName)
		, OnlineSub(NULL)
		, FriendsListName(EFriendsLists::ToString(EFriendsLists::Default))
		, bReadFriendsList(true)
		, bAcceptInvites(true)
		, bSendInvites(true)
		, bDeleteFriends(true)
		, bDeleteFriendsList(false)
		, bQueryRecentPlayers(true)
		, RecentPlayersNamespace(TEXT("ut"))
	{
	}

	/**
	 * Kicks off all of the testing process
	 *
	 * @param Invites list of friend ids to send invites for
	 */
	void Test(class UWorld* InWorld, const TArray<FString>& Invites);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
