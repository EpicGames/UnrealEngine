// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestIdentityInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestIdentityInterface::Test(UWorld* InWorld, const FOnlineAccountCredentials& InAccountCredentials, bool bOnlyRunLogoutTest)
{
	// Toggle the various tests to run
	if (bOnlyRunLogoutTest)
	{
		bRunLoginTest = false;
		bRunLogoutTest = true;
	}

	AccountCredentials = InAccountCredentials;
	OnlineIdentity = Online::GetIdentityInterface(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineIdentity.IsValid())
	{
		// Add delegates for the various async calls
		OnLoginCompleteDelegateHandle  = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(LocalUserIdx, OnLoginCompleteDelegate);
		OnLogoutCompleteDelegateHandle = OnlineIdentity->AddOnLogoutCompleteDelegate_Handle(LocalUserIdx, OnLogoutCompleteDelegate);
		
		// kick off next test
		StartNextTest();
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning,
			TEXT("Failed to get online identity interface for %s"), *SubsystemName);

		// done with the test
		FinishTest();
	}
}

void FTestIdentityInterface::StartNextTest()
{
	if (bRunLoginTest)
	{
		// no external account credentials so use internal secret key for login
		OnlineIdentity->Login(LocalUserIdx, AccountCredentials);
	}
	else if (bRunLogoutTest)
	{
		IsTheUserLoggedIn();
		OnlineIdentity->Logout(LocalUserIdx);
	}
	else
	{
		// done with the test
		FinishTest();
	}
}

void FTestIdentityInterface::FinishTest()
{
	if (OnlineIdentity.IsValid())
	{
		// Clear delegates for the various async calls
		OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(LocalUserIdx, OnLoginCompleteDelegateHandle);
		OnlineIdentity->ClearOnLogoutCompleteDelegate_Handle(LocalUserIdx, OnLogoutCompleteDelegateHandle);
	}
	SetTestStatus(true);
	delete this;
}

void FTestIdentityInterface::OnLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error)
{
	if (bWasSuccessful)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Successful logged in user. UserId=[%s] "), 
			*UserId.ToDebugString());

		// update user info for newly registered user
		UserInfo = OnlineIdentity->GetUserAccount(UserId);
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to log in new user. Error=[%s]"), 
			*Error);

	}
	// Mark test as done
	bRunLoginTest = false;
	// kick off next test
	StartNextTest();
}

void FTestIdentityInterface::OnLogoutComplete(int32 LocalUserNum, bool bWasSuccessful)
{
	if (bWasSuccessful)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Successful logged out user. LocalUserNum=[%d] "),
			LocalUserNum);
	}
	else if (!bIsUserLoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("User is not logged in to be able to be logged out."));
	}
	// If the user was logged out at the start of the test then there will be nothing to log out.
	else
	{
		UE_LOG_ONLINE_IDENTITY(Error, TEXT("Failed to log out user."));
	}

	UserInfo.Reset();
	// Mark test as done
	bRunLogoutTest = false;
	// kick off next test
	StartNextTest();
}

bool FTestIdentityInterface::GetTestStatus() const
{
	return bIsTestFinished;
}

void FTestIdentityInterface::SetTestStatus(const bool& NewStatus)
{
	bIsTestFinished = NewStatus;
}

bool FTestIdentityInterface::IsTheUserLoggedIn()
{
	// Create a new Identity pointer since this function can be potentially called before original pointer can be initialized.
	IOnlineIdentityPtr TempInterfacePointer = Online::GetIdentityInterface( 
		GEngine->GetWorld(), 
		SubsystemName.Len() ? FName( *SubsystemName, FNAME_Find ) : NAME_None );
	
	ELoginStatus::Type CurrentLoginStatus = TempInterfacePointer->GetLoginStatus(LocalUserIdx);
	bIsUserLoggedIn = false;

	if ( CurrentLoginStatus == ELoginStatus::LoggedIn )
	{
		bIsUserLoggedIn = true;
	}

	return bIsUserLoggedIn;
}

#endif //WITH_DEV_AUTOMATION_TESTS
