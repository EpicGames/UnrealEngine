// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineIdentityInterface.h"

class FUserOnlineAccount;

class Error;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the identity interface
 */
 class FTestIdentityInterface
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineIdentityPtr OnlineIdentity;
	/** Delegate to use for authenticating new user */
	FOnLoginCompleteDelegate OnLoginCompleteDelegate;
	/** Delegate to use for logging out a user */
	FOnLogoutCompleteDelegate OnLogoutCompleteDelegate;
	/** Handles to the above delegates */
	FDelegateHandle OnLoginCompleteDelegateHandle;
	FDelegateHandle OnLogoutCompleteDelegateHandle;
	/** true if authentication test should be run */
	bool bRunLoginTest;
	/** true if logout test should be run */
	bool bRunLogoutTest;
	/** Registered user info */
	TSharedPtr<FUserOnlineAccount> UserInfo;
	/** local user to run tests for */
	int32 LocalUserIdx;
	/** true if the test was successful. */
	bool bIsTestFinished;
	/** true if the user is logged in*/
	bool bIsUserLoggedIn;

	FOnlineAccountCredentials AccountCredentials;

	/** Hidden on purpose */
	FTestIdentityInterface()
		: SubsystemName()
		, LocalUserIdx(0)
	{
	}

	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/**
	 * See OnlineIdentityInterface
	 */
	void OnLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error);
	void OnLogoutComplete(int32 LocalUserNum, bool bWasSuccessful);

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystemName the subsystem to test
	 */
	FTestIdentityInterface(const FString& InSubsystemName)
		: SubsystemName(InSubsystemName)
		, OnlineIdentity(NULL)
		, OnLoginCompleteDelegate(FOnLoginCompleteDelegate::CreateRaw(this, &FTestIdentityInterface::OnLoginComplete))
		, OnLogoutCompleteDelegate(FOnLogoutCompleteDelegate::CreateRaw(this, &FTestIdentityInterface::OnLogoutComplete))
		, bRunLoginTest(true)
		, bRunLogoutTest(false)
		, LocalUserIdx(0)
		, bIsTestFinished(false)
	{
	}

	/**
	 * Kicks off all of the testing process
	 */
	void Test(UWorld* InWorld, const FOnlineAccountCredentials& InAccountCredentials, bool bOnlyRunLogoutTest=false);

	/**
	* Used to get and set the private variables.
	*/
	bool GetTestStatus() const;
	void SetTestStatus( const bool& NewStatus );

	bool IsTheUserLoggedIn();
 };

#endif //WITH_DEV_AUTOMATION_TESTS
