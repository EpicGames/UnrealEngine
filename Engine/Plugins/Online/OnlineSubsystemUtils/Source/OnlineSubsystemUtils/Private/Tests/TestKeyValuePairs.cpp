// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineKeyValuePair.h"
#include "OnlineSubsystem.h"

#if WITH_DEV_AUTOMATION_TESTS

/** Simple test cases for key value pair code */
void TestKeyValuePairs()
{
	bool bSuccess = true;

	TArray<uint8> TestData;
	TestData.Add((uint8)200);

 	typedef FOnlineKeyValuePairs<FName, FVariantData> TestDataType;
	TestDataType TestKeyValuePairs;
 
 	// Test Templates
 	TestKeyValuePairs.Add(TEXT("INTValue"), FVariantData(512)); 
 	TestKeyValuePairs.Add(TEXT("FLOATValue"), FVariantData(512.0f)); 
 	TestKeyValuePairs.Add(TEXT("QWORDValue"), FVariantData((uint64)512)); 
 	TestKeyValuePairs.Add(TEXT("DOUBLEValue"), FVariantData(512000.0)); 
 	TestKeyValuePairs.Add(TEXT("STRINGValue"), FVariantData(TEXT("This Is A Test!"))); 
 	TestKeyValuePairs.Add(TEXT("BLOBValue"), FVariantData(TestData)); 

	UE_LOG_ONLINE(Display, TEXT("ConstIterator"));
	for (TestDataType::TConstIterator It(TestKeyValuePairs); It; ++It)
	{
		UE_LOG_ONLINE(Display, TEXT("%s = %s"), *It.Key().ToString(), *It.Value().ToString());
	}

	UE_LOG_ONLINE(Display, TEXT("Iterator"));
	for (TestDataType::TIterator It(TestKeyValuePairs); It; ++It)
	{
		UE_LOG_ONLINE(Display, TEXT("Iterator %s = %s"), *It.Key().ToString(), *It.Value().ToString());
	}

	UE_LOG_ONLINE(Display, TEXT("Finding all elements"));
	if (TestKeyValuePairs.Find(TEXT("INTValue")) == NULL)
	{
		bSuccess = false;
	}
	if (TestKeyValuePairs.Find(TEXT("FLOATValue")) == NULL)
	{
		bSuccess = false;
	}
	if (TestKeyValuePairs.Find(TEXT("QWORDValue")) == NULL)
	{
		bSuccess = false;
	}
	if (TestKeyValuePairs.Find(TEXT("DOUBLEValue")) == NULL)
	{
		bSuccess = false;
	}
	if (TestKeyValuePairs.Find(TEXT("STRINGValue")) == NULL)
	{
		bSuccess = false;
	}
	if (TestKeyValuePairs.Find(TEXT("BLOBValue")) == NULL)
	{
		bSuccess = false;
	}

	if (!bSuccess)
	{
		UE_LOG_ONLINE(Display, TEXT("Not all elements found!"));
	}

	TestKeyValuePairs.Remove(TEXT("INTValue"));
	TestKeyValuePairs.Remove(TEXT("BLOBValue"));

	UE_LOG_ONLINE(Display, TEXT("Iterator AFTER removing int32 and Blob elements"));
	for (TestDataType::TIterator It(TestKeyValuePairs); It; ++It)
	{
		UE_LOG_ONLINE(Display, TEXT("Iterator %s = %s"), *It.Key().ToString(), *It.Value().ToString());
	}

	TestKeyValuePairs.Empty();
	UE_LOG_ONLINE(Display, TEXT("Iterator AFTER emptying structure"));
	for (TestDataType::TIterator It(TestKeyValuePairs); It; ++It)
	{
		UE_LOG_ONLINE(Display, TEXT("Iterator %s = %s"), *It.Key().ToString(), *It.Value().ToString());
	}

	bSuccess = TestKeyValuePairs.Num() == 0 ? true : false;

	// Test basic variant data type functionality
	FVariantData OrigKeyValuePair;
	FVariantData CopyValue;

	{
		// Test int32
		int32 OutValue = 0;
		int32 TestValue = 5;
		OrigKeyValuePair.SetValue(TestValue);
		OrigKeyValuePair.GetValue(OutValue);
		bSuccess = bSuccess && (OutValue == TestValue);   
		
		CopyValue = OrigKeyValuePair; 
		UE_LOG_ONLINE(Display, TEXT("int32 Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());

		OrigKeyValuePair.Increment<int32,EOnlineKeyValuePairDataType::Int32>((int32)1);
		UE_LOG_ONLINE(Display, TEXT("+1 Now is %s"), *OrigKeyValuePair.ToString());

		OrigKeyValuePair.Decrement<int32,EOnlineKeyValuePairDataType::Int32>((int32)1);
		UE_LOG_ONLINE(Display, TEXT("-1 Now is %s"), *OrigKeyValuePair.ToString());

		bSuccess = bSuccess && (OrigKeyValuePair == CopyValue); 
		bSuccess = bSuccess && OrigKeyValuePair.FromString(TEXT("5"));
		UE_LOG_ONLINE(Display, TEXT("int32 Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());
	}

	{
		// Test float
		float OutValue = 0.0f;
		float TestValue = 5.0f;
		OrigKeyValuePair.SetValue(TestValue);
		OrigKeyValuePair.GetValue(OutValue);
		bSuccess = bSuccess && (OutValue == TestValue);

		CopyValue = OrigKeyValuePair; 
		UE_LOG_ONLINE(Display, TEXT("float Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());

		OrigKeyValuePair.Increment<float,EOnlineKeyValuePairDataType::Float>(1.0f);
		UE_LOG_ONLINE(Display, TEXT("+1 Now is %s"), *OrigKeyValuePair.ToString());

		OrigKeyValuePair.Decrement<float,EOnlineKeyValuePairDataType::Float>(1.0f);
		UE_LOG_ONLINE(Display, TEXT("-1 Now is %s"), *OrigKeyValuePair.ToString());

		bSuccess = bSuccess && (OrigKeyValuePair == CopyValue); 
		bSuccess = bSuccess && OrigKeyValuePair.FromString(TEXT("5.0"));
		UE_LOG_ONLINE(Display, TEXT("float Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());
	}

	{
		// Test double
		double OutValue = 0.0;
		double TestValue = 5.0;
		OrigKeyValuePair.SetValue(TestValue);
		OrigKeyValuePair.GetValue(OutValue);
		bSuccess = bSuccess && (OutValue == TestValue);

		CopyValue = OrigKeyValuePair; 
		UE_LOG_ONLINE(Display, TEXT("double Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());

		OrigKeyValuePair.Increment<double,EOnlineKeyValuePairDataType::Double>(1.0);
		UE_LOG_ONLINE(Display, TEXT("+1 Now is %s"), *OrigKeyValuePair.ToString());

		OrigKeyValuePair.Decrement<double,EOnlineKeyValuePairDataType::Double>(1.0);
		UE_LOG_ONLINE(Display, TEXT("-1 Now is %s"), *OrigKeyValuePair.ToString());

		bSuccess = bSuccess && (OrigKeyValuePair == CopyValue); 
		bSuccess = bSuccess && OrigKeyValuePair.FromString(TEXT("5.0"));
		UE_LOG_ONLINE(Display, TEXT("double Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());
	}

	{
		// Test uint64
		uint64 OutValue = 0;
		uint64 TestValue = 524288;
		OrigKeyValuePair.SetValue(TestValue);
		OrigKeyValuePair.GetValue(OutValue);
		bSuccess = bSuccess && (OutValue == TestValue);

		CopyValue = OrigKeyValuePair; 
		UE_LOG_ONLINE(Display, TEXT("uint64 Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString()); 

		OrigKeyValuePair.Increment<uint64,EOnlineKeyValuePairDataType::Int64>(1);
		UE_LOG_ONLINE(Display, TEXT("+1 Now is %s"), *OrigKeyValuePair.ToString());

		OrigKeyValuePair.Decrement<uint64,EOnlineKeyValuePairDataType::Int64>(1);
		UE_LOG_ONLINE(Display, TEXT("-1 Now is %s"), *OrigKeyValuePair.ToString());

		bSuccess = bSuccess && (OrigKeyValuePair == CopyValue);
		bSuccess = bSuccess && OrigKeyValuePair.FromString(TEXT("524288"));
		UE_LOG_ONLINE(Display, TEXT("uint64 Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());
	}

	{
		// Test String
		FString OutValue;
		FString TestValue(TEXT("This is a test!"));
		OrigKeyValuePair.SetValue(TestValue);
		OrigKeyValuePair.GetValue(OutValue);
		bSuccess = bSuccess && (OutValue == TestValue);

		CopyValue = OrigKeyValuePair; 
		bSuccess = bSuccess && (OrigKeyValuePair == CopyValue);
		UE_LOG_ONLINE(Display, TEXT("STRING Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString()); 
	}

	{
		// Test Blob
		TArray<uint8> OutValue;
		TArray<uint8> TestValue;
		for (int32 i=0; i<512; i++)
		{
			TestValue.Add(FMath::Rand() % 255);
		}

		OrigKeyValuePair.SetValue(TestValue);
		OrigKeyValuePair.GetValue(OutValue);
		bSuccess = bSuccess && (OutValue == TestValue);

		CopyValue = OrigKeyValuePair; 
		bSuccess = bSuccess && (OrigKeyValuePair == CopyValue); 
		UE_LOG_ONLINE(Display, TEXT("BLOB Test %s == %s"), *OrigKeyValuePair.ToString(), *CopyValue.ToString());
	}

	UE_LOG_ONLINE(Warning, TEXT("KeyValuePairTest: %s!"), bSuccess ? TEXT("PASSED") : TEXT("FAILED"));
}

#endif //WITH_DEV_AUTOMATION_TESTS
