// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestLeaderboardInterface.h"
#include "OnlineSubsystemUtils.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 *	Example of a leaderboard write object
 */
class TestLeaderboardWrite : public FOnlineLeaderboardWrite
{
public:
	TestLeaderboardWrite()
	{
		// Default properties
		new (LeaderboardNames) FString("TestLeaderboard");
		RatedStat = FString("TestIntStat1");
		DisplayFormat = ELeaderboardFormat::Number;
		SortMethod = ELeaderboardSort::Descending;
		UpdateMethod = ELeaderboardUpdateMethod::KeepBest;
	}
};

/**
 *	Example of a leaderboard read object
 */
class TestLeaderboardRead : public FOnlineLeaderboardRead
{
public:
	TestLeaderboardRead(const FString& InLeaderboardName, const FString& InSortedColumn, const TMap<FString, EOnlineKeyValuePairDataType::Type>& InColumns)
	{
		LeaderboardName = InLeaderboardName;
		SortedColumn = InSortedColumn;

		for (TPair<FString, EOnlineKeyValuePairDataType::Type> Column : InColumns)
		{
			new (ColumnMetadata) FColumnMetaData(Column.Key, Column.Value);
		}
	}
};

FTestLeaderboardInterface::FTestLeaderboardInterface(const FString& InSubsystem) :
	Subsystem(InSubsystem),
	bOverallSuccess(true),
	bReadLeaderboardAttempted(false),
	Leaderboards(NULL),
	TestPhase(0),
	LastTestPhase(-1)
{
	// Define delegates
	LeaderboardFlushDelegate = FOnLeaderboardFlushCompleteDelegate::CreateRaw(this, &FTestLeaderboardInterface::OnLeaderboardFlushComplete);
	LeaderboardReadCompleteDelegate = FOnLeaderboardReadCompleteDelegate::CreateRaw(this, &FTestLeaderboardInterface::OnLeaderboardReadComplete);
	LeaderboardReadRankCompleteDelegate = FOnLeaderboardReadCompleteDelegate::CreateRaw(this, &FTestLeaderboardInterface::OnLeaderboardRankReadComplete);
	LeaderboardReadRankUserCompleteDelegate = FOnLeaderboardReadCompleteDelegate::CreateRaw(this, &FTestLeaderboardInterface::OnLeaderboardUserRankReadComplete);
}

FTestLeaderboardInterface::~FTestLeaderboardInterface()
{
	if(Leaderboards.IsValid())
	{
		Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadCompleteDelegateHandle);
		Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankCompleteDelegateHandle);
		Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankUserCompleteDelegateHandle);
		Leaderboards->ClearOnLeaderboardFlushCompleteDelegate_Handle(LeaderboardFlushDelegateHandle);
	}

	Leaderboards = NULL;
}

void FTestLeaderboardInterface::Test(UWorld* InWorld, const FString& InLeaderboardName, const FString& InColumnName, TMap<FString, EOnlineKeyValuePairDataType::Type>&& InColumns, const FString& InUserId)
{
	FindRankUserId = InUserId;
	OnlineSub = Online::GetSubsystem(InWorld, FName(*Subsystem));
	if (!OnlineSub)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get online subsystem for %s"), *Subsystem);

		bOverallSuccess = false;
		return;
	}

	if (OnlineSub->GetIdentityInterface().IsValid())
	{
		UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
	}

	// Cache interfaces
	Leaderboards = OnlineSub->GetLeaderboardsInterface();
	if (!Leaderboards.IsValid())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get online leaderboards interface for %s"), *Subsystem);

		bOverallSuccess = false;
		return;
	}

	LeaderboardName = InLeaderboardName;
	SortedColumn = InColumnName;
	Columns = MoveTemp(InColumns);
}

void FTestLeaderboardInterface::TestFromConfig(UWorld* InWorld)
{
	OnlineSub = Online::GetSubsystem(InWorld, FName(*Subsystem));
	if (!OnlineSub)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get online subsystem for %s"), *Subsystem);

		bOverallSuccess = false;
		return;
	}

	if (OnlineSub->GetIdentityInterface().IsValid())
	{
		UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
	}

	// Cache interfaces
	Leaderboards = OnlineSub->GetLeaderboardsInterface();
	if (!Leaderboards.IsValid())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get online leaderboards interface for %s"), *Subsystem);

		bOverallSuccess = false;
		return;
	}

	FString ConfigLeaderboardName;
	FString ConfigSortedColumn;
	TArray<FString> ConfigAllColumns;
	TArray<FString> ConfigAllColumnTypes;
	FString ConfigWriteColumnName;
	FString ConfigUserLookup;

	if (!GConfig->GetString(TEXT("TestLeaderboardInterface"), TEXT("LeaderboardName"), ConfigLeaderboardName, GEngineIni))
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Config setup for leaderboards had no LeaderboardName!"));
		bOverallSuccess = false;
		return;
	}

	if (!GConfig->GetString(TEXT("TestLeaderboardInterface"), TEXT("LeaderboardSortedColumn"), ConfigSortedColumn, GEngineIni))
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Config setup for leaderboards had no LeaderboardSortedColumn!"));
		bOverallSuccess = false;
		return;
	}

	if (!GConfig->GetArray(TEXT("TestLeaderboardInterface"), TEXT("LeaderboardColumns"), ConfigAllColumns, GEngineIni))
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Config setup for leaderboards had no LeaderboardColumns!"));
		bOverallSuccess = false;
		return;
	}

	if (!GConfig->GetArray(TEXT("TestLeaderboardInterface"), TEXT("LeaderboardColumnTypes"), ConfigAllColumnTypes, GEngineIni))
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Config setup for leaderboards had no LeaderboardColumnTypes!"));
		bOverallSuccess = false;
		return;
	}

	if (!GConfig->GetString(TEXT("TestLeaderboardInterface"), TEXT("LeaderboardWriteColumnName"), ConfigWriteColumnName, GEngineIni))
	{
		// write test is optional
	}

	if (!GConfig->GetString(TEXT("TestLeaderboardInterface"), TEXT("LeaderboardUserIdLookup"), ConfigUserLookup, GEngineIni))
	{
		// user id lookup is optional
	}

	if(ConfigAllColumns.Num() != ConfigAllColumnTypes.Num())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Differing numbers of columns vs column types!!"));
		bOverallSuccess = false;
		return;
	}

	LeaderboardName = ConfigLeaderboardName;
	SortedColumn = ConfigSortedColumn;
	WriteColumn = ConfigWriteColumnName;
	FindRankUserId = ConfigUserLookup;
	
	for(int i = 0; i < ConfigAllColumns.Num(); i++)
	{
		FString& ColumnName = ConfigAllColumns[i];
		FString& ColumnType = ConfigAllColumnTypes[i];

		Columns.Add(ColumnName, EOnlineKeyValuePairDataType::FromString(ColumnType));
	}
}

void FTestLeaderboardInterface::WriteLeaderboards()
{
	TestLeaderboardWrite WriteObject;
	
	// Set some data
	WriteObject.SetIntStat(FString("TestIntStat1"), 50);
	WriteObject.SetFloatStat(FString("TestFloatStat1"), 99.5f);

	// Write it to the buffers
	Leaderboards->WriteLeaderboards(TEXT("TEST"), *UserId, WriteObject);
	TestPhase++;
}

void FTestLeaderboardInterface::OnLeaderboardFlushComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_LEADERBOARD(Verbose, TEXT("OnLeaderboardFlushComplete Session: %s bWasSuccessful: %d"), *SessionName.ToString(), bWasSuccessful);
	bOverallSuccess = bOverallSuccess && bWasSuccessful;

	Leaderboards->ClearOnLeaderboardFlushCompleteDelegate_Handle(LeaderboardFlushDelegateHandle);
	TestPhase++;
}

void FTestLeaderboardInterface::FlushLeaderboards()
{
	LeaderboardFlushDelegateHandle = Leaderboards->AddOnLeaderboardFlushCompleteDelegate_Handle(LeaderboardFlushDelegate);
	Leaderboards->FlushLeaderboards(TEXT("TEST"));
}

void FTestLeaderboardInterface::PrintLeaderboards()
{
	for (int32 RowIdx = 0; RowIdx < ReadObject->Rows.Num(); ++RowIdx)
	{
		const FOnlineStatsRow& StatsRow = ReadObject->Rows[RowIdx];
		UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("   Leaderboard stats for: Nickname = %s, Rank = %d"), *StatsRow.NickName, StatsRow.Rank);

		for (FStatsColumnArray::TConstIterator It(StatsRow.Columns); It; ++It)
		{
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("     %s = %s"), *It.Key(), *It.Value().ToString());
		}
	}
}

void FTestLeaderboardInterface::OnLeaderboardReadComplete(bool bWasSuccessful)
{
	UE_LOG_ONLINE_LEADERBOARD(Verbose, TEXT("OnLeaderboardReadComplete bWasSuccessful: %d"), bWasSuccessful);
	bOverallSuccess = bOverallSuccess && (bReadLeaderboardAttempted == bWasSuccessful);

	PrintLeaderboards();

	Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadCompleteDelegateHandle);
	bReadLeaderboardAttempted = false;
	TestPhase++;
}

void FTestLeaderboardInterface::OnLeaderboardRankReadComplete(bool bWasSuccessful)
{
	UE_LOG_ONLINE_LEADERBOARD(Verbose, TEXT("OnLeaderboardRankReadComplete bWasSuccessful: %d"), bWasSuccessful);
	bOverallSuccess = bOverallSuccess && (bReadLeaderboardAttempted == bWasSuccessful);

	PrintLeaderboards();

	Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankCompleteDelegateHandle);
	bReadLeaderboardAttempted = false;
	TestPhase++;
}

void FTestLeaderboardInterface::OnLeaderboardUserRankReadComplete(bool bWasSuccessful)
{
	UE_LOG_ONLINE_LEADERBOARD(Verbose, TEXT("OnLeaderboardUserRankReadComplete bWasSuccessful: %d"), bWasSuccessful);
	bOverallSuccess = bOverallSuccess && (bReadLeaderboardAttempted == bWasSuccessful);

	PrintLeaderboards();

	Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankUserCompleteDelegateHandle);
	bReadLeaderboardAttempted = false;
	TestPhase++;
}

void FTestLeaderboardInterface::ReadLeaderboards()
{
	ReadObject = MakeShareable(new TestLeaderboardRead(LeaderboardName, SortedColumn, Columns));
	FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

	TArray<FUniqueNetIdRef> QueryPlayers;
	QueryPlayers.Add(UserId.ToSharedRef());

	LeaderboardReadCompleteDelegateHandle = Leaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadCompleteDelegate);
	bReadLeaderboardAttempted = Leaderboards->ReadLeaderboards(QueryPlayers, ReadObjectRef);
}

void FTestLeaderboardInterface::ReadLeaderboardsFriends()
{
	ReadObject = MakeShareable(new TestLeaderboardRead(LeaderboardName, SortedColumn, Columns));
	FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

	LeaderboardReadCompleteDelegateHandle = Leaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadCompleteDelegate);
	bReadLeaderboardAttempted = Leaderboards->ReadLeaderboardsForFriends(0, ReadObjectRef);
}

void FTestLeaderboardInterface::ReadLeaderboardsRank(int32 Rank, int32 Range)
{
	ReadObject = MakeShareable(new TestLeaderboardRead(LeaderboardName, SortedColumn, Columns));
	FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

	LeaderboardReadRankCompleteDelegateHandle = Leaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankCompleteDelegate);
	bReadLeaderboardAttempted = Leaderboards->ReadLeaderboardsAroundRank(Rank, Range, ReadObjectRef);
}

void FTestLeaderboardInterface::ReadLeaderboardsUser(const FUniqueNetId& InUserId, int32 Range)
{
	if (!OnlineSub || !OnlineSub->GetIdentityInterface().IsValid())
	{
		bOverallSuccess = false;
		++TestPhase;
		return;
	}

	ReadObject = MakeShareable(new TestLeaderboardRead(LeaderboardName, SortedColumn, Columns));
	FOnlineLeaderboardReadRef ReadObjectRef = ReadObject.ToSharedRef();

	// Need to get a shared reference for ReadLeaderboardsAroundUser
	FUniqueNetIdPtr ArbitraryId = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(InUserId.ToString());

	if (ArbitraryId.IsValid())
	{
		LeaderboardReadRankUserCompleteDelegateHandle = Leaderboards->AddOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankUserCompleteDelegate);
		bReadLeaderboardAttempted = Leaderboards->ReadLeaderboardsAroundUser(ArbitraryId.ToSharedRef(), Range, ReadObjectRef);
	}
	else
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Cannot run the leaderboards around user test as it failed to start. UserId not valid"));
		bOverallSuccess = false;
		Leaderboards->ClearOnLeaderboardReadCompleteDelegate_Handle(LeaderboardReadRankUserCompleteDelegateHandle);
		++TestPhase;
	}
}

void FTestLeaderboardInterface::ReadLeaderboardsUser(int32 Range)
{
	FUniqueNetIdRef FindUser = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(FindRankUserId).ToSharedRef();
	ReadLeaderboardsUser(*FindUser, Range);
}

bool FTestLeaderboardInterface::Tick( float DeltaTime )
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTestLeaderboardInterface_Tick);

	if (TestPhase != LastTestPhase)
	{
		if (!bOverallSuccess)
		{
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("Testing failed in phase %d"), LastTestPhase);
			TestPhase = 7;
		}

		LastTestPhase = TestPhase;

		switch(TestPhase)
		{
		case 0:
			if(FindRankUserId.IsEmpty())
			{
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("Test will be skipping write test as a column was not provided."));
				++TestPhase;
				return true;
			}
			else
			{
				WriteLeaderboards();
			}
			break;
		case 1:
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("// Beginning FlushLeaderboards"));
			FlushLeaderboards();
			break;
		case 2:
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("// Beginning ReadLeaderboards (reading self)"));
			ReadLeaderboards();
			break;
		case 3:
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("// Beginning ReadLeaderboardsFriends"));
			ReadLeaderboardsFriends();
			break;
		case 4:
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("// Beginning ReadLeaderboardsRank polling users from 1 to 8"));
			ReadLeaderboardsRank(3, 5);
			break;
		case 5:
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("// Beginning ReadLeaderboardsUser polling all users +- 5 spaces from the local user"));
			ReadLeaderboardsUser(*UserId, 5);
			break;
		case 6:
		{
			if (FindRankUserId.IsEmpty())
			{
				++TestPhase;
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("Test will be skipping arbitrary lookup as an id was not provided."));
				return true;
			}
			else
			{
				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("// Beginning ReadLeaderboardsUser polling all users +- 1 from the designated user (%s)"), *FindRankUserId);
				ReadLeaderboardsUser(1);
			}
		} break;
		case 7:
			UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("TESTING COMPLETE Success:%s!"), bOverallSuccess ? TEXT("true") : TEXT("false"));
			delete this;
			return false;
		}
	}
	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
