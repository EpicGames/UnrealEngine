// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Interfaces/OnlineLeaderboardInterface.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the friends interface
 */
 class FTestLeaderboardInterface : public FTSTickerObjectBase
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString Subsystem;

	/** Cached Online Subsystem Pointer */
	class IOnlineSubsystem* OnlineSub;

	/** Keep track of success across all functions and callbacks */
	bool bOverallSuccess;

	/** Set every time we try to read a Leaderboard, indicates if the read was attempted or not */
	bool bReadLeaderboardAttempted;

	/** Logged in UserId */
	FUniqueNetIdPtr UserId;

	/** Passed in UserId */
	FString FindRankUserId;

	/** Passed in LeaderboardName */
	FString LeaderboardName;

	/** Passed in ColumnName */
	FString SortedColumn;

	/** Passed in WriteColumn */
	FString WriteColumn;

	/** If true, don't try to perform the write test. */
	bool bSkipWriteTest = false;

	/** Passed in Columns */
	TMap<FString, EOnlineKeyValuePairDataType::Type> Columns;

	/** Convenient access to the leaderboard interfaces */
	IOnlineLeaderboardsPtr Leaderboards;

	/** Leaderboard read object */
	FOnlineLeaderboardReadPtr ReadObject;

	/** Delegate called when leaderboard data has been successfully committed to the backend service */ 
	FOnLeaderboardFlushCompleteDelegate LeaderboardFlushDelegate;
	/** Delegate called when a leaderboard has been successfully read */
	FOnLeaderboardReadCompleteDelegate LeaderboardReadCompleteDelegate;
	FOnLeaderboardReadCompleteDelegate LeaderboardReadRankCompleteDelegate;
	FOnLeaderboardReadCompleteDelegate LeaderboardReadRankUserCompleteDelegate;

	/** Handles to the above delegates */ 
	FDelegateHandle LeaderboardFlushDelegateHandle;
	FDelegateHandle LeaderboardReadCompleteDelegateHandle;
	FDelegateHandle LeaderboardReadRankCompleteDelegateHandle;
	FDelegateHandle LeaderboardReadRankUserCompleteDelegateHandle;

	/** Current phase of testing */
	int32 TestPhase;
	/** Last phase of testing triggered */
	int32 LastTestPhase;

	/** Hidden on purpose */
	FTestLeaderboardInterface()
		: Subsystem()
	{
	}

	/**
	 *	Write out some test data to a leaderboard
	 */
	void WriteLeaderboards();

	/**
	 *	Delegate called when leaderboard data has been successfully committed to the backend service
	 */
	void OnLeaderboardFlushComplete(FName SessionName, bool bWasSuccessful);

	/**
	 *	Commit the leaderboard writes to the backend service
	 */
	void FlushLeaderboards();

	/**
	 *	Delegate called when a leaderboard has been successfully read
	 */
	void OnLeaderboardReadComplete(bool bWasSuccessful);
	void OnLeaderboardRankReadComplete(bool bWasSuccessful);
	void OnLeaderboardUserRankReadComplete(bool bWasSuccessful);

	/**
	 *	Read in some predefined data from a leaderboard
	 */
	void ReadLeaderboards();
	void ReadLeaderboardsFriends();
	void ReadLeaderboardsRank(int32 Rank, int32 Range);
	void ReadLeaderboardsUser(const FUniqueNetId& InUserId, int32 Range);
	void ReadLeaderboardsUser(int32 Range);

	/** Utilities */
	void PrintLeaderboards();

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestLeaderboardInterface(const FString& InSubsystem);

	virtual ~FTestLeaderboardInterface();

	// FTSTickerObjectBase

	bool Tick( float DeltaTime ) override;

	// FTestLeaderboardInterface

	/**
	 * Kicks off all of the testing process
	 */
	void Test(UWorld* InWorld, const FString& InLeaderboardName, const FString& InSortedColumn, TMap<FString, EOnlineKeyValuePairDataType::Type>&& InColumns, const FString& InUserId);

	void TestFromConfig(UWorld* InWorld);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
