// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tests/TestMessageInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

struct TestAttribute
{
	FString Name;
	FVariantData Value;

	TestAttribute (const FString &InName, const FVariantData &InValue) :
		Name(InName),
		Value(InValue)
	{
	}
};

static TestAttribute TestAttributeList [] =
{
	TestAttribute("INTValue", FVariantData(512)),
	TestAttribute("FLOATValue", FVariantData(512.0f)),
	TestAttribute("QWORDValue", FVariantData((uint64)512)),
	TestAttribute("DOUBLEValue", FVariantData(512000.0)),
	TestAttribute("STRINGValue", FVariantData(TEXT("This Is A Test!")))
};

static uint8 BLOBTestValue [] = 
{
	0xde, 0xad, 0xbe, 0xef, 0xfa, 0xde, 0xbe, 0xad
};

FTestMessageInterface::FTestMessageInterface(const FString& InSubsystem)
	: OnlineSub(NULL)
	, bEnumerateMessages(true)
	, bReadMessages(true)
	, bSendMessages(true)
	, bDeleteMessages(false)
{
	UE_LOG_ONLINE(Display, TEXT("FTestMessageInterface::FTestMessageInterface"));
	SubsystemName = InSubsystem;
}


FTestMessageInterface::~FTestMessageInterface()
{
	UE_LOG_ONLINE(Display, TEXT("FTestMessageInterface::~FTestMessageInterface"));
}

void FTestMessageInterface::Test(UWorld* InWorld, const TArray<FString>& InRecipients)
{
	UE_LOG_ONLINE(Display, TEXT("FTestMessageInterface::Test"));

	OnlineSub = Online::GetSubsystem(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineSub != NULL &&
		OnlineSub->GetIdentityInterface().IsValid() &&
		OnlineSub->GetMessageInterface().IsValid())
	{
		// Add our delegate for the async call
		OnEnumerateMessagesCompleteDelegate = FOnEnumerateMessagesCompleteDelegate::CreateRaw(this, &FTestMessageInterface::OnEnumerateMessagesComplete);
		OnReadMessageCompleteDelegate       = FOnReadMessageCompleteDelegate      ::CreateRaw(this, &FTestMessageInterface::OnReadMessageComplete);
		OnSendMessageCompleteDelegate       = FOnSendMessageCompleteDelegate      ::CreateRaw(this, &FTestMessageInterface::OnSendMessageComplete);
		OnDeleteMessageCompleteDelegate     = FOnDeleteMessageCompleteDelegate    ::CreateRaw(this, &FTestMessageInterface::OnDeleteMessageComplete);

		OnEnumerateMessagesCompleteDelegateHandle = OnlineSub->GetMessageInterface()->AddOnEnumerateMessagesCompleteDelegate_Handle(0, OnEnumerateMessagesCompleteDelegate);
		OnReadMessageCompleteDelegateHandle       = OnlineSub->GetMessageInterface()->AddOnReadMessageCompleteDelegate_Handle      (0, OnReadMessageCompleteDelegate);
		OnSendMessageCompleteDelegateHandle       = OnlineSub->GetMessageInterface()->AddOnSendMessageCompleteDelegate_Handle      (0, OnSendMessageCompleteDelegate);
		OnDeleteMessageCompleteDelegateHandle     = OnlineSub->GetMessageInterface()->AddOnDeleteMessageCompleteDelegate_Handle    (0, OnDeleteMessageCompleteDelegate);

		// list of users to send messages to
		for (int32 Idx=0; Idx < InRecipients.Num(); Idx++)
		{
			FUniqueNetIdPtr UserId = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(InRecipients[Idx]);
			if (UserId.IsValid())
			{
				Recipients.Add(UserId.ToSharedRef());
			}
		}

		// kick off next test
		StartNextTest();
	}
	else
	{
		UE_LOG_ONLINE(Warning,
			TEXT("Failed to get message interface for %s"), *SubsystemName);
		
		FinishTest();
	}
}

void FTestMessageInterface::StartNextTest()
{
	if (bEnumerateMessages)
	{
		OnlineSub->GetMessageInterface()->EnumerateMessages(0);
	}
	else if (bReadMessages && MessagesToRead.Num() > 0)
	{
		OnlineSub->GetMessageInterface()->ReadMessage(0, *MessagesToRead[0]);
	}
	else if (bSendMessages && Recipients.Num() > 0)
	{
		FUniqueNetIdPtr UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
		if (UserId.IsValid())
		{
			FOnlineMessagePayload TestPayload;

			int AttributeCount = sizeof(TestAttributeList)/sizeof(TestAttribute);
			for (int i = 0; i < AttributeCount; ++i)
			{
				TestPayload.SetAttribute(TestAttributeList[i].Name, TestAttributeList[i].Value);
			}

			TArray<uint8> TestData;
			for (int i = 0; i < sizeof(BLOBTestValue); ++i)
			{
				TestData.Add(BLOBTestValue[i]);
			}
 			TestPayload.SetAttribute(TEXT("BLOBValue"), FVariantData(TestData));

			OnlineSub->GetMessageInterface()->SendMessage(0, Recipients, TEXT("TestType"), TestPayload);
		}
		else
		{
			bSendMessages = false;
			StartNextTest();
		}
	}
	else if (bDeleteMessages && MessagesToDelete.Num() > 0)
	{
		OnlineSub->GetMessageInterface()->DeleteMessage(0, *MessagesToDelete[0]);
	}
	else
	{
		FinishTest();
	}
}

void FTestMessageInterface::FinishTest()
{
	if (OnlineSub != NULL &&
		OnlineSub->GetMessageInterface().IsValid())
	{
		// Clear delegates for the various async calls
		OnlineSub->GetMessageInterface()->ClearOnEnumerateMessagesCompleteDelegate_Handle(0, OnEnumerateMessagesCompleteDelegateHandle);
		OnlineSub->GetMessageInterface()->ClearOnReadMessageCompleteDelegate_Handle      (0, OnReadMessageCompleteDelegateHandle);
		OnlineSub->GetMessageInterface()->ClearOnSendMessageCompleteDelegate_Handle      (0, OnSendMessageCompleteDelegateHandle);
		OnlineSub->GetMessageInterface()->ClearOnDeleteMessageCompleteDelegate_Handle    (0, OnDeleteMessageCompleteDelegateHandle);
	}
	delete this;
}

void FTestMessageInterface::OnEnumerateMessagesComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ErrorStr)
{
	UE_LOG_ONLINE(Log,
		TEXT("EnumerateMessages() for player (%d) was success=%d"), LocalPlayer, bWasSuccessful);

	if (bWasSuccessful)
	{
		TArray< TSharedRef<FOnlineMessageHeader> > MessageHeaders;
		if (OnlineSub->GetMessageInterface()->GetMessageHeaders(LocalPlayer, MessageHeaders))
		{
			UE_LOG_ONLINE(Log,
				TEXT("GetMessageHeaders(%d) returned %d message headers"), LocalPlayer, MessageHeaders.Num());
			
			// Clear old entries
			MessagesToRead.Empty();
			MessagesToDelete.Empty();

			// Log each message header data out
			for (int32 Index = 0; Index < MessageHeaders.Num(); Index++)
			{
				const FOnlineMessageHeader& Header = *MessageHeaders[Index];
				UE_LOG_ONLINE(Log,
					TEXT("\t message id (%s)"), *Header.MessageId->ToDebugString());
				UE_LOG_ONLINE(Log,
					TEXT("\t\t from user id (%s)"), *Header.FromUserId->ToDebugString());
				UE_LOG_ONLINE(Log,
					TEXT("\t\t from name: %s"), *Header.FromName);
				UE_LOG_ONLINE(Log,
					TEXT("\t\t type (%s)"), *Header.Type);
				UE_LOG_ONLINE(Log,
					TEXT("\t\t time stamp (%s)"), *Header.TimeStamp);

				// Add to list of messages to download
				MessagesToRead.AddUnique(Header.MessageId);
				// Add to list of messages to delete
				MessagesToDelete.AddUnique(Header.MessageId);
			}
		}	
		else
		{
			UE_LOG_ONLINE(Log,
				TEXT("GetMessageHeaders(%d) failed"), LocalPlayer);
		}
	}
	// done with this part of the test
	bEnumerateMessages = false;
	// kick off next test
	StartNextTest();
}

void FTestMessageInterface::OnReadMessageComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueMessageId& MessageId, const FString& ErrorStr)
{
	UE_LOG_ONLINE(Log,
		TEXT("ReadMessage() for player (%d) was success=%d"), LocalPlayer, bWasSuccessful);

	// Dump the message content back out
	if (bWasSuccessful)
	{
		TSharedPtr<class FOnlineMessage> Message = OnlineSub->GetMessageInterface()->GetMessage(LocalPlayer, MessageId);

		int AttributeCount = sizeof(TestAttributeList)/sizeof(TestAttribute);
		for (int i = 0; i < AttributeCount; ++i)
		{
			FVariantData Value;
			if (Message->Payload.GetAttribute(TestAttributeList[i].Name, Value))
			{
				if (Value != TestAttributeList[i].Value)
				{
					UE_LOG_ONLINE(Log,
						TEXT("Attribute %s is the wrong value in the received message payload"), *TestAttributeList[i].Name);
				}
				else
				{
					UE_LOG_ONLINE(Log,
						TEXT("Attribute %s MATCHED in the received message payload"), *TestAttributeList[i].Name);
				}
			}
			else
			{
				UE_LOG_ONLINE(Log,
					TEXT("Attribute %s is missing from the received message payload"), *TestAttributeList[i].Name);
			}
		}

		{
			FVariantData BLOBValue;
			if (Message->Payload.GetAttribute(TEXT("BLOBValue"), BLOBValue))
			{
				TArray<uint8> TestData;
				BLOBValue.GetValue(TestData);
				if (TestData.Num() != sizeof(BLOBTestValue))
				{
					UE_LOG_ONLINE(Log,
						TEXT("Attribute BLOBValue is the wrong size in the received message payload"));
				}
				else
				{
					bool bIsDataGood = true;
					for (int i = 0; i < sizeof(BLOBTestValue); ++i)
					{
						if (TestData[i] != BLOBTestValue[i])
						{
							UE_LOG_ONLINE(Log,
								TEXT("Attribute BLOBValue contains the wrong data at position %d in the received message payload"), i);
							bIsDataGood = false;
							break;
						}
					}

					if (bIsDataGood)
					{
						UE_LOG_ONLINE(Log,
							TEXT("Attribute BLOBValue MATCHED in the received message payload"));
					}
				}
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("Attribute BLOBValue is missing from the received message payload"));
			}
		}
	}

	// done with this part of the test if no more messages to download
	MessagesToRead.RemoveAt(0);
	if (MessagesToRead.Num() == 0)
	{
		bReadMessages = false;
	}
	// kick off next test
	StartNextTest();
}

void FTestMessageInterface::OnSendMessageComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ErrorStr)
{
	UE_LOG_ONLINE(Log,
		TEXT("SendMessage() for player (%d) was success=%d"), LocalPlayer, bWasSuccessful);

	// done with this part of the test
	bSendMessages = false;
	bEnumerateMessages = true;
	// kick off next test
	StartNextTest();
}

void FTestMessageInterface::OnDeleteMessageComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueMessageId& MessageId, const FString& ErrorStr)
{
	UE_LOG_ONLINE(Log,
		TEXT("DeleteMessage() for player (%d) was success=%d"), LocalPlayer, bWasSuccessful);

	// done with this part of the test if no more messages to delete
	MessagesToDelete.RemoveAt(0);
	if (MessagesToDelete.Num() == 0)
	{
		bDeleteMessages = false;
	}
	// kick off next test
	StartNextTest();
}

#endif //WITH_DEV_AUTOMATION_TESTS
