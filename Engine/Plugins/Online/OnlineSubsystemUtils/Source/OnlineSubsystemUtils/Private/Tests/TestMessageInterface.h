// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineMessageInterface.h"

class IOnlineSubsystem;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the online messaging interface
 */
class FTestMessageInterface
{
private:

	/** The subsystem that was requested to be tested or the default if empty */
	FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineSubsystem* OnlineSub;
	/** Delegate to use for enumerating messages for a user */
	FOnEnumerateMessagesCompleteDelegate OnEnumerateMessagesCompleteDelegate;
	/** Delegate to use for downloading messages for a user */
	FOnReadMessageCompleteDelegate OnReadMessageCompleteDelegate;
	/** Delegate to use for sending messages for a user */
	FOnSendMessageCompleteDelegate OnSendMessageCompleteDelegate;
	/** Delegate to use for deleting messages for a user */
	FOnDeleteMessageCompleteDelegate OnDeleteMessageCompleteDelegate;

	FDelegateHandle OnEnumerateMessagesCompleteDelegateHandle;
	FDelegateHandle OnReadMessageCompleteDelegateHandle;
	FDelegateHandle OnSendMessageCompleteDelegateHandle;
	FDelegateHandle OnDeleteMessageCompleteDelegateHandle;

	/** true to enable message enumeration */
	bool bEnumerateMessages;
	/** true to enable downloading of each message that was enumerated */
	bool bReadMessages;
	/** true to send test messages */
	bool bSendMessages;
	/** true to delete all messages that were enumerated */
	bool bDeleteMessages;
	/** List of messages to download */
	TArray<FUniqueMessageIdRef> MessagesToRead;
	/** List of recipients for the test message */
	TArray<FUniqueNetIdRef> Recipients;
	/** List of messages to delete */
	TArray<FUniqueMessageIdRef> MessagesToDelete;

	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/**
	 * See OnlineMessageInterface.h
	 */
	void OnEnumerateMessagesComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ErrorStr);
	void OnReadMessageComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueMessageId& MessageId, const FString& ErrorStr);
	void OnSendMessageComplete(int32 LocalPlayer, bool bWasSuccessful, const FString& ErrorStr);
	void OnDeleteMessageComplete(int32 LocalPlayer, bool bWasSuccessful, const FUniqueMessageId& MessageId, const FString& ErrorStr);

public:

	/**
	 *	Constructor which sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestMessageInterface(const FString& InSubsystem);

	/**
	 *	Destructor
	 */
	~FTestMessageInterface();

	/**
	 *	Kicks off all of the testing process
	 *
	 * @param InRecipients list of user ids to send test messages to
	 */
	void Test(class UWorld* InWorld, const TArray<FString>& InRecipients);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
