// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestPresenceInterface.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemUtils.h"
#include "OnlineSessionSettings.h"
#include "TimerManager.h"

#if WITH_DEV_AUTOMATION_TESTS

#define PRESENCE_TEST_LOG_KEY 50
#define PRESENCE_TEST_TIMEOUT 120

FTestPresenceInterface::~FTestPresenceInterface()
{
	CleanupTest();
}

void FTestPresenceInterface::CleanupTest()
{
	if (PresenceInt.IsValid())
	{
		PresenceInt->ClearOnPresenceReceivedDelegate_Handle(OnPresenceRecievedDelegateHandle);
	}

	if (SessionInt.IsValid())
	{
		SessionInt->ClearOnCreateSessionCompleteDelegate_Handle(OnCreateSessionCompleteDelegateHandle);
		if (bHasCreatedSession)
		{
			SessionInt->DestroySession(NAME_GameSession);
		}
	}

	if (World != nullptr)
	{
		World->GetTimerManager().ClearTimer(DelayedSessionPresenceWriteHandle);
	}

	if (GEngine)
	{
		GEngine->OnWorldDestroyed().RemoveAll(this);	
	}
}

void FTestPresenceInterface::OnWorldDestroyed(UWorld* InWorld)
{
	if (InWorld == World)
	{
		World = nullptr;
	}
}

FString FTestPresenceInterface::PrintTestFailure() const
{
	FString ReturnStr;
	if (EnumHasAllFlags(CompletedTasks, RequiredFlags))
	{
		return TEXT("No failures");
	}

	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::PushSelf))
	{
		ReturnStr += TEXT("<Self Push> ");
	}

	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::PushSelfSession))
	{
		ReturnStr += TEXT("<Self Session Push> ");
	}

	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::FetchSelf))
	{
		ReturnStr += TEXT("<Self Fetch> ");
	}

	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::FetchFriend))
	{
		ReturnStr += TEXT("<Friend Fetch> ");
	}

	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::WorkingDelegate))
	{
		ReturnStr += TEXT("<Delegate Updates> ");
	}

	if (EnumHasAnyFlags(RequiredFlags, EPresenceTestStatus::FetchRandom) && !EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::FetchRandom))
	{
		ReturnStr += TEXT("<Random User Fetch> ");
	}

	return ReturnStr;
}

bool FTestPresenceInterface::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTestPresenceInterface_Tick);

	// If we don't get an update within 2min while this test runs, mark the delegate watch task as a failure
	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::WorkingDelegate) &&
		FPlatformTime::Seconds() - TestTimeStart > PRESENCE_TEST_TIMEOUT &&
		TestTimeStart > 0)
	{
		bHasFailed = true;
		TasksAttempted |= EPresenceTestStatus::WorkingDelegate;
		TestTimeStart = 0; // Reset the time so we don't run the task again
	}

	// Handle failing out the arbitrary fetch test
	if (!EnumHasAnyFlags(CompletedTasks, EPresenceTestStatus::FetchRandom) &&
		EnumHasAnyFlags(RequiredFlags, EPresenceTestStatus::FetchRandom) &&
		FPlatformTime::Seconds() - ArbitraryFetchTimeStart > PRESENCE_TEST_TIMEOUT &&
		ArbitraryFetchTimeStart > 0)
	{
		bHasFailed = true;
		TasksAttempted |= EPresenceTestStatus::FetchRandom;
		ArbitraryFetchTimeStart = 0;
	}

	if (EnumHasAllFlags(CompletedTasks, RequiredFlags) || (bHasFailed && EnumHasAllFlags(TasksAttempted, RequiredFlags)))
	{
		UE_LOG_ONLINE_PRESENCE(Log, TEXT("Presence testing completed! Success: %d Failed Tasks: %s"), !bHasFailed, *PrintTestFailure());
		if (GEngine)
		{
			GEngine->AddOnScreenDebugMessage(PRESENCE_TEST_LOG_KEY, 5.f, (bHasFailed) ? FColor::Red : FColor::Green, 
				TEXT("Presence Test has completed. See log for details."));
		}
		delete this;
		return false;
	}
	else if (EnumHasAnyFlags(TasksAttempted, EPresenceTestStatus::PushSelf) && !bHasCreatedSession)
	{
		// Fail out if there is another session in use.
		if (SessionInt->GetNumSessions() != 0)
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence testing was aborted as another session was detected."));
			delete this;
			return false;
		}

		bHasCreatedSession = true;

		// Create a session
		FOnlineSessionSettings SessionSettings;
		SessionSettings.Set(SETTING_MAPNAME, FString(TEXT("PresenceTest")), EOnlineDataAdvertisementType::ViaOnlineService);
		SessionSettings.bAllowJoinViaPresence = true;
		SessionSettings.NumPublicConnections = 2;
		SessionSettings.bIsLANMatch = false;
		SessionSettings.bUsesPresence = true; // For steam, this will make the connection string not get pushed to the backend.
		SessionSettings.bUseLobbiesIfAvailable = true;
		SessionSettings.bShouldAdvertise = true;
		SessionSettings.bAllowJoinInProgress = true;
		SessionSettings.bAllowInvites = true;

		if (!SessionInt->CreateSession(*CurrentUser, NAME_GameSession, SessionSettings))
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test session failed to create!"));
			TasksAttempted |= EPresenceTestStatus::DelayedWriteFlags;
			bHasFailed = true;
		}
	}
	return true;
}

void FTestPresenceInterface::Test(UWorld* InWorld, const FString& RandomUser)
{
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("Starting Presence Interface Tests"));

	// Grab the OSS
	OnlineSub = Online::GetSubsystem(InWorld, FName(*SubsystemName));
	check(OnlineSub);
	check(GEngine);

	SessionInt = OnlineSub->GetSessionInterface();
	PresenceInt = OnlineSub->GetPresenceInterface();
	FriendsInt = OnlineSub->GetFriendsInterface();

	if (!SessionInt.IsValid() || !PresenceInt.IsValid() || !FriendsInt.IsValid() || !OnlineSub->GetIdentityInterface().IsValid())
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("The Presence test must be ran in an OSS that implements the Session, Identity, Presence and Friends interfaces"));
		delete this;
		return;
	}

	// As we need to push some session information around, it will be best that we require the user is not already in a session
	if (SessionInt->GetNumSessions() != 0)
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test cannot be run while already in a session."));
		delete this;
		return;
	}

	// Set the time out for watching for updates
	TestTimeStart = FPlatformTime::Seconds();

	// Create some delegate watchers and register them
	OnPresenceRecievedDelegate = FOnPresenceReceivedDelegate::CreateRaw(this, &FTestPresenceInterface::OnPresenceRecieved);
	OnPresenceRecievedDelegateHandle = PresenceInt->AddOnPresenceReceivedDelegate_Handle(OnPresenceRecievedDelegate);

	OnCreateSessionCompleteDelegate = FOnCreateSessionCompleteDelegate::CreateRaw(this, &FTestPresenceInterface::OnCreateSessionComplete);
	OnCreateSessionCompleteDelegateHandle = SessionInt->AddOnCreateSessionCompleteDelegate_Handle(OnCreateSessionCompleteDelegate);

	// Get the user
	CurrentUser = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);

	// If we get a null sharedpointer or if the netid is invalid, we cannot continue.
	if (!CurrentUser.IsValid() || !CurrentUser->IsValid())
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test requires a valid user logged in!"));
		delete this;
		return;
	}

	World = InWorld;
	// This test typically times out after two minutes. This timer is set for a bit longer than that, just in case.
	GEngine->AddOnScreenDebugMessage(PRESENCE_TEST_LOG_KEY, 145.0f, FColor::Yellow, TEXT("Presence Test is running. Please wait until it completes."));
	GEngine->OnWorldDestroyed().AddRaw(this, &FTestPresenceInterface::OnWorldDestroyed);

	// Grab the user's friends
	// This will then random from the friends list and fetch a profile from here.
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("Reading Friends List"));

	FriendsInt->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateRaw(this, &FTestPresenceInterface::OnReadFriendsListComplete));

	// First, fetch your own profile.
	PresenceInt->QueryPresence(*CurrentUser, IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateRaw(this, &FTestPresenceInterface::OnSelfPresenceFetchComplete));

	// Fetch a non-friend profile (if the platform supports it)
	if (!RandomUser.IsEmpty())
	{
		FUniqueNetIdPtr ArbitraryId = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(RandomUser);
		if (ArbitraryId.IsValid())
		{
			PresenceInt->QueryPresence(*ArbitraryId, IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateRaw(this, &FTestPresenceInterface::OnRandomUserFetchComplete));
			ArbitraryFetchTimeStart = FPlatformTime::Seconds();
			RequiredFlags |= EPresenceTestStatus::FetchRandom;
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Presence Interface Test could not create a player id for the given arbitrary user id! Skipping test!"));
		}
	}
	else
	{
		UE_LOG_ONLINE_PRESENCE(Log, TEXT("Presence Interface Test is skipping random user presence fetch"));
	}

	// Push a presence update
	FVariantData Val1, Val2, Val3;
	Val1.SetValue(TEXT("PresenceTestString"));
	Val2.SetValue(6);
	Val3.SetValue(-3.5f);
	CachedPresence.StatusStr = TEXT("CurrentlyTesting");
	CachedPresence.Properties.Add(DefaultPlatformKey, Val1);
	CachedPresence.Properties.Add(DefaultAppIdKey, Val2);
	CachedPresence.Properties.Add(CustomPresenceDataKey, Val3);

	PresenceInt->SetPresence(*CurrentUser, CachedPresence, IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateRaw(this, &FTestPresenceInterface::OnPresencePushComplete));
}

void FTestPresenceInterface::PrintPresence(const FUniqueNetId& UserId, TSharedPtr<FOnlineUserPresence>& PresenceData) const
{
	if (!UserId.IsValid() || !PresenceData.IsValid())
	{
		UE_LOG_ONLINE_PRESENCE(Log, TEXT("Could not print out status data as it is invalid!"));
		return;
	}

	UE_LOG_ONLINE_PRESENCE(Log, TEXT("UserId %s [%s] has presence: InSession[%d] Online[%d] Playing[%d] ThisGame[%d] Joinable[%d] StatusStr[%s]"), 
		*UserId.ToString(),
		EOnlinePresenceState::ToString(PresenceData->Status.State),
		((PresenceData->SessionId.IsValid()) ? PresenceData->SessionId->IsValid() : 0),
		PresenceData->bIsOnline,
		PresenceData->bIsPlaying,
		PresenceData->bIsPlayingThisGame,
		PresenceData->bIsJoinable,
		((PresenceData->Status.StatusStr.IsEmpty()) ? TEXT("No Status") : *PresenceData->Status.StatusStr));

	UE_LOG_ONLINE_PRESENCE(Log, TEXT("Properties [%d] Key=Value"), PresenceData->Status.Properties.Num());
	for (FPresenceProperties::TConstIterator Itr(PresenceData->Status.Properties); Itr; ++Itr)
	{
		const FString& Key = Itr.Key();
		const FString& Value = Itr.Value().ToString();
		UE_LOG_ONLINE_PRESENCE(Log, TEXT("%s=%s"), *Key, *Value);
	}
}

bool FTestPresenceInterface::DoesPresenceMatchWithCache(const TSharedPtr<FOnlineUserPresence>& PresenceToTest) const
{
	// This function will claim a test success if the following is true:
	//	- The tested presence is a valid ptr
	//	- The status strings are exactly the same
	//	- The number of keys in the tested presence are the same or more than the cached (to account for platform "magic keys")
	//	- All the keys in the cached presence appear in the tested presence
	//	- All the key values in the cached presence are the same in the tested presence
	// If any of these fail, the test fails.

	if (!PresenceToTest.IsValid())
	{
		return false;
	}

	if (CachedPresence.StatusStr != PresenceToTest->Status.StatusStr)
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test fails, status strings are different"));
		return false;
	}

	if (PresenceToTest->Status.Properties.Num() < CachedPresence.Properties.Num())
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test fails. Tested presence has %d keys over %d keys"), 
			PresenceToTest->Status.Properties.Num(),
			CachedPresence.Properties.Num());
		return false;
	}

	for (FPresenceProperties::TConstIterator Itr(CachedPresence.Properties); Itr; ++Itr)
	{
		const FString& Key = Itr.Key();
		const FString& Value = Itr.Value().ToString();
		if (PresenceToTest->Status.Properties.Find(Key) == nullptr)
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test fails, missing key %s"), *Key);
			return false;
		}

		const FString& TestValue = PresenceToTest->Status.Properties.Find(Key)->ToString();
		if (Value != TestValue)
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Presence test fails, key %s has different values. Cached=%s Has=%s"), *Key, *Value, *TestValue);
			return false;
		}
	}

	return true;
}

void FTestPresenceInterface::OnSelfPresenceFetchComplete(const FUniqueNetId& UserId, const bool bWasSuccessful)
{
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("FTestPresenceInterface::OnSelfPresenceFetchComplete Success=%d"), bWasSuccessful);
	TasksAttempted |= EPresenceTestStatus::FetchSelf;
	if (bWasSuccessful)
	{
		TSharedPtr<FOnlineUserPresence> PresenceData;
		if (PresenceInt.IsValid() && PresenceInt->GetCachedPresence(UserId, PresenceData) == EOnlineCachedResult::Success)
		{
			PrintPresence(UserId, PresenceData);
			CompletedTasks |= EPresenceTestStatus::FetchSelf;
			return;
		}
	}
	bHasFailed = true;
}

void FTestPresenceInterface::OnPresencePushComplete(const FUniqueNetId& UserId, const bool bWasSuccessful)
{
	bool bPushCorrect = false;
	if (bWasSuccessful)
	{
		TSharedPtr<FOnlineUserPresence> PresenceData;
		if (PresenceInt.IsValid() && PresenceInt->GetCachedPresence(*CurrentUser, PresenceData) == EOnlineCachedResult::Success)
		{
			PrintPresence(UserId, PresenceData);

			if (DoesPresenceMatchWithCache(PresenceData))
			{
				CompletedTasks |= EPresenceTestStatus::PushSelf;
				bPushCorrect = true;
			}
		}
	}

	if (!bPushCorrect)
	{
		bHasFailed = true;
	}
	
	TasksAttempted |= EPresenceTestStatus::PushSelf;
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("FTestPresenceInterface::OnPresencePushComplete Task Success=%d. Data correct=%d"), bWasSuccessful, bPushCorrect);
}

void FTestPresenceInterface::OnPresencePushWithSessionComplete(const FUniqueNetId& UserId, const bool bWasSuccessful)
{
	bool bPushCorrect = false;
	if (bWasSuccessful)
	{
		TSharedPtr<FOnlineUserPresence> PresenceData;
		if (PresenceInt.IsValid() && PresenceInt->GetCachedPresence(*CurrentUser, PresenceData) == EOnlineCachedResult::Success)
		{
			PrintPresence(UserId, PresenceData);
			if (PresenceData->bIsPlaying && PresenceData->bIsPlayingThisGame && 
				PresenceData->SessionId.IsValid() && PresenceData->SessionId->IsValid() && DoesPresenceMatchWithCache(PresenceData))
			{
				CompletedTasks |= EPresenceTestStatus::PushSelfSession;
				bPushCorrect = true;
			}
		}
	}

	if (!bPushCorrect)
	{
		bHasFailed = true;
	}

	TasksAttempted |= EPresenceTestStatus::PushSelfSession;
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("FTestPresenceInterface::OnPresencePushWithSessionComplete Task Success=%d. Data correct=%d"), bWasSuccessful, bPushCorrect);
}

void FTestPresenceInterface::OnFriendPresenceFetchComplete(const FUniqueNetId& UserId, const bool bWasSuccessful)
{
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("FTestPresenceInterface::OnFriendPresenceFetchComplete. User: %s. Success=%d"), *UserId.ToString(), bWasSuccessful);

	TasksAttempted |= EPresenceTestStatus::FetchFriend;
	if (bWasSuccessful)
	{
		TSharedPtr<FOnlineUserPresence> PresenceData;
		if (PresenceInt.IsValid() && PresenceInt->GetCachedPresence(UserId, PresenceData) == EOnlineCachedResult::Success)
		{
			PrintPresence(UserId, PresenceData);
			CompletedTasks |= EPresenceTestStatus::FetchFriend;
			return;
		}
	}
	bHasFailed = true;
}

void FTestPresenceInterface::OnRandomUserFetchComplete(const FUniqueNetId& UserId, const bool bWasSuccessful)
{
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("FTestPresenceInterface::OnRandomUserFetchComplete User: %s Success=%d"), *UserId.ToString(), bWasSuccessful);
	TasksAttempted |= EPresenceTestStatus::FetchRandom;
	if (bWasSuccessful)
	{
		TSharedPtr<FOnlineUserPresence> PresenceData;
		if (PresenceInt.IsValid() && PresenceInt->GetCachedPresence(UserId, PresenceData) == EOnlineCachedResult::Success)
		{
			PrintPresence(UserId, PresenceData);
			CompletedTasks |= EPresenceTestStatus::FetchRandom;
			return;
		}
	}
	bHasFailed = true;
}

void FTestPresenceInterface::OnPresenceRecieved(const FUniqueNetId& UserId, const TSharedRef<FOnlineUserPresence>& PresenceData)
{
	UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Received a presence update event about user %s"), *UserId.ToString());
	// We expect to get a status update about someone on your friends list at least.
	CompletedTasks |= EPresenceTestStatus::WorkingDelegate;
	TasksAttempted |= EPresenceTestStatus::WorkingDelegate;
}

void FTestPresenceInterface::OnReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("Friends list fetch task for presence test complete. Success=%d. Error=%s"), bWasSuccessful, *ErrorStr);

	if (bWasSuccessful)
	{
		FriendsInt->GetFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FriendsCache);

		UE_LOG_ONLINE_PRESENCE(Log, TEXT("Found %d friends!"), FriendsCache.Num());

		if (FriendsCache.Num() > 0)
		{
			int32 FriendToFetch = FMath::RandRange(0, FriendsCache.Num() - 1);
			if (FriendsCache.IsValidIndex(FriendToFetch))
			{
				UE_LOG_ONLINE_PRESENCE(Log, TEXT("Fetching presence for friend number %d"), FriendToFetch);

				// Start the friends data task
				PresenceInt->QueryPresence(FriendsCache[FriendToFetch]->GetUserId().Get(), IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateRaw(this, &FTestPresenceInterface::OnFriendPresenceFetchComplete));

				return;
			}
			else
			{
				UE_LOG_ONLINE_PRESENCE(Log, TEXT("Friend index not valid: %d"), FriendToFetch);
			}
		}
		else
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Test user has no friends! Cannot fetch friend presence data if user has no friends!"));
		}
	}

	TasksAttempted |= EPresenceTestStatus::FetchFriend;
	bHasFailed = true;
}

void FTestPresenceInterface::OnCreateSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_PRESENCE(Log, TEXT("Session creation task for presence test complete. Success? %d"), bWasSuccessful);
	if (bWasSuccessful && World != nullptr)
	{
		// Add a delay write to pushing new session information for the following reasons:
		//	- On some platforms, session information isn't immediately populated or present in the cache
		//	- See if existing presence gets overwritten properly with new values
		World->GetTimerManager().SetTimer(DelayedSessionPresenceWriteHandle, 
			FTimerDelegate::CreateRaw(this, &FTestPresenceInterface::DelayedSessionPresenceWrite), 1.0f, false, 5.0f);
		return;
	}

	TasksAttempted |= EPresenceTestStatus::DelayedWriteFlags;
	bHasFailed = true;
}

void FTestPresenceInterface::DelayedSessionPresenceWrite()
{
	// If by the time this task has completed and you have not gotten a working delegate call,
	// we will consider this task as attempted.
	TasksAttempted |= EPresenceTestStatus::WorkingDelegate;

	// Push session information
	FVariantData Val1;
	Val1.SetValue(TEXT("UnrealEngine"));
	CachedPresence.Properties.Add(TEXT("Callsign"), Val1);
	CachedPresence.Properties[DefaultAppIdKey].SetValue(32);
	PresenceInt->SetPresence(*CurrentUser, CachedPresence,
		IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateRaw(this, &FTestPresenceInterface::OnPresencePushWithSessionComplete));
}

#endif
