// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Engine/TimerHandle.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlineSessionDelegates.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"

class FOnlineFriend;

#if WITH_DEV_AUTOMATION_TESTS

enum class EPresenceTestStatus : uint8
{
	None = 0,
	FetchSelf = 1 << 0,
	FetchFriend = 1 << 1,
	FetchRandom = 1 << 2,
	PushSelf = 1 << 3,
	PushSelfSession = 1 << 4,
	WorkingDelegate = 1 << 5,

	RequiredPassingFlags = FetchSelf|FetchFriend|PushSelf|PushSelfSession|WorkingDelegate,
	DelayedWriteFlags = PushSelfSession|WorkingDelegate
};

ENUM_CLASS_FLAGS(EPresenceTestStatus);

class FTestPresenceInterface : public FTSTickerObjectBase
{
public:
	FTestPresenceInterface(const FString& InSubName) :
		SubsystemName(InSubName),
		World(nullptr),
		RequiredFlags(EPresenceTestStatus::RequiredPassingFlags),
		CompletedTasks(EPresenceTestStatus::None),
		TasksAttempted(EPresenceTestStatus::None),
		bHasCreatedSession(false),
		bHasFailed(false),
		TestTimeStart(0.0),
		ArbitraryFetchTimeStart(0.0)
	{
	}

	virtual ~FTestPresenceInterface();

	// FTSTickerObjectBase
	bool Tick(float DeltaTime) override;

	void Test(class UWorld* InWorld, const FString& RandomUser);

	void PrintPresence(const FUniqueNetId& UserId, TSharedPtr<FOnlineUserPresence>& PresenceData) const;

private:
	FTestPresenceInterface() {}

	/** The subsystem that was requested to be tested or the default if empty */
	FString SubsystemName;

	/** Given world (for timers) */
	class UWorld* World;

	/** Online Subsystem */
	class IOnlineSubsystem* OnlineSub;

	/** Friends interface (for figuring out friends to test fetching with) */
	IOnlineFriendsPtr FriendsInt;

	/** Presence interface (the thing going to be tested) */
	IOnlinePresencePtr PresenceInt;

	/** Session interface (needed for pushing connection information) */
	IOnlineSessionPtr SessionInt;

	/** Cached friends */
	TArray<TSharedRef<FOnlineFriend> > FriendsCache;

	/** Current user */
	FUniqueNetIdPtr CurrentUser;

	/** Statuses */
	EPresenceTestStatus RequiredFlags;
	EPresenceTestStatus CompletedTasks;
	EPresenceTestStatus TasksAttempted;
	bool bHasCreatedSession;
	bool bHasFailed;
	double TestTimeStart;
	double ArbitraryFetchTimeStart;

	/** Cached presence object for testing if the presences are the same */
	FOnlineUserPresenceStatus CachedPresence;

	/** Delegates */
	FOnPresenceReceivedDelegate OnPresenceRecievedDelegate;
	FOnCreateSessionCompleteDelegate OnCreateSessionCompleteDelegate;
	
	/** Delegate handles */
	FDelegateHandle OnPresenceRecievedDelegateHandle;
	FDelegateHandle OnCreateSessionCompleteDelegateHandle;

	/** Delegate functions */
	void OnSelfPresenceFetchComplete(const FUniqueNetId& UserId, const bool bWasSuccessful);
	void OnPresencePushComplete(const FUniqueNetId& UserId, const bool bWasSuccessful);
	void OnPresencePushWithSessionComplete(const FUniqueNetId& UserId, const bool bWasSuccessful);
	void OnFriendPresenceFetchComplete(const FUniqueNetId& UserId, const bool bWasSuccessful);
	void OnRandomUserFetchComplete(const FUniqueNetId& UserId, const bool bWasSuccessful);
	void OnPresenceRecieved(const FUniqueNetId& UserId, const TSharedRef<FOnlineUserPresence>& PresenceData);
	void OnReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr);
	void OnCreateSessionComplete(FName SessionName, bool bWasSuccessful);
	void OnWorldDestroyed(UWorld* InWorld);

	/** Timers for making sure session information is propagated on the backend */
	FTimerHandle DelayedSessionPresenceWriteHandle;
	void DelayedSessionPresenceWrite();

	/** Human readable str of failed tests */
	FString PrintTestFailure() const;

	/** Testing if presence push is correct */
	bool DoesPresenceMatchWithCache(const TSharedPtr<FOnlineUserPresence>& PresenceToTest) const;

	/** Clean up */
	void CleanupTest();
};

#endif //WITH_DEV_AUTOMATION_TESTS
