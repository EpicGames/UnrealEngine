// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tests/TestPurchaseInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

FTestPurchaseInterface::FTestPurchaseInterface(const FString& InSubsystem)
	: OnlineSub(nullptr)
{
	UE_LOG_ONLINE_PURCHASE(Display, TEXT("FTestPurchaseInterface::FTestPurchaseInterface"));
	SubsystemName = InSubsystem;
}

FTestPurchaseInterface::~FTestPurchaseInterface()
{
	UE_LOG_ONLINE_PURCHASE(Display, TEXT("FTestPurchaseInterface::~FTestPurchaseInterface"));
}

void FTestPurchaseInterface::Test(UWorld* InWorld, const FString& Namespace, const TArray<FString>& InOffersIds)
{
	UE_LOG_ONLINE_PURCHASE(Display, TEXT("FTestPurchaseInterface::Test"));

	OnlineSub = Online::GetSubsystem(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineSub != nullptr)
	{
		if (OnlineSub->GetIdentityInterface().IsValid())
		{
			LocalUserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
		}
		if (LocalUserId.IsValid())
		{
			if (OnlineSub->GetPurchaseInterface().IsValid())
			{
				if (InOffersIds.Num() > 0)
				{
					// list of offers
					for (const FString& OfferId : InOffersIds)
					{
						CheckoutRequest.AddPurchaseOffer(FOfferNamespace(Namespace), FUniqueOfferId(OfferId), 1);
					}
					// kick off next test
					StartNextTest();
					return;
				}
				else
				{
					UE_LOG_ONLINE_PURCHASE(Warning,
						TEXT("No offer ids specified for purchase"));
				}
			}
			else
			{
				UE_LOG_ONLINE_PURCHASE(Warning,
					TEXT("Failed to get Purchase interface for %s"), *SubsystemName);
			}
		}
		else
		{
			UE_LOG_ONLINE_PURCHASE(Warning,
				TEXT("No logged in user."));
		}
	}
	else
	{
		UE_LOG_ONLINE_PURCHASE(Warning,
			TEXT("Failed to get OSS interface for %s."), *SubsystemName);
	}
	// finish due to failure
	FinishTest();
}

void FTestPurchaseInterface::StartNextTest()
{
	if (!OnlineSub->GetPurchaseInterface()->IsAllowedToPurchase(*LocalUserId))
	{
		UE_LOG_ONLINE_PURCHASE(Warning,
			TEXT("Purchasing not allowed for user %s"), *LocalUserId->ToDebugString());
		
		FinishTest();
	}
	else
	{
		// query receipts prior to checkout
		OnlineSub->GetPurchaseInterface()->QueryReceipts(*LocalUserId, true,
			FOnQueryReceiptsComplete::CreateRaw(this, &FTestPurchaseInterface::OnQueryReceiptsPreCheckout));
	}
}

static void DumpReceiptInfo(const TArray<FPurchaseReceipt>& Receipts)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("Receipts=[%d]"), Receipts.Num());
	for (const FPurchaseReceipt& Receipt : Receipts)
	{
		UE_LOG_ONLINE_PURCHASE(Log, TEXT("%s"), *Receipt.ToJson());
	}
}

void FTestPurchaseInterface::OnQueryReceiptsPreCheckout(const FOnlineError& Result)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("Query receipts pre-checkout completed with Result=[%s]"), *Result.ToLogString());

	TArray<FPurchaseReceipt> Receipts;
	OnlineSub->GetPurchaseInterface()->GetReceipts(*LocalUserId, Receipts);
	DumpReceiptInfo(Receipts);	

	// purchase requested offers
	OnlineSub->GetPurchaseInterface()->Checkout(*LocalUserId, CheckoutRequest,
		FOnPurchaseCheckoutComplete::CreateRaw(this, &FTestPurchaseInterface::OnCheckoutComplete));
}

void FTestPurchaseInterface::OnCheckoutComplete(const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("Checkout completed with Result=[%s]"), *Result.ToLogString());

	// query receipts again after checkout
	OnlineSub->GetPurchaseInterface()->QueryReceipts(*LocalUserId, true,
		FOnQueryReceiptsComplete::CreateRaw(this, &FTestPurchaseInterface::OnQueryReceiptsPostCheckout));
}

void FTestPurchaseInterface::OnQueryReceiptsPostCheckout(const FOnlineError& Result)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("Query receipts post-checkout completed with Result=[%s]"), *Result.ToLogString());

	TArray<FPurchaseReceipt> Receipts;
	OnlineSub->GetPurchaseInterface()->GetReceipts(*LocalUserId, Receipts);
	DumpReceiptInfo(Receipts);

	// completed
	FinishTest();
}

void FTestPurchaseInterface::FinishTest()
{
	delete this;
}

#endif //WITH_DEV_AUTOMATION_TESTS
