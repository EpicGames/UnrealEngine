// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE includes


// Module includes
#include "Interfaces/OnlinePurchaseInterface.h"

class IOnlineSubsystem;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the Purchase interface
 */
class FTestPurchaseInterface
{

private:

	/** The subsystem that was requested to be tested or the default if empty */
	FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineSubsystem* OnlineSub;
	/** Logged in user */
	FUniqueNetIdPtr LocalUserId;
	/** Contains the product id to be purchased */
	FPurchaseCheckoutRequest CheckoutRequest;
	
	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/** checkout completion */
	void OnCheckoutComplete(const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt);
	/** receipt query completion */
	void OnQueryReceiptsPreCheckout(const FOnlineError& Result);
	/** receipt query completion */
	void OnQueryReceiptsPostCheckout(const FOnlineError& Result);

public:

	/**
	 *	Constructor which sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestPurchaseInterface(const FString& InSubsystem);

	/**
	 *	Destructor
	 */
	~FTestPurchaseInterface();

	/**
	 *	Kicks off all of the testing process
	 *
	 * @param Namespace for the offers
	 * @param InPurchaseIds list of offer/product ids to to purchase
	 */
	void Test(class UWorld* InWorld, const FString& Namespace, const TArray<FString>& InOffersIds);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
