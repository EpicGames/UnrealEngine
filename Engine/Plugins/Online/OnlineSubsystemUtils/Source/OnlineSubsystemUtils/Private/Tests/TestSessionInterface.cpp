// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestSessionInterface.h"
#include "GameFramework/GameMode.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemUtils.h"


#if WITH_DEV_AUTOMATION_TESTS

/**
 *	Example of a hosted session
 */
class TestOnlineGameSettings : public FOnlineSessionSettings
{
 public:
	TestOnlineGameSettings(bool bTestingLAN = false, const FOnlineSessionSettings& SettingsOverride = FOnlineSessionSettings())
	{
		NumPublicConnections = 10;
		NumPrivateConnections = 0;
		bIsLANMatch = bTestingLAN;
		bShouldAdvertise = true;
		bAllowJoinInProgress = true;
		bAllowInvites = true;
		bAllowJoinViaPresence = true;
		bAllowJoinViaPresenceFriendsOnly = false;

		Set(FName(TEXT("TESTSETTING1")), (int32)5, EOnlineDataAdvertisementType::ViaOnlineService, 0);
		Set(FName(TEXT("TESTSETTING2")), (float)5.0f, EOnlineDataAdvertisementType::ViaOnlineService, 1);
		Set(FName(TEXT("TESTSETTING3")), FString(TEXT("Hello")), EOnlineDataAdvertisementType::ViaOnlineService, 2);
		Set(FName(TEXT("TESTSETTING4")), FString(TEXT("Test4")), EOnlineDataAdvertisementType::ViaPingOnly);
		Set(FName(TEXT("TESTSETTING5")), FString(TEXT("Test5")), EOnlineDataAdvertisementType::ViaPingOnly);
		Set(SETTING_CUSTOM, FString(TEXT("CustomData123")), EOnlineDataAdvertisementType::ViaOnlineService);

		for (FSessionSettings::TConstIterator It(SettingsOverride.Settings); It; ++It)
		{
			FName Key = It.Key();
			const FOnlineSessionSetting& Setting = It.Value();

			FOnlineSessionSetting* HostSetting = Settings.Find(Key);
			if (HostSetting)
			{
				HostSetting->Data = Setting.Data;
			}
		}
	}
	
	virtual ~TestOnlineGameSettings()
	{
 
	}

	void AddWorldSettings(UWorld* InWorld)
	{
		if (InWorld)
		{
			const FString MapName = InWorld->GetMapName();
			Set(SETTING_MAPNAME, MapName, EOnlineDataAdvertisementType::ViaOnlineService);

			AGameModeBase const* const GameModeBase = InWorld->GetAuthGameMode();
			AGameMode const* const GameMode = Cast<AGameMode>(GameModeBase);
			if (GameModeBase != NULL)
			{
				// Game type
				FString GameModeStr = GameModeBase->GetClass()->GetName();
				Set(SETTING_GAMEMODE, GameModeStr, EOnlineDataAdvertisementType::ViaOnlineService);
			}

			if (GameMode != NULL)
			{
				// Bot count
				Set(SETTING_NUMBOTS, GameMode->NumBots, EOnlineDataAdvertisementType::ViaOnlineService);
			}
		}
	}
 };

/**
 *	Example of a session search query
 */
class TestOnlineSearchSettings : public FOnlineSessionSearch
{
public:
	TestOnlineSearchSettings(bool bSearchingLAN = false, const FOnlineSessionSettings& SettingsOverride = FOnlineSessionSettings())
	{
		bIsLanQuery = bSearchingLAN;
		MaxSearchResults = 10;
		PingBucketSize = 50;

		QuerySettings.Set(FName(TEXT("TESTSETTING1")), (int32)5, EOnlineComparisonOp::Equals, 0);
		QuerySettings.Set(FName(TEXT("TESTSETTING2")), (float)5.0f, EOnlineComparisonOp::Equals, 1);
		QuerySettings.Set(FName(TEXT("TESTSETTING3")), FString(TEXT("Hello")), EOnlineComparisonOp::Equals, 2);

		for (FSessionSettings::TConstIterator It(SettingsOverride.Settings); It; ++It)
		{
			FName Key = It.Key();
			const FOnlineSessionSetting& Setting = It.Value();

			FOnlineSessionSearchParam* QuerySetting = QuerySettings.SearchParams.Find(Key);
			if (QuerySetting)
			{
				QuerySetting->Data = Setting.Data;
			}
		}

	}

	virtual ~TestOnlineSearchSettings()
	{

	}
};

void FTestSessionInterface::Test(UWorld* InWorld, bool bTestLAN, bool bIsMatchmaking, const FOnlineSessionSettings& SettingsOverride)
{
	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(InWorld, FName(*Subsystem));
	check(OnlineSub);

	World = InWorld;
	GEngine->OnWorldDestroyed().AddRaw(this, &FTestSessionInterface::WorldDestroyed);

	if (OnlineSub->GetIdentityInterface().IsValid())
	{
		UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
	}

	// Cache interfaces
	Identity = OnlineSub->GetIdentityInterface();
	SessionInt = OnlineSub->GetSessionInterface();
	check(SessionInt.IsValid());
	Friends = OnlineSub->GetFriendsInterface();

	// Define delegates
	OnCreateSessionCompleteDelegate = FOnCreateSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnCreateSessionComplete);
	OnStartSessionCompleteDelegate = FOnStartSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnStartSessionComplete);
	OnEndSessionCompleteDelegate = FOnEndSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnEndSessionComplete);
	OnDestroySessionCompleteDelegate = FOnDestroySessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnDestroySessionComplete);

	OnUpdateSessionCompleteDelegate = FOnUpdateSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnUpdateSessionComplete);

	OnJoinSessionCompleteDelegate = FOnJoinSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnJoinSessionComplete);
	OnEndForJoinSessionCompleteDelegate = FOnEndSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnEndForJoinSessionComplete);
	OnDestroyForJoinSessionCompleteDelegate = FOnDestroySessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnDestroyForJoinSessionComplete);

	OnFindFriendSessionCompleteDelegate = FOnFindFriendSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnFindFriendSessionComplete);
	OnFindFriendSessionForListFriendSessionsCompleteDelegate = FOnFindFriendSessionCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnFindFriendSessionForListFriendSessionsComplete);

	OnRegisterPlayersCompleteDelegate = FOnRegisterPlayersCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnRegisterPlayerComplete);
	OnUnregisterPlayersCompleteDelegate = FOnRegisterPlayersCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnUnregisterPlayerComplete);

	OnFindSessionsCompleteDelegate = FOnFindSessionsCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnFindSessionsComplete);
	OnCancelFindSessionsCompleteDelegate = FOnCancelFindSessionsCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnCancelFindSessionsComplete);
	
	OnMatchmakingCompleteDelegate = FOnMatchmakingCompleteDelegate::CreateRaw(this, &FTestSessionInterface::OnMatchmakingComplete);

	// Read friends list and cache it
	if (Friends.IsValid())
	{
		Friends->ReadFriendsList(0, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateRaw(this, &FTestSessionInterface::OnReadFriendsListComplete));
	}

	if (bIsMatchmaking)
	{
		HostSettings = MakeShareable(new TestOnlineGameSettings(bTestLAN, SettingsOverride));
		HostSettings->AddWorldSettings(InWorld);

		SearchSettings = MakeShareable(new TestOnlineSearchSettings(bTestLAN, SettingsOverride));
		TSharedRef<FOnlineSessionSearch> SearchSettingsRef = SearchSettings.ToSharedRef();

		OnMatchmakingCompleteDelegateHandle = SessionInt->AddOnMatchmakingCompleteDelegate_Handle(OnMatchmakingCompleteDelegate);

		TArray<FUniqueNetIdRef> LocalPlayers;
		LocalPlayers.Add(UserId.ToSharedRef());

		SessionInt->StartMatchmaking(LocalPlayers, NAME_GameSession, *HostSettings, SearchSettingsRef);
	}
	// Setup sessions
	else if (bIsHost)
	{
		HostSettings = MakeShareable(new TestOnlineGameSettings(bTestLAN, SettingsOverride));
		HostSettings->AddWorldSettings(InWorld);

		OnCreateSessionCompleteDelegateHandle = SessionInt->AddOnCreateSessionCompleteDelegate_Handle(OnCreateSessionCompleteDelegate);
		SessionInt->CreateSession(0, NAME_GameSession, *HostSettings);
	}
	else
	{
		SearchSettings = MakeShareable(new TestOnlineSearchSettings(bTestLAN, SettingsOverride));
		TSharedRef<FOnlineSessionSearch> SearchSettingsRef = SearchSettings.ToSharedRef();

		OnFindSessionsCompleteDelegateHandle = SessionInt->AddOnFindSessionsCompleteDelegate_Handle(OnFindSessionsCompleteDelegate);
		SessionInt->FindSessions(0, SearchSettingsRef);
	}

	// Always on the lookout for invite acceptance (via actual invite or join in an external client)
	OnSessionUserInviteAcceptedDelegate = FOnSessionUserInviteAcceptedDelegate::CreateRaw(this, &FTestSessionInterface::OnSessionUserInviteAccepted);
	OnSessionUserInviteAcceptedDelegateHandle = SessionInt->AddOnSessionUserInviteAcceptedDelegate_Handle(OnSessionUserInviteAcceptedDelegate);
}

void FTestSessionInterface::ClearDelegates()
{
	SessionInt->ClearOnSessionUserInviteAcceptedDelegate_Handle(OnSessionUserInviteAcceptedDelegateHandle);
	GEngine->OnWorldDestroyed().RemoveAll(this);
}

void FTestSessionInterface::WorldDestroyed( UWorld* InWorld )
{
	if (InWorld == World)
	{
		World = NULL;
	}
}

void FTestSessionInterface::OnReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnReadFriendsListComplete LocalUserNum: %d bSuccess: %d %s"), LocalUserNum, bWasSuccessful, *ErrorStr);
	if (bWasSuccessful)
	{
		Friends->GetFriendsList(LocalUserNum, ListName, FriendsCache);
	}
}

void FTestSessionInterface::OnSessionUserInviteAccepted(bool bWasSuccessful, int32 ControllerId, FUniqueNetIdPtr InUserId, const FOnlineSessionSearchResult& SearchResult)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnSessionInviteAccepted ControllerId: %d bSuccess: %d"), ControllerId, bWasSuccessful);
	// Don't clear invite accept delegate

	if (bWasSuccessful)
	{
		JoinSession(ControllerId, NAME_GameSession, SearchResult);
	}
}

void FTestSessionInterface::OnEndForJoinSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnEndForJoinSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnEndSessionCompleteDelegate_Handle(OnEndForJoinSessionCompleteDelegateHandle);
	OnDestroyForJoinSessionCompleteDelegateHandle = DestroyExistingSession(SessionName, OnDestroyForJoinSessionCompleteDelegate);
}

FDelegateHandle FTestSessionInterface::EndExistingSession(FName SessionName, FOnEndSessionCompleteDelegate& Delegate)
{
	FDelegateHandle DelegateHandle = SessionInt->AddOnEndSessionCompleteDelegate_Handle(Delegate);
	SessionInt->EndSession(SessionName);
	return DelegateHandle;
}

void FTestSessionInterface::OnDestroyForJoinSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnDestroyForJoinSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnDestroySessionCompleteDelegate_Handle(OnDestroyForJoinSessionCompleteDelegateHandle);
	JoinSession(0, SessionName, CachedSessionResult);
}

FDelegateHandle FTestSessionInterface::DestroyExistingSession(FName SessionName, FOnDestroySessionCompleteDelegate& Delegate)
{
	FDelegateHandle DelegateHandle = SessionInt->AddOnDestroySessionCompleteDelegate_Handle(Delegate);
	SessionInt->DestroySession(SessionName);
	return DelegateHandle;
}

void FTestSessionInterface::OnRegisterPlayerComplete(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnRegisterPlayerComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnRegisterPlayersCompleteDelegate_Handle(OnRegisterPlayersCompleteDelegateHandle);
}

void FTestSessionInterface::OnUnregisterPlayerComplete(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnUnregisterPlayerComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnUnregisterPlayersCompleteDelegate_Handle(OnUnregisterPlayersCompleteDelegateHandle);
}

void FTestSessionInterface::JoinSession(int32 ControllerId, FName SessionName, const FOnlineSessionSearchResult& SearchResult)
{
	// Clean up existing sessions if applicable
	EOnlineSessionState::Type SessionState = SessionInt->GetSessionState(SessionName);
	if (SessionState != EOnlineSessionState::NoSession)
	{
		CachedSessionResult = SearchResult;
		OnEndForJoinSessionCompleteDelegateHandle = EndExistingSession(SessionName, OnEndForJoinSessionCompleteDelegate);
	}
	else
	{
		OnJoinSessionCompleteDelegateHandle = SessionInt->AddOnJoinSessionCompleteDelegate_Handle(OnJoinSessionCompleteDelegate);
		SessionInt->JoinSession(ControllerId, SessionName, SearchResult);
	}
}

void FTestSessionInterface::OnCreateSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnCreateSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnCreateSessionCompleteDelegate_Handle(OnCreateSessionCompleteDelegateHandle);

	bWasLastCommandSuccessful = bWasSuccessful;
}

void FTestSessionInterface::OnStartSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnStartSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnStartSessionCompleteDelegate_Handle(OnStartSessionCompleteDelegateHandle);

	bWasLastCommandSuccessful = bWasSuccessful;
}

void FTestSessionInterface::OnEndSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnEndSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnEndSessionCompleteDelegate_Handle(OnEndSessionCompleteDelegateHandle);

	bWasLastCommandSuccessful = bWasSuccessful;
}

void FTestSessionInterface::OnDestroySessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnDestroySessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnDestroySessionCompleteDelegate_Handle(OnDestroySessionCompleteDelegateHandle);
}

void FTestSessionInterface::OnUpdateSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnUpdateSessionComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	SessionInt->ClearOnUpdateSessionCompleteDelegate_Handle(OnUpdateSessionCompleteDelegateHandle);
}

void FTestSessionInterface::OnJoinSessionComplete(FName SessionName, EOnJoinSessionCompleteResult::Type Result)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnJoinSessionComplete %s bSuccess: %d"), *SessionName.ToString(), static_cast<int32>(Result));
	SessionInt->ClearOnJoinSessionCompleteDelegate_Handle(OnJoinSessionCompleteDelegateHandle);

	if (Result == EOnJoinSessionCompleteResult::Success)
	{
		FString URL;
		if (World && SessionInt->GetResolvedConnectString(SessionName, URL))
		{
			APlayerController* PC = UGameplayStatics::GetPlayerController(World, 0);
			if (PC)
			{
				PC->ClientTravel(URL, TRAVEL_Absolute);
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to join session %s"), *SessionName.ToString());
		}
	}
}

void FTestSessionInterface::OnFindFriendSessionComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& SearchResult)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnFindFriendSessionComplete LocalUserNum: %d bSuccess: %d"), LocalUserNum, bWasSuccessful);

	FDelegateHandle DelegateHandle = OnFindFriendSessionCompleteDelegateHandles.FindRef(LocalUserNum);
	SessionInt->ClearOnFindFriendSessionCompleteDelegate_Handle(LocalUserNum, DelegateHandle);
	OnFindFriendSessionCompleteDelegateHandles.Remove(LocalUserNum);
	if (bWasSuccessful)
	{
		// Can't just use SearchResult.IsValid() here - it's possible the SessionInfo pointer is valid, but not the data until we actually join the session
		if (SearchResult.Num() > 0 && SearchResult[0].Session.OwningUserId.IsValid() && SearchResult[0].Session.SessionInfo.IsValid())
		{
			JoinSession(LocalUserNum, NAME_GameSession, SearchResult[0]);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Join friend returned no search result."));
		}
	}
}

void FTestSessionInterface::OnFindFriendSessionForListFriendSessionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& SearchResult)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnFindFriendSessionComplete LocalUserNum: %d bSuccess: %d Result Count: %d"), LocalUserNum, bWasSuccessful, SearchResult.Num());

	FDelegateHandle DelegateHandle = OnFindFriendSessionCompleteDelegateHandles.FindRef(LocalUserNum);
	SessionInt->ClearOnFindFriendSessionCompleteDelegate_Handle(LocalUserNum, DelegateHandle);
	OnFindFriendSessionCompleteDelegateHandles.Remove(LocalUserNum);

	for (const FOnlineSessionSearchResult& Result : SearchResult)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("\tSession:"));
		DumpSession(&Result.Session);
	}
}

void FTestSessionInterface::OnFindSessionsComplete(bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnFindSessionsComplete bSuccess: %d"), bWasSuccessful);
	SessionInt->ClearOnFindSessionsCompleteDelegate_Handle(OnFindSessionsCompleteDelegateHandle);

	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Num Search Results: %d"), SearchSettings->SearchResults.Num());
	for (int32 SearchIdx=0; SearchIdx<SearchSettings->SearchResults.Num(); SearchIdx++)
	{
		const FOnlineSessionSearchResult& SearchResult = SearchSettings->SearchResults[SearchIdx];
		DumpSession(&SearchResult.Session);
	}
}

void FTestSessionInterface::OnFindSessionByIdComplete(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& SearchResult)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnFindSessionByIdComplete bSuccess: %d"), bWasSuccessful);

	DumpSession(&SearchResult.Session);

	SearchSettings = MakeShared<TestOnlineSearchSettings>(false);
	SearchSettings->SearchResults.Emplace(SearchResult);
}

void FTestSessionInterface::OnCancelFindSessionsComplete(bool bWasSuccessful)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnCancelFindSessionsComplete bSuccess: %d"), bWasSuccessful);
	SessionInt->ClearOnCancelFindSessionsCompleteDelegate_Handle(OnCancelFindSessionsCompleteDelegateHandle);
}

void FTestSessionInterface::OnMatchmakingComplete(FName SessionName, bool bWasSuccessful)
{
	SessionInt->ClearOnMatchmakingCompleteDelegate_Handle(OnMatchmakingCompleteDelegateHandle);
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("OnMatchmakingComplete %s bSuccess: %d"), *SessionName.ToString(), bWasSuccessful);
	if (bWasSuccessful)
	{
		SessionInt->DumpSessionState();
	}
}

bool FTestSessionInterface::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTestSessionInterface_Tick);

	if (TestPhase != LastTestPhase)
	{
		LastTestPhase = TestPhase;
		if (!bOverallSuccess)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("Testing failed in phase %d"), LastTestPhase);
			TestPhase = 3;
		}

		switch(TestPhase)
		{
		case 0:
			break;
		case 1:
			UE_LOG_ONLINE_SESSION(Log, TEXT("TESTING COMPLETE Success:%s!"), bOverallSuccess ? TEXT("true") : TEXT("false"));
			delete this;
			return false;
		}
	}
	return true;
}

bool FTestSessionInterface::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	int32 LocalUserNum = 0;

	// Ignore any execs that don't start with TESTSESSION
	if (FParse::Command(&Cmd, TEXT("TESTSESSION")))
	{
		// Get optional session name "GAME" otherwise
		FString Token;
		FName SessionName = FParse::Value(Cmd, TEXT("Name="), Token) ? FName(*Token) : NAME_GameSession;
		if (Token.Len() > 0)
		{
			Cmd += FCString::Strlen(TEXT("Name=")) + Token.Len();
		}
		
		if (FParse::Command(&Cmd, TEXT("SEARCH")))
		{
			if (SearchSettings.IsValid())
			{
				TSharedRef<FOnlineSessionSearch> SearchSettingsRef = SearchSettings.ToSharedRef();

				OnFindSessionsCompleteDelegateHandle = SessionInt->AddOnFindSessionsCompleteDelegate_Handle(OnFindSessionsCompleteDelegate);
				SessionInt->FindSessions(LocalUserNum, SearchSettingsRef);
			}
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("FINDBYID")))
		{
			FString SessionIdStr;
			if (FParse::Token(Cmd, SessionIdStr, true))
			{
				FUniqueNetIdPtr SessionId = SessionInt->CreateSessionIdFromString(SessionIdStr);
				if (SessionId.IsValid())
				{
					FOnSingleSessionResultCompleteDelegate CompletionDelegate;
					CompletionDelegate.BindRaw(this, &FTestSessionInterface::OnFindSessionByIdComplete);
					FUniqueNetIdPtr LocalUserId = Identity->GetUniquePlayerId(LocalUserNum);
					check(LocalUserId.IsValid());
					SessionInt->FindSessionById(*LocalUserId, *SessionId, *LocalUserId, CompletionDelegate);
				}
			}
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("JOIN")))
		{
			int32 SearchIdx = 0;
			TCHAR SearchIdxStr[256];
			if (FParse::Token(Cmd, SearchIdxStr, UE_ARRAY_COUNT(SearchIdxStr), true))
			{
				SearchIdx = FCString::Atoi(SearchIdxStr);
			}
			if (SearchSettings->SearchResults.IsValidIndex(SearchIdx))
			{
				JoinSession(LocalUserNum, SessionName, SearchSettings->SearchResults[SearchIdx]);
			}
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("JOINFRIEND")))
		{
			if (FParse::Command(&Cmd, TEXT("LOBBY")))
			{
				TCHAR FriendNameStr[256];
				if (FParse::Token(Cmd, FriendNameStr, UE_ARRAY_COUNT(FriendNameStr), true))
				{
					for (int32 FriendIdx=0; FriendIdx<FriendsCache.Num(); FriendIdx++)
					{
						const FOnlineFriend& Friend = *FriendsCache[FriendIdx];
						if (Friend.GetDisplayName() == FriendNameStr)
						{
							OnFindFriendSessionCompleteDelegateHandles.Add(LocalUserNum, SessionInt->AddOnFindFriendSessionCompleteDelegate_Handle(LocalUserNum, OnFindFriendSessionCompleteDelegate));
							SessionInt->FindFriendSession(LocalUserNum, *Friend.GetUserId());
							break;
						}
					}
				}
			}
			else
			{
				TCHAR FriendIdStr[256];
				if (FParse::Token(Cmd, FriendIdStr, UE_ARRAY_COUNT(FriendIdStr), true))
				{
					FUniqueNetIdPtr FriendId = Identity->CreateUniquePlayerId((uint8*)FriendIdStr, FCString::Strlen(FriendIdStr));
					if (!FriendId.IsValid())
					{
						FriendId = Identity->CreateUniquePlayerId(FriendIdStr);
					}

					if (FriendId.IsValid())
					{
						OnFindFriendSessionCompleteDelegateHandles.Add(LocalUserNum, SessionInt->AddOnFindFriendSessionCompleteDelegate_Handle(LocalUserNum, OnFindFriendSessionCompleteDelegate));
						SessionInt->FindFriendSession(LocalUserNum, *FriendId);
					}
				}
			}
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("LISTFRIENDSESSIONS")))
		{
			TArray<FUniqueNetIdRef> FriendList;
			for (int32 FriendIdx = 0; FriendIdx < FriendsCache.Num(); FriendIdx++)
			{
				const FOnlineFriend& Friend = *FriendsCache[FriendIdx];
				FriendList.Add(Friend.GetUserId());
			}
			SessionInt->FindFriendSession(*Identity->GetUniquePlayerId(LocalUserNum).ToSharedRef(), FriendList);
			OnFindFriendSessionCompleteDelegateHandles.Add(LocalUserNum, SessionInt->AddOnFindFriendSessionCompleteDelegate_Handle(LocalUserNum, OnFindFriendSessionForListFriendSessionsCompleteDelegate));
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("CREATE")))
		{
			// Mutually exclusive settings
			bool bTestLAN = FParse::Command(&Cmd, TEXT("LAN")) ? true : false;
			bool bTestPresence = FParse::Command(&Cmd, TEXT("PRESENCE")) ? true : false;

			HostSettings = MakeShareable(new TestOnlineGameSettings(bTestLAN));
			HostSettings->bUsesPresence = bTestPresence;
			HostSettings->AddWorldSettings(InWorld);

			OnCreateSessionCompleteDelegateHandle = SessionInt->AddOnCreateSessionCompleteDelegate_Handle(OnCreateSessionCompleteDelegate);
			SessionInt->CreateSession(0, SessionName, *HostSettings);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("START")))
		{
			OnStartSessionCompleteDelegateHandle = SessionInt->AddOnStartSessionCompleteDelegate_Handle(OnStartSessionCompleteDelegate);
			SessionInt->StartSession(SessionName);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("UPDATE")))
		{
			bool bUpdateOnline = FParse::Command(&Cmd, TEXT("ONLINE")) ? true : false;
			OnUpdateSessionCompleteDelegateHandle = SessionInt->AddOnUpdateSessionCompleteDelegate_Handle(OnUpdateSessionCompleteDelegate);
			HostSettings->Set(FName(TEXT("UPDATESETTING1")), FString(TEXT("Test1")), EOnlineDataAdvertisementType::ViaOnlineService);
			HostSettings->Set(FName(TEXT("UPDATESETTING2")), 99, EOnlineDataAdvertisementType::ViaOnlineService);
			SessionInt->UpdateSession(SessionName, *HostSettings, bUpdateOnline);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("END")))
		{
			OnEndSessionCompleteDelegateHandle = EndExistingSession(SessionName, OnEndSessionCompleteDelegate);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("DESTROY")))
		{
			OnDestroySessionCompleteDelegateHandle = DestroyExistingSession(SessionName, OnDestroySessionCompleteDelegate);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("REGISTER")))
		{
			bool bWasInvited = FParse::Command(&Cmd, TEXT("INVITED")) ? true : false;
			OnRegisterPlayersCompleteDelegateHandle = SessionInt->AddOnRegisterPlayersCompleteDelegate_Handle(OnRegisterPlayersCompleteDelegate);
			SessionInt->RegisterPlayer(SessionName, *UserId, bWasInvited);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("UNREGISTER")))
		{
			OnUnregisterPlayersCompleteDelegateHandle = SessionInt->AddOnUnregisterPlayersCompleteDelegate_Handle(OnUnregisterPlayersCompleteDelegate);
			SessionInt->UnregisterPlayer(SessionName, *UserId);
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("INVITE")))
		{
			if (FParse::Command(&Cmd, TEXT("UI")))
			{
				Online::GetSubsystem(InWorld, FName(*Subsystem))->GetExternalUIInterface()->ShowInviteUI(LocalUserNum);
			}
			else
			{
				FString FriendStr = FParse::Token(Cmd, true);
				if (!FriendStr.IsEmpty())
				{
					bool bFound = false;
					for (auto Friend : FriendsCache)
					{
						if (Friend->GetDisplayName() == FriendStr)
						{
							SessionInt->SendSessionInviteToFriend(LocalUserNum, SessionName, *Friend->GetUserId());
							bFound = true;
							break;
						}
					}
					if (!bFound)
					{
						// use friend str as id instead of display name
						FUniqueNetIdPtr FriendUserId = Identity->CreateUniquePlayerId(FriendStr);
						if (FriendUserId.IsValid())
						{	
							SessionInt->SendSessionInviteToFriend(LocalUserNum, SessionName, *FriendUserId);
						}
					}
				}
			}

			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("DUMPSESSIONS")))
		{
			SessionInt->DumpSessionState();
			bWasHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("QUIT")))
		{
			UE_LOG_ONLINE_SESSION(Display, TEXT("Destroying TestSession."));
			SessionInt->CancelFindSessions();
			bWasHandled = true;
			// Make this last
			delete this;
		}
	}

	return bWasHandled;
}

#endif //WITH_DEV_AUTOMATION_TESTS
