// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Interfaces/OnlineSessionDelegates.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"
#include "Interfaces/OnlineSessionInterface.h"

class FOnlineFriend;

class TestOnlineGameSettings;
class TestOnlineSearchSettings;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the friends interface
 */
 class FTestSessionInterface : public FTSTickerObjectBase, public FSelfRegisteringExec
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString Subsystem;

	/** Keep track of success across all functions and callbacks */
	bool bOverallSuccess;

	/** Am I testing the host interface */
	bool bIsHost;

	/** Logged in UserId */
	FUniqueNetIdPtr UserId;

	/** World test is occurring in */
	UWorld* World;

	/** Friends list cached */
	TArray< TSharedRef<FOnlineFriend> > FriendsCache;

	/** Convenient access to the identity interface */
	IOnlineIdentityPtr Identity;
	/** Convenient access to the session interface */
	IOnlineSessionPtr SessionInt;
	/** Convenient access to friends interface */
	IOnlineFriendsPtr Friends;

	/** Delegate for handling an accepted invite */
	FOnSessionUserInviteAcceptedDelegate OnSessionUserInviteAcceptedDelegate;

	/** Delegate for creating a new session */
	FOnCreateSessionCompleteDelegate OnCreateSessionCompleteDelegate;
	/** Delegate for starting a session */
	FOnStartSessionCompleteDelegate OnStartSessionCompleteDelegate;
	/** Delegate for ending a session */
	FOnEndSessionCompleteDelegate OnEndSessionCompleteDelegate;
	/** Delegate for destroying a session */
	FOnDestroySessionCompleteDelegate OnDestroySessionCompleteDelegate;
	/** Delegate for updating a session */
	FOnUpdateSessionCompleteDelegate OnUpdateSessionCompleteDelegate;
	/** Delegate for registering player(s) with a session */
	FOnRegisterPlayersCompleteDelegate OnRegisterPlayersCompleteDelegate;
	/** Delegate for unregistering player(s) with a session */
	FOnUnregisterPlayersCompleteDelegate OnUnregisterPlayersCompleteDelegate;

	/** Delegate for searching for sessions */
	FOnFindSessionsCompleteDelegate OnFindSessionsCompleteDelegate;
	/** Delegate for canceling a search */
	FOnCancelFindSessionsCompleteDelegate OnCancelFindSessionsCompleteDelegate;

	/** Delegate after joining a session */
	FOnJoinSessionCompleteDelegate OnJoinSessionCompleteDelegate;

	/** Delegate for destroying a session after previously ending it */
	FOnEndSessionCompleteDelegate OnEndForJoinSessionCompleteDelegate;
	/** Delegate for joining a new session after previously destroying it */
	FOnDestroySessionCompleteDelegate OnDestroyForJoinSessionCompleteDelegate;

	/** Delegate for matchmaking */
	FOnMatchmakingCompleteDelegate OnMatchmakingCompleteDelegate;

	/** Handles for the above delegates */
	FDelegateHandle OnReadFriendsListCompleteDelegateHandle;
	FDelegateHandle OnSessionUserInviteAcceptedDelegateHandle;
	FDelegateHandle OnCreateSessionCompleteDelegateHandle;
	FDelegateHandle OnStartSessionCompleteDelegateHandle;
	FDelegateHandle OnEndSessionCompleteDelegateHandle;
	FDelegateHandle OnDestroySessionCompleteDelegateHandle;
	FDelegateHandle OnUpdateSessionCompleteDelegateHandle;
	FDelegateHandle OnRegisterPlayersCompleteDelegateHandle;
	FDelegateHandle OnUnregisterPlayersCompleteDelegateHandle;
	FDelegateHandle OnFindSessionsCompleteDelegateHandle;
	FDelegateHandle OnCancelFindSessionsCompleteDelegateHandle;
	FDelegateHandle OnJoinSessionCompleteDelegateHandle;
	FDelegateHandle OnEndForJoinSessionCompleteDelegateHandle;
	FDelegateHandle OnDestroyForJoinSessionCompleteDelegateHandle;
	FDelegateHandle OnMatchmakingCompleteDelegateHandle;

	/** Delegate for joining a friend (JIP) */
	FOnFindFriendSessionCompleteDelegate OnFindFriendSessionCompleteDelegate;
	FOnFindFriendSessionCompleteDelegate OnFindFriendSessionForListFriendSessionsCompleteDelegate;

	/** Per-player delegate handles for OnFindFriendSessionComplete */
	TMap<int32, FDelegateHandle> OnFindFriendSessionCompleteDelegateHandles;

	/** Settings defined when acting as host */
	TSharedPtr<class TestOnlineGameSettings> HostSettings;
	/** Search settings defined when searching as client */
	TSharedPtr<class TestOnlineSearchSettings> SearchSettings;

	/** Cached invite/search result while in the process of tearing down an existing session */
	FOnlineSessionSearchResult CachedSessionResult;

	/**
	 * Ends an existing session of a given name
	 *
	 * @param SessionName name of session to end
	 * @param Delegate delegate to call at session end
	 *
	 * @return delegate handle created when assigning the passed in delegate parameter
	 */
	FDelegateHandle EndExistingSession(FName SessionName, FOnEndSessionCompleteDelegate& Delegate);

	/**
	 * Destroys an existing session of a given name
	 *
	 * @param SessionName name of session to destroy
	 * @param Delegate delegate to call at session destruction
	 *
	 * @return delegate handle created when assigning the passed in delegate parameter
	 */
	FDelegateHandle DestroyExistingSession(FName SessionName, FOnDestroySessionCompleteDelegate& Delegate);

	/**
	 * Join a session of a given name after potentially tearing down an existing one
	 *
	 * @param LocalUserNum local user id
	 * @param SessionName name of session to join
	 * @param SearchResult the session to join
	 */
	void JoinSession(int32 ControllerId, FName SessionName, const FOnlineSessionSearchResult& SearchResult);

	/**
	 * Transition from ending a session to destroying a session
	 *
	 * @param SessionName session that just ended
	 * @param bWasSuccessful was the end session attempt successful
	 */
	void OnEndForJoinSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Transition from destroying a session to joining a new one of the same name
	 *
	 * @param SessionName name of session recently destroyed
	 * @param bWasSuccessful was the destroy attempt successful
	 */
	void OnDestroyForJoinSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Delegate used when the friends read request has completed
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ListName name of the friends list that was operated on
	 * @param ErrorStr string representing the error condition
	 */
	void OnReadFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr);

	/**
	 * Delegate fired when an invite request has been accepted (via external client)
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param ControllerId the controller number of the accepting user
	 * @param UserId the user being invited
	 * @param InviteResult the search/settings for the session we're joining via invite
	 */
	void OnSessionUserInviteAccepted(const bool bWasSuccessful, const int32 LocalUserNum, FUniqueNetIdPtr UserId, const FOnlineSessionSearchResult& SearchResult);

	/**
	 * Delegate fired when a session create request has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnCreateSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Delegate fired when the online session has transitioned to the started state
	 *
	 * @param SessionName the name of the session the that has transitioned to started
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnStartSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Delegate fired when the online session has transitioned to the ending state
	 *
	 * @param SessionName the name of the session the that was ended
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnEndSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Delegate fired when a destroying an online session has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnDestroySessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Delegate fired when an update to an online session has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnUpdateSessionComplete(FName SessionName, bool bWasSuccessful);

	/**
	 * Delegate fired when player(s) have been registered with the session
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param Players array of players registered with the call
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnRegisterPlayerComplete(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasSuccessful);
	
	/**
	 * Delegate fired when player(s) have been unregistered with the session
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param Players array of players unregistered with the call
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnUnregisterPlayerComplete(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasSuccessful);

	/**
	 * Delegate fired when the joining process for an online session has completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnJoinSessionComplete(FName SessionName, EOnJoinSessionCompleteResult::Type Result);

	/**
	 * Delegate fired when after finding a friend's session to join
	 *
	 * @param LocalUserNum local user attempting to join friend
	 * @param bWasSuccessful true if the async action completed without error, false if there was an 
	 * @param SearchResult search result containing friend's session information
	 */
	void OnFindFriendSessionComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& SearchResult);

	/**
	* Delegate fired when after requesting all friend sessions
	*
	* @param LocalUserNum local user attempting to join friend
	* @param bWasSuccessful true if the async action completed without error, false if there was an
	* @param SearchResult search result containing all friends' session information
	*/
	void OnFindFriendSessionForListFriendSessionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& SearchResult);

	/**
	 * Delegate fired when the search for an online session has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnFindSessionsComplete(bool bWasSuccessful);

	/**
	 * Delegate fired when the search for an online session has completed
	 *
	 * @param LocalUserNum the controller number of the accepting user
	 * @param bWasSuccessful the session was found and is joinable, false otherwise
	 * @param SearchResult the search/settings for the session result we've been given
	 */
	void OnFindSessionByIdComplete(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& SearchResult);

	/**
	 * Delegate fired when the cancellation of a search for an online session has completed
	 *
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnCancelFindSessionsComplete(bool bWasSuccessful);

	/**
	 * Delegate fired when matchmaking has been completed
	 *
	 * @param SessionName the name of the session this callback is for
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	void OnMatchmakingComplete(FName SessionName, bool bWasSuccessful);

	/** Allows the world pointer to be cleaned up if it is going to be destroyed */
	void WorldDestroyed( UWorld* InWorld );

	/** Current phase of testing */
	int32 TestPhase;
	/** Last phase of testing triggered */
	int32 LastTestPhase;
	/** Was the last command run successful */
	bool bWasLastCommandSuccessful;

	/** Hidden on purpose */
	FTestSessionInterface() :
		Subsystem(),
		bOverallSuccess(false),
		bIsHost(false),
		Identity(NULL),
		SessionInt(NULL),
		Friends(NULL),
		TestPhase(0),
		LastTestPhase(-1),
		bWasLastCommandSuccessful(true)
	{
	}

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestSessionInterface(const FString& InSubsystem, bool bInIsHost) :
		Subsystem(InSubsystem),
		bOverallSuccess(true),
		bIsHost(bInIsHost),
		Identity(NULL),
		SessionInt(NULL),
		Friends(NULL),
		TestPhase(0),
		LastTestPhase(-1),
		bWasLastCommandSuccessful(true)
	{
	}

	~FTestSessionInterface()
	{
		ClearDelegates();
		SessionInt = NULL;
		Friends = NULL;
	}

	// FTSTickerObjectBase
	bool Tick(float DeltaTime) override;

	// FTestSessionInterface

	/**
	 * Kicks off all of the testing process
	 *z
	 * @param bTestLAN setup settings for LAN test
	 */
	void Test(UWorld* InWorld, bool bTestLAN, bool bIsMatchmaking, const FOnlineSessionSettings& SettingsOverride);

	/**
	 * Clear out any existing delegates created
	 */
	void ClearDelegates();

protected:
	// FSelfRegisteringExec
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
 };

#endif //WITH_DEV_AUTOMATION_TESTS
