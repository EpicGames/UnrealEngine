// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tests/TestSharingInterface.h"
#include "OnlineSubsystemUtils.h"
#include "ImageCore.h"

#if WITH_DEV_AUTOMATION_TESTS

FTestSharingInterface::FTestSharingInterface(const FString& InSubsystem)
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::FTestSharingInterface"));
	SubsystemName = InSubsystem;
}


FTestSharingInterface::~FTestSharingInterface()
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::~FTestSharingInterface"));
	
	if(TestStatusUpdate.Image)
	{
		delete TestStatusUpdate.Image;
		TestStatusUpdate.Image = NULL;
	}
}


void FTestSharingInterface::Test(UWorld* InWorld, bool bWithImage)
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::Test"));

	IOnlineSubsystem* OnlineSub = Online::GetSubsystem(InWorld, FName(*SubsystemName));
	check(OnlineSub); 

	SharingInterface = OnlineSub->GetSharingInterface();
	if (SharingInterface.IsValid())
	{
		TestStatusUpdate.Message = FString::Printf(TEXT("This is a test post for UE Sharing support! Date = %s"), *FDateTime::Now().ToString());
		TestStatusUpdate.PostPrivacy = EOnlineStatusUpdatePrivacy::OnlyMe;
		if (bWithImage)
		{
			TestStatusUpdate.Image = new FImage(256, 256, ERawImageFormat::BGRA8, EGammaSpace::Linear);
		}

		// Kick off the first part of the test,
		RequestPermissionsToSharePosts();
	}
	else
	{
		UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface TESTS SKIPPED. Sharing Interface implementation not found"));
		delete this;
	}
}


void FTestSharingInterface::RequestPermissionsToSharePosts()
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::RequestPermissionsToSharePosts"));

	ResponsesReceived = 0;
	RequestPermissionsToPostToFeedDelegate = FOnRequestNewPublishPermissionsCompleteDelegate::CreateRaw(this, &FTestSharingInterface::OnStatusPostingPermissionsUpdated);

	// We need to be permitted to post on the users behalf to share this update
	EOnlineSharingCategory PublishPermissions = EOnlineSharingCategory::SubmitPosts;

	for (int32 PlayerIndex = 0; PlayerIndex < MAX_LOCAL_PLAYERS; PlayerIndex++)
	{
		RequestPermissionsToPostToFeedDelegateHandles.Add(PlayerIndex, SharingInterface->AddOnRequestNewPublishPermissionsCompleteDelegate_Handle(PlayerIndex, RequestPermissionsToPostToFeedDelegate));
		SharingInterface->RequestNewPublishPermissions(PlayerIndex, PublishPermissions, TestStatusUpdate.PostPrivacy);
	}
}


void FTestSharingInterface::OnStatusPostingPermissionsUpdated(int32 LocalUserNum, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface::OnStatusPostingPermissionsUpdated() - %d"), bWasSuccessful);

	FDelegateHandle DelegateHandle = RequestPermissionsToPostToFeedDelegateHandles.FindRef(LocalUserNum);
	SharingInterface->ClearOnRequestNewPublishPermissionsCompleteDelegate_Handle(LocalUserNum, DelegateHandle);
	RequestPermissionsToPostToFeedDelegateHandles.Remove(LocalUserNum);

	if( ++ResponsesReceived == MAX_LOCAL_PLAYERS )
	{
		SharePost();
	}
}


void FTestSharingInterface::SharePost()
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::SharePost"));

	ResponsesReceived = 0;
	OnPostSharedDelegate = FOnSharePostCompleteDelegate::CreateRaw(this, &FTestSharingInterface::OnPostShared);

	for (int32 PlayerIndex = 0; PlayerIndex < MAX_LOCAL_PLAYERS; PlayerIndex++)
	{
		OnPostSharedDelegateHandles.Add(PlayerIndex, SharingInterface->AddOnSharePostCompleteDelegate_Handle(PlayerIndex, OnPostSharedDelegate));
		SharingInterface->ShareStatusUpdate(PlayerIndex, TestStatusUpdate);
	}
}


void FTestSharingInterface::OnPostShared(int32 LocalPlayer, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::OnPostShared[PlayerIdx:%i - Successful:%i]"), LocalPlayer, bWasSuccessful);

	FDelegateHandle DelegateHandle = OnPostSharedDelegateHandles.FindRef(LocalPlayer);
	SharingInterface->ClearOnSharePostCompleteDelegate_Handle(LocalPlayer, DelegateHandle);
	OnPostSharedDelegateHandles.Remove(LocalPlayer);
	if( ++ResponsesReceived == MAX_LOCAL_PLAYERS )
	{
		RequestPermissionsToReadNewsFeed();
	}
}


void FTestSharingInterface::RequestPermissionsToReadNewsFeed()
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::RequestPermissionsToReadNewsFeed"));

	ResponsesReceived = 0;
	RequestPermissionsToReadFeedDelegate = FOnRequestNewReadPermissionsCompleteDelegate::CreateRaw(this, &FTestSharingInterface::OnReadFeedPermissionsUpdated);

	// We need to be permitted to post on the users behalf to share this update
	EOnlineSharingCategory ReadPermissions = EOnlineSharingCategory::ReadPosts;

	for (int32 PlayerIndex = 0; PlayerIndex < MAX_LOCAL_PLAYERS; PlayerIndex++)
	{
		RequestPermissionsToReadFeedDelegateHandles.Add(PlayerIndex, SharingInterface->AddOnRequestNewReadPermissionsCompleteDelegate_Handle(PlayerIndex, RequestPermissionsToReadFeedDelegate));
		SharingInterface->RequestNewReadPermissions(PlayerIndex, ReadPermissions);
	}
}


void FTestSharingInterface::OnReadFeedPermissionsUpdated(int32 LocalUserNum, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface::OnReadFeedPermissionsUpdated() - %d"), bWasSuccessful);

	FDelegateHandle DelegateHandle = RequestPermissionsToReadFeedDelegateHandles.FindRef(LocalUserNum);
	SharingInterface->ClearOnRequestNewReadPermissionsCompleteDelegate_Handle(LocalUserNum, DelegateHandle);
	RequestPermissionsToReadFeedDelegateHandles.Remove(LocalUserNum);

	if( ++ResponsesReceived == MAX_LOCAL_PLAYERS )
	{
		ReadNewsFeed();
	}
}


void FTestSharingInterface::ReadNewsFeed()
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("FTestSharingInterface::ReadNewsFeed"));

	ResponsesReceived = 0;
	OnNewsFeedReadDelegate = FOnReadNewsFeedCompleteDelegate::CreateRaw(this, &FTestSharingInterface::OnNewsFeedRead);

	for (int32 PlayerIndex = 0; PlayerIndex < MAX_LOCAL_PLAYERS; PlayerIndex++)
	{
		OnNewsFeedReadDelegateHandles.Add(PlayerIndex, SharingInterface->AddOnReadNewsFeedCompleteDelegate_Handle(PlayerIndex, OnNewsFeedReadDelegate));
		SharingInterface->ReadNewsFeed(PlayerIndex);
	}
}


void FTestSharingInterface::OnNewsFeedRead(int32 LocalPlayer, bool bWasSuccessful)
{
	UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface::OnNewsFeedRead[PlayerIdx:%i - Successful:%i]"), LocalPlayer, bWasSuccessful);

	if (bWasSuccessful)
	{
		// Get the 1st cached post
		FOnlineStatusUpdate FirstReadStatusUpdate;
		SharingInterface->GetCachedNewsFeed(LocalPlayer, 0, FirstReadStatusUpdate);
		UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface first read update: %s"), *FirstReadStatusUpdate.Message);

		// Get all the cached posts
		TArray<FOnlineStatusUpdate> AllReadStatusUpdates;
		SharingInterface->GetCachedNewsFeeds(LocalPlayer, AllReadStatusUpdates);
		UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface number of read updates: %d"), AllReadStatusUpdates.Num());

		for (int Idx = 0; Idx < AllReadStatusUpdates.Num(); ++Idx)
		{
			const FOnlineStatusUpdate& StatusUpdate = AllReadStatusUpdates[Idx];
			UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface status update [%d]: %s"), Idx, *StatusUpdate.Message);
		}
	}

	FDelegateHandle DelegateHandle = OnNewsFeedReadDelegateHandles.FindRef(LocalPlayer);
	SharingInterface->ClearOnReadNewsFeedCompleteDelegate_Handle(LocalPlayer, DelegateHandle);
	if( ++ResponsesReceived == MAX_LOCAL_PLAYERS )
	{
		UE_LOG_ONLINE_SHARING(Display, TEXT("FTestSharingInterface TESTS COMPLETED"));
		delete this;
	}
}

#endif //WITH_DEV_AUTOMATION_TESTS
