// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE includes

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineSharingInterface.h"

// Module includes

class UWorld;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the sharing interface
 */
class FTestSharingInterface
{

private:

	/** The subsystem that was requested to be tested or the default if empty */
	FString SubsystemName;

	/** Access to the OSS Sharing Interface */
	IOnlineSharingPtr SharingInterface;

	/** The number of replies received from players */
	int32 ResponsesReceived;

	/** The status update object we are posting to the platform news feed */
	FOnlineStatusUpdate TestStatusUpdate;

private:
	
	/**
	 * Use the OSS to request permissions so we can post to the news feed.
	 */
	void RequestPermissionsToSharePosts();

	/**
	 * Delegate fired when we have a response after we have requested publish permissions to a users news feed.
	 *
	 * @param LocalUserNum		- The local player number of the user posting to fb
	 * @param bWasSuccessful	- Whether the permissions were updated successfully
	 */
	void OnStatusPostingPermissionsUpdated(int32 LocalUserNum, bool bWasSuccessful);


	/** Delegate used to notify this interface that permissions are updated, and that we can now post a status update */
	FOnRequestNewPublishPermissionsCompleteDelegate RequestPermissionsToPostToFeedDelegate;

	/** Per-player handles for the above delegate */
	TMap<int32, FDelegateHandle> RequestPermissionsToPostToFeedDelegateHandles;


	/**
	 * Use the OSS to request permissions so we can read the news feed.
	 */
	void RequestPermissionsToReadNewsFeed();

	/**
	 * Delegate fired when we have a response after we have requested read permissions of a users news feed
	 *
	 * @param LocalUserNum		- The local player number of the user posting to fb
	 * @param bWasSuccessful	- Whether the permissions were updated successfully
	 */
	void OnReadFeedPermissionsUpdated(int32 LocalUserNum, bool bWasSuccessful);


	/** Delegate used to notify this interface that permissions are updated, and that we can now read the news feed */
	FOnRequestNewReadPermissionsCompleteDelegate RequestPermissionsToReadFeedDelegate;


	/** Per-player handles for the above delegate */
	TMap<int32, FDelegateHandle> RequestPermissionsToReadFeedDelegateHandles;

private:

	/**
	 *	Create a test message and post it through the OSS to a feed
	 */
	void SharePost();


	/** Delegate which is reported whether a successful post was made to a users news feed */
	FOnSharePostCompleteDelegate OnPostSharedDelegate;


	/** Per-player handles for the above delegate */
	TMap<int32, FDelegateHandle> OnPostSharedDelegateHandles;


	/**
	 *	Fn bound to the FOnSharePostCompleteDelegate
	 *
	 * @param LocalUserNum		- The controller number of the associated user
	 * @param bWasSuccessful	- True if a post was successfully uploaded
	 */
	void OnPostShared(int32 LocalUserNum, bool bWasSuccessful);


private:

	/**
	 *	Read the news feeds through the OSS
	 */
	void ReadNewsFeed();


	/** Delegate which is reported whether reading the news feed was successful or not for the given user */
	FOnReadNewsFeedCompleteDelegate OnNewsFeedReadDelegate;


	/** Per-player handles for the above delegate */
	TMap<int32, FDelegateHandle> OnNewsFeedReadDelegateHandles;


	/**
	 *	Fn bound to the FOnReadNewsFeedCompleteDelegate
	 *
	 * @param LocalUserNum		- The controller number of the associated user
	 * @param bWasSuccessful	- True if a post was successfully uploaded
	 */
	void OnNewsFeedRead(int32 LocalPlayer, bool bWasSuccessful);


public:

	/**
	 *	Constructor which sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestSharingInterface(const FString& InSubsystem);
	

	/**
	 *	Destructor
	 */
	~FTestSharingInterface();


	/**
	 *	Kicks off all of the testing process
	 *
	 * @param bWithImage - Do we want to test sharing an image
	 */
	void Test(class UWorld* InWorld, bool bWithImage);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
