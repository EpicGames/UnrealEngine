// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestStatsInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

FTestStatsInterface::FTestStatsInterface(const FString& InSubsystem) :
	Subsystem(InSubsystem),
	bOverallSuccess(true),
	Stats(NULL),
	CurrentTestPhase(EStatsTestPhase::ReadStatsForOneUser),
	LastTestPhase(EStatsTestPhase::Invalid)
{
}

FTestStatsInterface::~FTestStatsInterface()
{
	Stats = NULL;
}

void FTestStatsInterface::Test(UWorld* InWorld, const TCHAR* Cmd)
{
	OnlineSub = Online::GetSubsystem(InWorld, FName(*Subsystem));
	if (!OnlineSub)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("[FTestStatsInterface::Test] Failed to get online subsystem %s"), *Subsystem);

		bOverallSuccess = false;
		return;
	}

	if (OnlineSub->GetIdentityInterface().IsValid())
	{
		UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
		
		if (!UserId.IsValid())
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("[FTestStatsInterface::Test] Couldn't find any valid user in identity interface"));

			bOverallSuccess = false;
			return;
		}
	}

	// Cache interfaces
	Stats = OnlineSub->GetStatsInterface();
	if (!Stats.IsValid())
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("[FTestStatsInterface::Test] Failed to get online Stats interface for online subsystem %s"), *Subsystem);

		bOverallSuccess = false;
		return;
	}

	FString StatToRead = FParse::Token(Cmd, false);
	while (!StatToRead.IsEmpty())
	{
		StatsToRead.Add(StatToRead);
		StatToRead = FParse::Token(Cmd, false);
	}

	if (StatsToRead.Num() == 0)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("[FTestStatsInterface::Test] No stat names were specified. Please add the stat names you want to use for the test."));

		bOverallSuccess = false;
		return;
	}
}

void FTestStatsInterface::WriteStats(bool bIncrementStats)
{
	UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::WriteStats]"));

	TArray<FOnlineStatsUserUpdatedStats> Writes;
	FOnlineStatsUserUpdatedStats& UpdatedValues = Writes.Emplace_GetRef(UserId.ToSharedRef());

	TSharedPtr<const FOnlineStatsUserStats> ReadUserStats = Stats->GetStats(UserId.ToSharedRef());
	if (ReadUserStats.IsValid())
	{
		for (const FString& StatName : StatsToRead)
		{
			if (const FOnlineStatValue* StatRead = ReadUserStats->Stats.Find(StatName))
			{
				if (StatRead->IsNumeric())
				{
					FOnlineStatValue NewData(*StatRead);

					bIncrementStats ? NewData.Increment() : NewData.Decrement();

					UE_LOG_ONLINE_STATS(Log, TEXT("Stat %s is being %s"), *StatName, bIncrementStats ? TEXT("incremented") : TEXT("decremented"));
					UpdatedValues.Stats.Add(*StatName, FOnlineStatUpdate(NewData, FOnlineStatUpdate::EOnlineStatModificationType::Set));
				}
				else
				{
					UE_LOG_ONLINE_STATS(Log, TEXT("Stat %s doesn't have a numeric value, skipped"), *StatName);
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_STATS(Log, TEXT("Stats not found for user %s"), *UserId->ToString());
	}

	// Write it to the buffers
	Stats->UpdateStats(UserId.ToSharedRef(), Writes, FOnlineStatsUpdateStatsComplete::CreateLambda([this](const FOnlineError& Error) {
		UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::WriteStats] Write test finished"));

		CurrentTestPhase++;

		if(!Error.bSucceeded)
		{
			UE_LOG_ONLINE_STATS(Error, TEXT("[FTestStatsInterface::WriteStats] WriteStats test failed with error: %s"), *Error.GetErrorMessage().ToString());
			bOverallSuccess = false;
		}
	}));
}

void FTestStatsInterface::ReadStats(bool bIncludeStatsToRead)
{
	if (bIncludeStatsToRead)
	{
		Stats->QueryStats(UserId.ToSharedRef(), { UserId.ToSharedRef() }, StatsToRead, FOnlineStatsQueryUsersStatsComplete::CreateRaw(this, &FTestStatsInterface::OnQueryUsersStatsComplete));
	}
	else
	{
		Stats->QueryStats(UserId.ToSharedRef(), UserId.ToSharedRef(), FOnlineStatsQueryUserStatsComplete::CreateRaw(this, &FTestStatsInterface::OnQueryUserStatsComplete));
	}
}

void FTestStatsInterface::OnQueryUserStatsComplete(const FOnlineError& ResultState, const TSharedPtr<const FOnlineStatsUserStats>& QueriedStats)
{
	UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::OnQueryUserStatsComplete] Read test finish with %d queried stats"), QueriedStats->Stats.Num());

	if (!ResultState.bSucceeded)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("[FTestStatsInterface::OnQueryUserStatsComplete] ReadStats first attempt failed with error: %s"), *ResultState.GetErrorMessage().ToString());

		if (ResultState.GetErrorResult() != EOnlineErrorResult::NotImplemented)
		{
			bOverallSuccess = false;
		}
	}
	else
	{
		for (const TPair<FString, FOnlineStatValue>& StatReads : QueriedStats->Stats)
		{
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::OnQueryUserStatsComplete] Queried stat with name %s has value %s"), *StatReads.Key, *StatReads.Value.ToString());
		}

		const TSharedPtr<const FOnlineStatsUserStats> ReadUserStats = Stats->GetStats(UserId.ToSharedRef());
		if (ReadUserStats.IsValid())
		{
			for (const TPair<FString, FOnlineStatValue>& StatReads : ReadUserStats->Stats)
			{
				UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::OnQueryUserStatsComplete] Cached stat with name %s has value %s"), *StatReads.Key, *StatReads.Value.ToString());
			}
		}
	}

	CurrentTestPhase++;
}

void FTestStatsInterface::OnQueryUsersStatsComplete(const FOnlineError& ResultState, const TArray<TSharedRef<const FOnlineStatsUserStats>>& UsersStatsResult)
{
	UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::OnQueryUsersStatsComplete] Read test finish for %d users with queried stats"), UsersStatsResult.Num());

	if (!ResultState.bSucceeded)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("[FTestStatsInterface::OnQueryUsersStatsComplete] ReadStats first attempt failed with error: %s"), *ResultState.GetErrorMessage().ToString());

		if (ResultState.GetErrorResult() != EOnlineErrorResult::NotImplemented)
		{
			bOverallSuccess = false;
		}
	}
	else
	{
		for (const TSharedRef<const FOnlineStatsUserStats>& QueriedStats : UsersStatsResult)
		{
			for (const TPair<FString, FOnlineStatValue>& StatReads : QueriedStats->Stats)
			{
				UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::OnQueryUsersStatsComplete] Queried stat with name %s has value %s"), *StatReads.Key, *StatReads.Value.ToString());
			}

			// We would keep track of all queried users here, but we only used our local user
			const TSharedPtr<const FOnlineStatsUserStats> ReadUserStats = Stats->GetStats(UserId.ToSharedRef());
			if (ReadUserStats.IsValid())
			{
				for (const TPair<FString, FOnlineStatValue>& StatReads : ReadUserStats->Stats)
				{
					UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::OnQueryUsersStatsComplete] Cached stat with name %s has value %s"), *StatReads.Key, *StatReads.Value.ToString());
				}
			}
		}	
	}

	CurrentTestPhase++;
}

bool FTestStatsInterface::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FTestStatsInterface_Tick);

	if (CurrentTestPhase != LastTestPhase)
	{
		if (!bOverallSuccess)
		{
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Testing failed in phase %d"), EnumToUnderlyingType(LastTestPhase));
			CurrentTestPhase = EStatsTestPhase::End;
		}

		LastTestPhase = CurrentTestPhase;

		switch (CurrentTestPhase)
		{
		case EStatsTestPhase::ReadStatsForOneUser:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning ReadStats (reading self, pre-increment, no stat names)"));
			ReadStats(false);
			break;
		case EStatsTestPhase::ReadStatsForManyUsers:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning ReadStats (reading self, pre-increment, including stat names)"));
			ReadStats(true);
			break;
		case EStatsTestPhase::WriteIncrementedStats:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning Write (incrementing stats)"));
			WriteStats(true);
			break;
		case EStatsTestPhase::ReadStatsForOneUserAfterIncrement:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning ReadStats (reading self, post-increment, no stat names)"));
			ReadStats(false);
			break;
		case EStatsTestPhase::ReadStatsForManyUsersaAfterIncrement:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning ReadStats (reading self, post-increment, including stat names)"));
			ReadStats(true);
			break;
		case EStatsTestPhase::WriteDecrementedStats:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning Write (decrementing stats)"));
			WriteStats(false);
			break;
		case EStatsTestPhase::ReadStatsForOneUserAfterDecrement:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning ReadStats (reading self, post-decrement, no stat names)"));
			ReadStats(false);
			break;
		case EStatsTestPhase::ReadStatsForManyUsersaAfterDecrement:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] Beginning ReadStats (reading self, post-decrement, including stat names)"));
			ReadStats(true);
			break;
		case EStatsTestPhase::End:
			UE_LOG_ONLINE_STATS(Log, TEXT("[FTestStatsInterface::Tick] TESTING COMPLETE Success: %s!"), *LexToString(bOverallSuccess));
			delete this;
			return false;
		}
	}
	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
