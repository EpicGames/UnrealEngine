// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Interfaces/OnlineStatsInterface.h"
#include "OnlineSubsystem.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the friends interface
 */
 class FTestStatsInterface : public FTSTickerObjectBase
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString Subsystem;

	/** Cached Online Subsystem Pointer */
	class IOnlineSubsystem* OnlineSub;

	/** Keep track of success across all functions and callbacks */
	bool bOverallSuccess;

	/** Names of stats to read, taken from the console command */
	TArray<FString> StatsToRead;

	/** Logged in UserId */
	FUniqueNetIdPtr UserId;

	/** Convenient access to the Stats interfaces */
	IOnlineStatsPtr Stats;

	enum class EStatsTestPhase
	{
		Invalid,
		ReadStatsForOneUser,
		ReadStatsForManyUsers,
		WriteIncrementedStats,
		ReadStatsForOneUserAfterIncrement,
		ReadStatsForManyUsersaAfterIncrement,
		WriteDecrementedStats,
		ReadStatsForOneUserAfterDecrement,
		ReadStatsForManyUsersaAfterDecrement,
		End
	};

	friend EStatsTestPhase& operator++(EStatsTestPhase& TestPhase) {
		const int TestPhaseInt = int(TestPhase) + 1;
		const int EndPhaseInt = int(EStatsTestPhase::End);
		TestPhase = EStatsTestPhase((TestPhaseInt > EndPhaseInt) ? EndPhaseInt : TestPhaseInt);
		return TestPhase;
	}

	friend EStatsTestPhase operator++(EStatsTestPhase& TestPhase, int) {
		const EStatsTestPhase Result = TestPhase;
		++TestPhase;
		return Result;
	}

	/** Current phase of testing */
	EStatsTestPhase CurrentTestPhase;
	/** Last phase of testing triggered */
	EStatsTestPhase LastTestPhase;

	/** Hidden on purpose */
	FTestStatsInterface()
		: Subsystem()
	{
	}

	/**
	 *	Write out some test data to a Stats
	 *
	 * @param bIncrementStats If this is true, stats written will be incremented, if it's not, they will be decremented
	 */
	void WriteStats(bool bIncrementStats);

	/**
	 *	Delegates called when a Stats has been successfully read, both for a single and multiple users
	 */
	void OnQueryUserStatsComplete(const FOnlineError& Error, const TSharedPtr<const FOnlineStatsUserStats>& QueriedStats);
	void OnQueryUsersStatsComplete(const FOnlineError& ResultState, const TArray<TSharedRef<const FOnlineStatsUserStats>>& UsersStatsResult);

	/**
	 * Read values for stats
	 *
	 * @param bIncludeStatsToRead Specify if the call should include stats defined in command line
	 */
	void ReadStats(bool bIncludeStatsToRead);

	/** Utilities */
	void PrintStats();

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestStatsInterface(const FString& InSubsystem);

	virtual ~FTestStatsInterface();

	// FTSTickerObjectBase

	bool Tick( float DeltaTime ) override;

	// FTestStatsInterface

	/**
	 * Kicks off all of the testing process
	 */
	void Test(UWorld* InWorld, const TCHAR* Cmd);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
