// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tests/TestStoreInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

FTestStoreInterface::FTestStoreInterface(const FString& InSubsystem)
	: OnlineSub(nullptr)
{
	UE_LOG_ONLINE_STOREV2(Display, TEXT("FTestStoreInterface::FTestStoreInterface"));
	SubsystemName = InSubsystem;
}

FTestStoreInterface::~FTestStoreInterface()
{
	UE_LOG_ONLINE_STOREV2(Display, TEXT("FTestStoreInterface::~FTestStoreInterface"));
}

void FTestStoreInterface::Test(UWorld* InWorld, const TArray<FString>& InOffersIds)
{
	UE_LOG_ONLINE_STOREV2(Display, TEXT("FTestStoreInterface::Test"));

	OnlineSub = Online::GetSubsystem(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineSub != nullptr)
	{
		if (OnlineSub->GetIdentityInterface().IsValid())
		{
			LocalUserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
		}
		if (LocalUserId.IsValid())
		{
			if (OnlineSub->GetStoreV2Interface().IsValid())
			{
				// list of offers 
				for (const FString& OfferId : InOffersIds)
				{
					RequestOfferIds.Add(FUniqueOfferId(OfferId));
				}				
				// kick off next test
				StartNextTest();
				return;
			}
			else
			{
				UE_LOG_ONLINE_STOREV2(Warning,
					TEXT("Failed to get Store interface for %s"), *SubsystemName);
			}
		}
		else
		{
			UE_LOG_ONLINE_STOREV2(Warning,
				TEXT("No logged in user."));
		}
	}
	else
	{
		UE_LOG_ONLINE_STOREV2(Warning,
			TEXT("Failed to get OSS interface for %s."), *SubsystemName);
	}
	// finish due to failure
	FinishTest();
}

void FTestStoreInterface::StartNextTest()
{
	if (RequestOfferIds.Num() > 0)
	{
		OnlineSub->GetStoreV2Interface()->QueryOffersById(*LocalUserId, RequestOfferIds,
			FOnQueryOnlineStoreOffersComplete::CreateRaw(this, &FTestStoreInterface::OnQueryOnlineStoreOffersComplete));
	}
	else
	{
		OnlineSub->GetStoreV2Interface()->QueryOffersByFilter(*LocalUserId, FOnlineStoreFilter(),
			FOnQueryOnlineStoreOffersComplete::CreateRaw(this, &FTestStoreInterface::OnQueryOnlineStoreOffersComplete));
	}
}

void FTestStoreInterface::OnQueryOnlineStoreOffersComplete(bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& ErrorString)
{
	UE_LOG_ONLINE_STOREV2(Log, TEXT("Query store offers completed with status bWasSuccessful=[%d] Error=[%s] OfferIds=[%d]"), bWasSuccessful, ErrorString.IsEmpty() ? TEXT("None") : *ErrorString, OfferIds.Num());

	if (bWasSuccessful)
	{
		TArray<FOnlineStoreOfferRef> StoreOffers;
		OnlineSub->GetStoreV2Interface()->GetOffers(StoreOffers);

		TArray<FOnlineStoreOfferRef> FoundOffers;
		UE_LOGF(LogOnline, Log, "Found %d offers in cache", StoreOffers.Num());
		for (const FOnlineStoreOfferRef& OfferRef : StoreOffers)
		{
			UE_LOGF(LogOnline, Log, "  Title=[%ls] Desc=[%ls] OfferId=[%ls] CurrencyCode=[%ls] PriceInt=[%lld] DisplayPrice=[%ls]", 
				*OfferRef->Title.ToString(),
				*OfferRef->Description.ToString(),
				*OfferRef->OfferId, 
				*OfferRef->CurrencyCode, 
				OfferRef->NumericPrice, 
				*OfferRef->GetDisplayPrice().ToString()
			);
			if (RequestOfferIds.Contains(OfferRef->OfferId))
			{
				FoundOffers.Add(OfferRef);
			}
		}
		UE_LOGF(LogOnline, Log, "Requested %d offers and found %d offers", RequestOfferIds.Num(), FoundOffers.Num());
	}

	FinishTest();
};

void FTestStoreInterface::FinishTest()
{
	delete this;
}

#endif //WITH_DEV_AUTOMATION_TESTS
