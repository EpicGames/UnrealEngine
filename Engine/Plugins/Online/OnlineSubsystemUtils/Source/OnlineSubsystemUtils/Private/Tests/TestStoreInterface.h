// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE includes


// Module includes
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "Online/CoreOnlineFwd.h"

class IOnlineSubsystem;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the Store interface
 */
class FTestStoreInterface
{

private:

	/** The subsystem that was requested to be tested or the default if empty */
	FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineSubsystem* OnlineSub;
	/** Logged in user */
	FUniqueNetIdPtr LocalUserId;
	/** Store offers to query */
	TArray<FUniqueOfferId> RequestOfferIds;
	
	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/** Callback for store query */
	void OnQueryOnlineStoreOffersComplete(bool bWasSuccessful, const TArray<FUniqueOfferId>& OfferIds, const FString& ErrorString);

public:

	/**
	 *	Constructor which sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestStoreInterface(const FString& InSubsystem);

	/**
	 *	Destructor
	 */
	~FTestStoreInterface();

	/**
	 *	Kicks off all of the testing process
	 *
	 * @param InOffersIds list of product ids to query
	 */
	void Test(class UWorld* InWorld, const TArray<FString>& InOffersIds);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
