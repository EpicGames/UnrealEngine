// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestTimeInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestTimeInterface::Test(UWorld* InWorld)
{
	OnlineTime = Online::GetTimeInterface(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineTime.IsValid())
	{
		// Create and add delegate for the async call
		OnQueryServerUtcTimeCompleteDelegate       = FOnQueryServerUtcTimeCompleteDelegate::CreateRaw(this, &FTestTimeInterface::OnQueryServerUtcTimeComplete);
		OnQueryServerUtcTimeCompleteDelegateHandle = OnlineTime->AddOnQueryServerUtcTimeCompleteDelegate_Handle(OnQueryServerUtcTimeCompleteDelegate);
		// Kick off the async query for server time
		OnlineTime->QueryServerUtcTime();
	}
	else
	{
		UE_LOG_ONLINE(Warning,
			TEXT("Failed to get server time interface for %s"), *SubsystemName);

		delete this;
	}
}

void FTestTimeInterface::OnQueryServerUtcTimeComplete(bool bWasSuccessful, const FString& DateTimeStr, const FString& Error)
{
	OnlineTime->ClearOnQueryServerUtcTimeCompleteDelegate_Handle(OnQueryServerUtcTimeCompleteDelegateHandle);

	if (bWasSuccessful)
	{
		UE_LOG_ONLINE(Log, TEXT("Successful query for server time. Result=[%s]"), *DateTimeStr);
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("Failed to query server time. Error=[%s]"), *Error);
	}

	// done with the test
	delete this;
}

#endif //WITH_DEV_AUTOMATION_TESTS
