// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineTimeInterface.h"

class Error;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the server time interface
 */
 class FTestTimeInterface
 {
	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineTimePtr OnlineTime;
	/** Delegate to use for querying for server time */
	FOnQueryServerUtcTimeCompleteDelegate OnQueryServerUtcTimeCompleteDelegate;
	/** The handle to the above delegate */
	FDelegateHandle OnQueryServerUtcTimeCompleteDelegateHandle;

	/** Hidden on purpose */
	FTestTimeInterface()
		: SubsystemName()
	{
	}

	/**
	 * Called when the time request from the server is complete
	 *
	 * @param bWasSuccessful true if server was contacted and a valid result received
	 * @param DateTimeStr string representing UTC server time (yyyy.MM.dd-HH.mm.ss)
	 * @param Error string representing the error condition
	 */
	void OnQueryServerUtcTimeComplete(bool bWasSuccessful, const FString& DateTimeStr, const FString& Error);

 public:
	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystemName the subsystem to test
	 */
	FTestTimeInterface(const FString& InSubsystemName)
		: SubsystemName(InSubsystemName)
		, OnlineTime(NULL)
	{
	}

	/**
	 * Kicks off all of the testing process
	 */
	void Test(class UWorld* InWorld);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
