// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestTitleFileInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

void FTestTitleFileInterface::Test(UWorld* InWorld)
{
	OnlineTitleFile = Online::GetTitleFileInterface(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineTitleFile.IsValid())
	{
		OnEnumerateFilesCompleteDelegateHandle = OnlineTitleFile->AddOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesCompleteDelegate);
		OnReadFileCompleteDelegateHandle       = OnlineTitleFile->AddOnReadFileCompleteDelegate_Handle      (OnReadFileCompleteDelegate);
		OnlineTitleFile->EnumerateFiles();
	}
	else
	{
		UE_LOG_ONLINE_TITLEFILE(Warning, TEXT("Failed to get title file interface for %s"), *SubsystemName);
	}
}

FTestTitleFileInterface::FTestTitleFileInterface(const FString& InSubsystemName)
	: SubsystemName(InSubsystemName)
	, OnEnumerateFilesCompleteDelegate(FOnEnumerateFilesCompleteDelegate::CreateRaw(this, &FTestTitleFileInterface::OnEnumerateFilesComplete))
	, OnReadFileCompleteDelegate(FOnReadFileCompleteDelegate::CreateRaw(this, &FTestTitleFileInterface::OnReadFileComplete))
	, NumPendingFileReads(0)
{
}

void FTestTitleFileInterface::FinishTest()
{
	if (OnlineTitleFile.IsValid())
	{
		OnlineTitleFile->DeleteCachedFiles(false);

		OnlineTitleFile->ClearOnEnumerateFilesCompleteDelegate_Handle(OnEnumerateFilesCompleteDelegateHandle);
		OnlineTitleFile->ClearOnReadFileCompleteDelegate_Handle      (OnReadFileCompleteDelegateHandle);
	}

	UE_LOG_ONLINE_TITLEFILE(Log, TEXT("Test finished"));

	delete this;
}

void FTestTitleFileInterface::OnEnumerateFilesComplete(bool bSuccess, const FString& ErrorStr)
{
	TArray<FCloudFileHeader> Files;
	OnlineTitleFile->GetFileList(Files);
	UE_LOG_ONLINE_TITLEFILE(Log, TEXT("Found %i files"), Files.Num());

	NumPendingFileReads = Files.Num();
	if (NumPendingFileReads > 0)
	{
		for (TArray<FCloudFileHeader>::TConstIterator It(Files); It; ++It)
		{
			// kick off reads
			const FCloudFileHeader& CloudFile = *It;
			OnlineTitleFile->ReadFile(CloudFile.DLName);
		}
	}
	else
	{
		// no files to read
		FinishTest();
	}
}

void FTestTitleFileInterface::OnReadFileComplete(bool bSuccess, const FString& Filename)
{
	if (bSuccess)
	{
		UE_LOG_ONLINE_TITLEFILE(Log, TEXT("File read. file=[%s]"), *Filename);
		TArray<uint8> FileContents;
		OnlineTitleFile->GetFileContents(Filename, FileContents);
		UE_LOG_ONLINE_TITLEFILE(Log, TEXT("File length=%d. file=[%s]"), FileContents.Num(), *Filename);
		OnlineTitleFile->ClearFile(Filename);
	}
	else
	{
		UE_LOG_ONLINE_TITLEFILE(Log, TEXT("File not read. file=[%s]"), *Filename);
	}
	
	NumPendingFileReads--;
	if (NumPendingFileReads <= 0)
	{
		FinishTest();
	}
}

#endif //WITH_DEV_AUTOMATION_TESTS
