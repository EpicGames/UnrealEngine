// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineTitleFileInterface.h"

#if WITH_DEV_AUTOMATION_TESTS

class FTestTitleFileInterface
{
public:
	void Test(class UWorld* InWorld);

	/**
	 * Sets the subsystem name to test
	 *
	 * @param InSubsystemName the subsystem to test
	 */
	FTestTitleFileInterface(const FString& InSubsystemName);

private:
	void FinishTest();

	/** The subsystem that was requested to be tested or the default if empty */
	const FString SubsystemName;

	/** The online interface to use for testing */
	IOnlineTitleFilePtr OnlineTitleFile;

	/** Title file list enumeration complete delegate */
	FOnEnumerateFilesCompleteDelegate OnEnumerateFilesCompleteDelegate;
	/** Title file download complete delegate */
	FOnReadFileCompleteDelegate OnReadFileCompleteDelegate;
	/** OnEnumerateFilesComplete delegate handle */
	FDelegateHandle OnEnumerateFilesCompleteDelegateHandle;
	/** OnReadFileComplete delegate handle */
	FDelegateHandle OnReadFileCompleteDelegateHandle;

	void OnEnumerateFilesComplete(bool bSuccess, const FString& ErrorStr);

	void OnReadFileComplete(bool bSuccess, const FString& Filename);

	/** Async file reads still pending completion */
	int32 NumPendingFileReads;
};

#endif //WITH_DEV_AUTOMATION_TESTS
