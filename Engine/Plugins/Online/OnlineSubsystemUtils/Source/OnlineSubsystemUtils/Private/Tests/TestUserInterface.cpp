// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tests/TestUserInterface.h"
#include "OnlineSubsystemUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

FTestUserInterface::FTestUserInterface(const FString& InSubsystem)
	: OnlineSub(NULL)
	, bQueryUserInfo(true)
{
	UE_LOG_ONLINE_USER(Display, TEXT("FTestUserInterface::FTestUserInterface"));
	SubsystemName = InSubsystem;
}


FTestUserInterface::~FTestUserInterface()
{
	UE_LOG_ONLINE_USER(Display, TEXT("FTestUserInterface::~FTestUserInterface"));
}

void FTestUserInterface::Test(UWorld* InWorld, const TArray<FString>& InUserIds)
{
	UE_LOG_ONLINE_USER(Display, TEXT("FTestUserInterface::Test"));

	OnlineSub = Online::GetSubsystem(InWorld, SubsystemName.Len() ? FName(*SubsystemName, FNAME_Find) : NAME_None);
	if (OnlineSub != NULL &&
		OnlineSub->GetIdentityInterface().IsValid() &&
		OnlineSub->GetUserInterface().IsValid())
	{
		// Add our delegate for the async call
		OnQueryUserInfoCompleteDelegate       = FOnQueryUserInfoCompleteDelegate::CreateRaw(this, &FTestUserInterface::OnQueryUserInfoComplete);
		OnQueryUserInfoCompleteDelegateHandle = OnlineSub->GetUserInterface()->AddOnQueryUserInfoCompleteDelegate_Handle(0, OnQueryUserInfoCompleteDelegate);

		// list of users to query
		for (int32 Idx=0; Idx < InUserIds.Num(); Idx++)
		{
			FUniqueNetIdPtr UserId = OnlineSub->GetIdentityInterface()->CreateUniquePlayerId(InUserIds[Idx]);
			if (UserId.IsValid())
			{
				QueryUserIds.Add(UserId.ToSharedRef());
			}
		}

		// Include our own user so we have at least one UserId to query for
		{
			FUniqueNetIdPtr UserId = OnlineSub->GetIdentityInterface()->GetUniquePlayerId(0);
			if (UserId.IsValid() && QueryUserIds.Contains(UserId.ToSharedRef()) == false)
			{
				QueryUserIds.Add(UserId.ToSharedRef());
			}
		}

		// kick off next test
		StartNextTest();
	}
	else
	{
		UE_LOG_ONLINE_USER(Warning,
			TEXT("Failed to get user interface for %s"), *SubsystemName);
		
		FinishTest();
	}
}

void FTestUserInterface::StartNextTest()
{
	if (bQueryUserInfo)
	{
		OnlineSub->GetUserInterface()->QueryUserInfo(0, QueryUserIds);
	}
	else
	{
		FinishTest();
	}
}

void FTestUserInterface::FinishTest()
{
	if (OnlineSub != NULL &&
		OnlineSub->GetUserInterface().IsValid())
	{
		// Clear delegates for the various async calls
		OnlineSub->GetUserInterface()->ClearOnQueryUserInfoCompleteDelegate_Handle(0, OnQueryUserInfoCompleteDelegateHandle);
	}
	delete this;
}

void FTestUserInterface::OnQueryUserInfoComplete(int32 LocalPlayer, bool bWasSuccessful, const TArray< FUniqueNetIdRef >& UserIds, const FString& ErrorStr)
{
	UE_LOG_ONLINE_USER(Log,
		TEXT("GetUserInterface() for player (%d) was success=%d"), LocalPlayer, bWasSuccessful);

	if (bWasSuccessful)
	{
		for (int32 UserIdx=0; UserIdx < UserIds.Num(); UserIdx++)
		{
			TSharedPtr<FOnlineUser> User = OnlineSub->GetUserInterface()->GetUserInfo(LocalPlayer, *UserIds[UserIdx]);
			if (User.IsValid())
			{
				UE_LOG_ONLINE_USER(Log,
					TEXT("PlayerId=%s found"), *UserIds[UserIdx]->ToDebugString());
				UE_LOG_ONLINE_USER(Log,
					TEXT("	DisplayName=%s"), *User->GetDisplayName());
				UE_LOG_ONLINE_USER(Log,
					TEXT("	RealName=%s"), *User->GetRealName());
			}
			else
			{
				UE_LOG_ONLINE_USER(Log,
					TEXT("PlayerId=%s not found"), *UserIds[UserIdx]->ToDebugString());
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_USER(Error, TEXT("GetUserInterface() failure. Error = %s"), *ErrorStr);
	}

	// done
	bQueryUserInfo = false;
	StartNextTest();
}

#endif //WITH_DEV_AUTOMATION_TESTS
