// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UE includes


// Module includes
#include "Interfaces/OnlineUserInterface.h"

class IOnlineSubsystem;

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Class used to test the User interface
 */
class FTestUserInterface
{

private:

	/** The subsystem that was requested to be tested or the default if empty */
	FString SubsystemName;
	/** The online interface to use for testing */
	IOnlineSubsystem* OnlineSub;
	/** Delegate to use for querying user info list */
	FOnQueryUserInfoCompleteDelegate OnQueryUserInfoCompleteDelegate;
	/** OnQueryUserInfoComplete delegate handle */
	FDelegateHandle OnQueryUserInfoCompleteDelegateHandle;

	/** List of User ids to query */
	TArray< FUniqueNetIdRef > QueryUserIds;

	/** true to enable user info query */
	bool bQueryUserInfo;
	/**
	 * Step through the various tests that should be run and initiate the next one
	 */
	void StartNextTest();

	/**
	 * Finish/cleanup the tests
	 */
	void FinishTest();

	/**
	 * See OnlineUserInterface.h
	 */
	void OnQueryUserInfoComplete(int32 LocalPlayer, bool bWasSuccessful, const TArray< FUniqueNetIdRef >& UserIds, const FString& ErrorStr);

public:

	/**
	 *	Constructor which sets the subsystem name to test
	 *
	 * @param InSubsystem the subsystem to test
	 */
	FTestUserInterface(const FString& InSubsystem);

	/**
	 *	Destructor
	 */
	~FTestUserInterface();

	/**
	 *	Kicks off all of the testing process
	 *
	 * @param InUserIds list of user ids to query
	 */
	void Test(class UWorld* InWorld, const TArray<FString>& InUserIds);
 };

#endif //WITH_DEV_AUTOMATION_TESTS
