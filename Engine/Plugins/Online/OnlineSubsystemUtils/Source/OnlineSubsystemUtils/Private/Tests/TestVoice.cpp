// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestVoice.h"
#include "Components/AudioComponent.h"
#include "Interfaces/VoiceCapture.h"
#include "Tests/TestVoiceData.inl"

#include "Interfaces/VoiceCodec.h"
#include "OnlineSubsystemUtils.h"
#include "VoiceModule.h"
#include "Sound/SoundWaveProcedural.h"

#if WITH_DEV_AUTOMATION_TESTS

#if !UE_BUILD_SHIPPING
#define VOICE_BUFFER_CHECK(Buffer, Size) \
	check(Buffer.Num() >= (int32)(Size))
#else
#define VOICE_BUFFER_CHECK(Buffer, Size)
#endif

FTestVoice::FTestVoice() :
	VoiceComp(nullptr),
	SoundStreaming(nullptr),
	VoiceCapture(nullptr),
	VoiceEncoder(nullptr),
	VoiceDecoder(nullptr),
	EncodeHint(UVOIPStatics::GetAudioEncodingHint()),
	InputSampleRate(UVOIPStatics::GetVoiceSampleRate()),
	OutputSampleRate(UVOIPStatics::GetVoiceSampleRate()),
	NumInChannels(UVOIPStatics::GetVoiceNumChannels()),
	NumOutChannels(UVOIPStatics::GetVoiceNumChannels()),
	bLastWasPlaying(false),
	StarvedDataCount(0),
	MaxRawCaptureDataSize(0),
	MaxCompressedDataSize(0),
	MaxUncompressedDataSize(0),
	CurrentUncompressedDataQueueSize(0),
	MaxUncompressedDataQueueSize(0),
	MaxRemainderSize(0),
	LastRemainderSize(0),
	CachedSampleCount(0),
	bUseTestSample(false),
	bZeroInput(false),
	bUseDecompressed(true),
	bZeroOutput(false)
{
}

FTestVoice::~FTestVoice()
{
	Shutdown();
}

void FTestVoice::Test()
{
	Init();
}

bool FTestVoice::Init()
{
	DeviceName = TEXT("Line 1 (Virtual Audio Cable)");
	EncodeHint = EAudioEncodeHint::VoiceEncode_Audio;
	InputSampleRate = 48000;
	OutputSampleRate = 48000;
	NumInChannels = 2;
	NumOutChannels = 2;
	
	InitVoiceCapture();
	InitVoiceEncoder();
	InitVoiceDecoder();
	
	return true;
}

void FTestVoice::InitVoiceCapture()
{
	ensure(!VoiceCapture.IsValid());
	VoiceCapture = FVoiceModule::Get().CreateVoiceCapture(DeviceName, InputSampleRate, NumInChannels);
	if (VoiceCapture.IsValid())
	{
		MaxRawCaptureDataSize = VoiceCapture->GetBufferSize();

		RawCaptureData.Empty(MaxRawCaptureDataSize);
		RawCaptureData.AddUninitialized(MaxRawCaptureDataSize);

		VoiceCapture->Start();
	}
}

void FTestVoice::InitVoiceEncoder()
{
	ensure(!VoiceEncoder.IsValid());
	VoiceEncoder = FVoiceModule::Get().CreateVoiceEncoder(InputSampleRate, NumInChannels, EncodeHint);
	if (VoiceEncoder.IsValid())
	{
		MaxRemainderSize = VOICE_STARTING_REMAINDER_SIZE;
		LastRemainderSize = 0;
		MaxCompressedDataSize = VOICE_MAX_COMPRESSED_BUFFER;

		CompressedData.Empty(MaxCompressedDataSize);
		CompressedData.AddUninitialized(MaxCompressedDataSize);

		Remainder.Empty(MaxRemainderSize);
		Remainder.AddUninitialized(MaxRemainderSize);
	}
}

void FTestVoice::InitVoiceDecoder()
{
	ensure(!VoiceDecoder.IsValid());
	if (VoiceCapture.IsValid())
	{
		VoiceDecoder = FVoiceModule::Get().CreateVoiceDecoder(OutputSampleRate, NumOutChannels);
		if (VoiceDecoder.IsValid())
		{
			// Approx 1 sec worth of data
			MaxUncompressedDataSize = NumOutChannels * OutputSampleRate * sizeof(uint16);

			UncompressedData.Empty(MaxUncompressedDataSize);
			UncompressedData.AddUninitialized(MaxUncompressedDataSize);

			MaxUncompressedDataQueueSize = MaxUncompressedDataSize * 5;
			{
				FScopeLock ScopeLock(&QueueLock);
				UncompressedDataQueue.Empty(MaxUncompressedDataQueueSize);
			}
		}
	}
}

void FTestVoice::Shutdown()
{
	RawCaptureData.Empty();
	CompressedData.Empty();
	UncompressedData.Empty();
	Remainder.Empty();

	{
		FScopeLock ScopeLock(&QueueLock);
		UncompressedDataQueue.Empty();
	}

	CleanupVoice();
	CleanupAudioComponent();
}

void FTestVoice::CleanupVoice()
{
	if (VoiceCapture.IsValid())
	{
		VoiceCapture->Shutdown();
		VoiceCapture = nullptr;
	}

	VoiceEncoder = nullptr;
	VoiceDecoder = nullptr;
}

void FTestVoice::CleanupAudioComponent()
{
	if (VoiceComp != nullptr)
	{
		VoiceComp->Stop();

		SoundStreaming->OnSoundWaveProceduralUnderflow.Unbind();
		SoundStreaming = nullptr;

		VoiceComp->RemoveFromRoot();
		VoiceComp = nullptr;

		bLastWasPlaying = false;
	}
}

void FTestVoice::CleanupQueue()
{
	FScopeLock ScopeLock(&QueueLock);
	UncompressedDataQueue.Reset();
}

void FTestVoice::SetStaticVoiceData(TArray<uint8>& VoiceData, uint32& TotalVoiceBytes)
{
	static bool bTimeToQueue = true;
	static double LastQueueTime = 0;
	double CurrentTime = FPlatformTime::Seconds();

	if (LastQueueTime > 0 && CurrentTime - LastQueueTime > 2)
	{
		bTimeToQueue = true;
	}

	if (bTimeToQueue)
	{
		TotalVoiceBytes = UE_ARRAY_COUNT(RawVoiceTestData);

		VoiceData.Empty(TotalVoiceBytes);
		VoiceData.AddUninitialized(TotalVoiceBytes);

		VOICE_BUFFER_CHECK(VoiceData, UE_ARRAY_COUNT(RawVoiceTestData));
		FMemory::Memcpy(VoiceData.GetData(), RawVoiceTestData, UE_ARRAY_COUNT(RawVoiceTestData));

		LastQueueTime = CurrentTime;
		bTimeToQueue = false;
	}
}

void FTestVoice::GenerateData(USoundWaveProcedural* InProceduralWave, int32 SamplesRequired)
{
	const int32 SampleSize = sizeof(uint16) * NumOutChannels;

	{
		FScopeLock ScopeLock(&QueueLock);
		CurrentUncompressedDataQueueSize = UncompressedDataQueue.Num();
		const int32 AvailableSamples = CurrentUncompressedDataQueueSize / SampleSize;
		if (AvailableSamples >= SamplesRequired)
		{
			InProceduralWave->QueueAudio(UncompressedDataQueue.GetData(), AvailableSamples * SampleSize);
			UncompressedDataQueue.RemoveAt(0, AvailableSamples * SampleSize, EAllowShrinking::No);
			CurrentUncompressedDataQueueSize -= (AvailableSamples * SampleSize);
		}
	}
}

bool FTestVoice::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTestVoice_Tick);

	if (VoiceCapture.IsValid())
	{
		if (!IsRunningDedicatedServer() && VoiceComp == nullptr)
		{
			VoiceComp = CreateVoiceAudioComponent(OutputSampleRate, NumOutChannels);
			VoiceComp->AddToRoot();
			SoundStreaming = CastChecked<USoundWaveProcedural>(VoiceComp->Sound);
			if (SoundStreaming)
			{
				// Bind the GenerateData callback with FOnSoundWaveProceduralUnderflow object
				SoundStreaming->OnSoundWaveProceduralUnderflow = FOnSoundWaveProceduralUnderflow::CreateRaw(this, &FTestVoice::GenerateData);
			}
		}

		if (VoiceComp)
		{
			check(SoundStreaming);

			bool bIsPlaying = VoiceComp->IsPlaying();
			if (bIsPlaying != bLastWasPlaying)
			{
				UE_LOGF(LogVoice, Log, "VOIP audio component %ls playing!", bIsPlaying ? TEXT("is") : TEXT("is not"));
				bLastWasPlaying = bIsPlaying;
			}

			StarvedDataCount = (!bIsPlaying || (SoundStreaming->GetAvailableAudioByteCount() != 0)) ? 0 : (StarvedDataCount + 1);
			if (StarvedDataCount > 1)
			{
				UE_LOGF(LogVoice, Log, "VOIP audio component starved %d frames!", StarvedDataCount);
			}

			bool bDoWork = false;
			uint32 TotalVoiceBytes = 0;

			if (bUseTestSample)
			{
				SetStaticVoiceData(RawCaptureData, TotalVoiceBytes);
				bDoWork = true;
			}
			else
			{
				uint32 NewVoiceDataBytes = 0;
				EVoiceCaptureState::Type MicState = VoiceCapture->GetCaptureState(NewVoiceDataBytes);
				if (MicState == EVoiceCaptureState::Ok && NewVoiceDataBytes > 0)
				{
					//UE_LOGF(LogVoice, Log, "Getting data! %d", NewVoiceDataBytes);
					if (LastRemainderSize > 0)
					{
						// Add back any data from the previous frame
						VOICE_BUFFER_CHECK(RawCaptureData, LastRemainderSize);
						FMemory::Memcpy(RawCaptureData.GetData(), Remainder.GetData(), LastRemainderSize);
					}

					// Add new data at the beginning of the last frame
					uint64 SampleCount;
					MicState = VoiceCapture->GetVoiceData(RawCaptureData.GetData() + LastRemainderSize, NewVoiceDataBytes, NewVoiceDataBytes, SampleCount);
					TotalVoiceBytes = NewVoiceDataBytes + LastRemainderSize;

					// Check to make sure this buffer has a valid, chronological buffer count.
					if (SampleCount <= CachedSampleCount)
					{
						UE_LOGF(LogVoice, Log, "Out of order or ambiguous sample count detected! This sample count: %llu Previous sample count: %llu", SampleCount, CachedSampleCount);
					}

					CachedSampleCount = SampleCount;

					VOICE_BUFFER_CHECK(RawCaptureData, TotalVoiceBytes);
					bDoWork = (MicState == EVoiceCaptureState::Ok);
				}
			}

			if (bDoWork && TotalVoiceBytes > 0)
			{
				// ZERO INPUT
				if (bZeroInput)
				{
					FMemory::Memzero(RawCaptureData.GetData(), TotalVoiceBytes);
				}	
				// ZERO INPUT END

				// COMPRESSION BEGIN
				uint32 CompressedDataSize = 0;
				if (VoiceEncoder.IsValid())
				{
					CompressedDataSize = MaxCompressedDataSize;
					LastRemainderSize = VoiceEncoder->Encode(RawCaptureData.GetData(), TotalVoiceBytes, CompressedData.GetData(), CompressedDataSize);
					VOICE_BUFFER_CHECK(RawCaptureData, CompressedDataSize);

					if (LastRemainderSize > 0)
					{
						if (LastRemainderSize > MaxRemainderSize)
						{
							UE_LOGF(LogVoice, Verbose, "Overflow!");
							Remainder.AddUninitialized(LastRemainderSize - MaxRemainderSize);
							MaxRemainderSize = Remainder.Num();
						}

						VOICE_BUFFER_CHECK(Remainder, LastRemainderSize);
						FMemory::Memcpy(Remainder.GetData(), RawCaptureData.GetData() + (TotalVoiceBytes - LastRemainderSize), LastRemainderSize);
					}
				}
				// COMPRESSION END

				// DECOMPRESSION BEGIN
				uint32 UncompressedDataSize = 0;
				if (VoiceDecoder.IsValid() && CompressedDataSize > 0)
				{
					UncompressedDataSize = MaxUncompressedDataSize;
					VoiceDecoder->Decode(CompressedData.GetData(), CompressedDataSize,
						UncompressedData.GetData(), UncompressedDataSize);
					VOICE_BUFFER_CHECK(UncompressedData, UncompressedDataSize);
				}
				// DECOMPRESSION END

				const uint8* VoiceDataPtr = nullptr;
				uint32 VoiceDataSize = 0;

				if (bUseDecompressed)
				{
					if (UncompressedDataSize > 0)
					{
						//UE_LOGF(LogVoice, Log, "Queuing uncompressed data! %d", UncompressedDataSize);
						if (bZeroOutput)
						{
							FMemory::Memzero((uint8*)UncompressedData.GetData(), UncompressedDataSize);
						}

						VoiceDataSize = UncompressedDataSize;
						VoiceDataPtr = UncompressedData.GetData();
					}
				}
				else
				{
					//UE_LOGF(LogVoice, Log, "Queuing original data! %d", UncompressedDataSize);
					VoiceDataPtr = RawCaptureData.GetData();
					VoiceDataSize = (TotalVoiceBytes - LastRemainderSize);
				}

				if (VoiceDataPtr && VoiceDataSize > 0)
				{
					FScopeLock ScopeLock(&QueueLock);

					const int32 OldSize = UncompressedDataQueue.Num();
					const int32 AmountToBuffer = (OldSize + (int32)VoiceDataSize);
					if (AmountToBuffer <= MaxUncompressedDataQueueSize)
					{
						UncompressedDataQueue.AddUninitialized(VoiceDataSize);

						VOICE_BUFFER_CHECK(UncompressedDataQueue, AmountToBuffer);
						FMemory::Memcpy(UncompressedDataQueue.GetData() + OldSize, VoiceDataPtr, VoiceDataSize);
						CurrentUncompressedDataQueueSize += VoiceDataSize;
					}
					else
					{
						UE_LOGF(LogVoice, Warning, "UncompressedDataQueue Overflow!");
					}
				}

				// Wait for approx 1 sec worth of data before playing
				if (!bIsPlaying && (CurrentUncompressedDataQueueSize > (MaxUncompressedDataSize / 2)))
				{
					UE_LOGF(LogVoice, Log, "Playback started");
					VoiceComp->Play();
				}
			}
		}
	}

	return true;
}

bool FTestVoice::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("killtestvoice")))
	{
		delete this;
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcstart")))
	{
		if (VoiceCapture.IsValid() && !VoiceCapture->IsCapturing())
		{
			VoiceCapture->Start();
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcstop")))
	{
		if (VoiceCapture.IsValid() && VoiceCapture->IsCapturing())
		{
			VoiceCapture->Stop();
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vchint")))
	{
		// vchint <0/1>
		FString NewHintStr = FParse::Token(Cmd, false);
		if (!NewHintStr.IsEmpty())
		{
			EAudioEncodeHint NewHint = (EAudioEncodeHint)FPlatformString::Atoi(*NewHintStr);
			if (NewHint >= EAudioEncodeHint::VoiceEncode_Voice && NewHint <= EAudioEncodeHint::VoiceEncode_Audio)
			{
				if (EncodeHint != NewHint)
				{
					EncodeHint = NewHint;

					CleanupAudioComponent();
					CleanupQueue();
					
					VoiceEncoder = nullptr;
					InitVoiceEncoder();

					VoiceDecoder->Reset();
				}
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcdevice")))
	{
		// vcsample <device name>
		FString NewDeviceName = FParse::Token(Cmd, false);
		if (!NewDeviceName.IsEmpty())
		{
			bool bQuotesRemoved = false;
			NewDeviceName = NewDeviceName.TrimQuotes(&bQuotesRemoved);
			if (VoiceCapture.IsValid())
			{
				if (VoiceCapture->ChangeDevice(NewDeviceName, InputSampleRate, NumInChannels))
				{
					DeviceName = NewDeviceName;
					CleanupAudioComponent();
					CleanupQueue();
					VoiceEncoder->Reset();
					VoiceDecoder->Reset();
					VoiceCapture->Start();
				}
				else
				{
					UE_LOGF(LogVoice, Warning, "Failed to change device name %ls", *DeviceName);
				}
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcin")))
	{
		// vcin <rate> <channels>
		FString SampleRateStr = FParse::Token(Cmd, false);
		int32 NewInSampleRate = !SampleRateStr.IsEmpty() ? FPlatformString::Atoi(*SampleRateStr) : InputSampleRate;

		FString NumChannelsStr = FParse::Token(Cmd, false);
		int32 NewNumInChannels = !NumChannelsStr.IsEmpty() ? FPlatformString::Atoi(*NumChannelsStr) : NumInChannels;

		if (NewInSampleRate != InputSampleRate ||
			NewNumInChannels != NumInChannels)
		{
			InputSampleRate = NewInSampleRate;
			NumInChannels = NewNumInChannels;

			if (VoiceCapture.IsValid())
			{
				VoiceCapture->Shutdown();
				VoiceCapture = nullptr;
			}

			InitVoiceCapture();

			VoiceEncoder = nullptr;
			InitVoiceEncoder();
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcout")))
	{
		// vcout <rate> <channels>
		FString SampleRateStr = FParse::Token(Cmd, false);
		int32 NewOutSampleRate = !SampleRateStr.IsEmpty() ? FPlatformString::Atoi(*SampleRateStr) : OutputSampleRate;

		FString NumChannelsStr = FParse::Token(Cmd, false);
		int32 NewNumOutChannels = !NumChannelsStr.IsEmpty() ? FPlatformString::Atoi(*NumChannelsStr) : NumOutChannels;

		if (NewOutSampleRate != OutputSampleRate ||
			NewNumOutChannels != NumOutChannels)
		{
			OutputSampleRate = NewOutSampleRate;
			NumOutChannels = NewNumOutChannels;

			VoiceDecoder = nullptr;
			InitVoiceDecoder();

			CleanupAudioComponent();
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcvbr")))
	{
		// vcvbr <true/false>
		FString VBRStr = FParse::Token(Cmd, false);
		int32 ShouldVBR = FPlatformString::Atoi(*VBRStr);
		bool bVBR = ShouldVBR != 0;
		if (VoiceEncoder.IsValid())
		{
			if (!VoiceEncoder->SetVBR(bVBR))
			{
				UE_LOGF(LogVoice, Warning, "Failed to set VBR %d", bVBR);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcbitrate")))
	{
		// vcbitrate <bitrate>
		FString BitrateStr = FParse::Token(Cmd, false);
		int32 NewBitrate = !BitrateStr.IsEmpty() ? FPlatformString::Atoi(*BitrateStr) : 0;
		if (VoiceEncoder.IsValid() && NewBitrate > 0)
		{
			if (!VoiceEncoder->SetBitrate(NewBitrate))
			{
				UE_LOGF(LogVoice, Warning, "Failed to set bitrate %d", NewBitrate);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vccomplexity")))
	{
		// vccomplexity <complexity>
		FString ComplexityStr = FParse::Token(Cmd, false);
		int32 NewComplexity = !ComplexityStr.IsEmpty() ? FPlatformString::Atoi(*ComplexityStr) : -1;
		if (VoiceEncoder.IsValid() && NewComplexity >= 0)
		{
			if (!VoiceEncoder->SetComplexity(NewComplexity))
			{
				UE_LOGF(LogVoice, Warning, "Failed to set complexity %d", NewComplexity);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcdecompress")))
	{
		// vcdecompress <0/1>
		FString DecompressStr = FParse::Token(Cmd, false);
		int32 ShouldDecompress = FPlatformString::Atoi(*DecompressStr);
		bUseDecompressed = (ShouldDecompress != 0);

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcdump")))
	{
		if (VoiceCapture.IsValid())
		{
			VoiceCapture->DumpState();
		}

		if (VoiceEncoder.IsValid())
		{
			VoiceEncoder->DumpState();
		}

		if (VoiceDecoder.IsValid())
		{
			VoiceDecoder->DumpState();
		}

		bWasHandled = true;
	}

	return bWasHandled;
}

#endif //WITH_DEV_AUTOMATION_TESTS
