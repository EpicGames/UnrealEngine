// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMisc.h"
#include "Containers/Ticker.h"

class UAudioComponent;
class USoundWaveProcedural;
class IVoiceCapture;
class IVoiceEncoder;
class IVoiceDecoder;
enum class EAudioEncodeHint : uint8;

#if WITH_DEV_AUTOMATION_TESTS

#define VOICE_MAX_COMPRESSED_BUFFER 20 * 1024
#define VOICE_STARTING_REMAINDER_SIZE 1 * 1024

class FTestVoice : public FTSTickerObjectBase, public FSelfRegisteringExec
{
public:

	FTestVoice();
	virtual ~FTestVoice();

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

	void Test();

protected:
	// FSelfRegisteringExec
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:

	/** Reference to audio component playing active streaming sound */
	UAudioComponent* VoiceComp;
	/** Internal reference to audio component's streaming wave object */
	USoundWaveProcedural* SoundStreaming;

	/** Active audio capture */
	TSharedPtr<IVoiceCapture> VoiceCapture;
	/** Active audio encoder */
	TSharedPtr<IVoiceEncoder> VoiceEncoder;
	/** Active audio decoder */
	TSharedPtr<IVoiceDecoder> VoiceDecoder;

	/** Name of current device under capture */
	FString DeviceName;
	/** Current type of audio under capture */
	EAudioEncodeHint EncodeHint;
	/** Current input sample rate provided */
	int32 InputSampleRate;
	/** Desired output sample rate */
	int32 OutputSampleRate;
	/** Current number of channels captured */
	int32 NumInChannels;
	/** Desired number of output channels */
	int32 NumOutChannels;

	/** Was the audio component playing last frame */
	bool bLastWasPlaying;
	/** Number of consecutive frames that the playback has been starved */
	int32 StarvedDataCount;

	/** Buffer for pre-encoded audio data */
	TArray<uint8> RawCaptureData;
	/** Maximum size of a single raw capture packet */
	int32 MaxRawCaptureDataSize;
	/** Buffer for compressed audio data */
	TArray<uint8> CompressedData;
	/** Maximum size of a single encoded packet */
	int32 MaxCompressedDataSize;
	/** Buffer for uncompressed audio data (valid during Tick only) */
	TArray<uint8> UncompressedData;
	/** Maximum size of a single decoded packet */
	int32 MaxUncompressedDataSize;

	/** Buffer for outgoing audio intended for procedural streaming */
	mutable FCriticalSection QueueLock;
	TArray<uint8> UncompressedDataQueue;
	/** Amount of data currently in the outgoing playback queue */
	int32 CurrentUncompressedDataQueueSize;
	/** Maximum size of the outgoing playback queue */
	int32 MaxUncompressedDataQueueSize;

	/** Buffer for data left uncompressed after call to Encode to be used next encoding */
	TArray<uint8> Remainder;
	/** Maximum size of the remainder buffer */
	int32 MaxRemainderSize;
	/** Current amount of raw data left over from last encode */
	int32 LastRemainderSize;
	/** Cached Sample Count to allow us to compare the SampleCount of a call to GetVoiceData against the previous call. */
	uint64 CachedSampleCount;
	/** Use hard coded audio test data in TestVoiceData.h */
	bool bUseTestSample;
	/** Zero out input before encoding */
	bool bZeroInput;
	/** Pass originating audio capture data directly to the audio component (skip Encode/Decode) */
	bool bUseDecompressed;
	/** Zero out output data before playback */
	bool bZeroOutput;

	/** First time initialization of all components necessary for end to end capture -> encode -> decode -> playback */
	bool Init();
	/** (Re)Initialize the audio capture object with current settings, reallocating buffers */
	void InitVoiceCapture();
	/** (Re)Initialize the audio encoder with current settings, reallocating buffers */
	void InitVoiceEncoder();
	/** (Re)Initialize the audio decoder with current settings, reallocating buffers */
	void InitVoiceDecoder();
	/** Cleanup and shutdown the entire object */
	void Shutdown();

	/** Free all audio objects (capture/encode/decode) */
	void CleanupVoice();
	/** Free audio component */
	void CleanupAudioComponent();
	/** Empty and reset the outgoing audio data queue */
	void CleanupQueue();

	/**
	 * Callback from streaming audio when data is requested for playback
	 *
	 * @param InProceduralWave SoundWave requesting more data
	 * @param SamplesRequired number of samples needed for immediate playback
	 */
	void GenerateData(USoundWaveProcedural* InProceduralWave, int32 SamplesRequired);

	/**
	 * Play static audio from TestVoiceData.h in a loop
	 *
	 * @param VoiceData out buffer to fill with data
	 * @param TotalVoiceBytes amount of data added to the buffer
	 */
	void SetStaticVoiceData(TArray<uint8>& VoiceData, uint32& TotalVoiceBytes);
};

#endif //WITH_DEV_AUTOMATION_TESTS
