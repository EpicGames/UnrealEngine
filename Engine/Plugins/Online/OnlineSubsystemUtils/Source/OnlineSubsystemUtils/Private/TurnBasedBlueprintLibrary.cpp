// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnBasedBlueprintLibrary.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "Interfaces/OnlineTurnBasedInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TurnBasedBlueprintLibrary)

UTurnBasedBlueprintLibrary::UTurnBasedBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UTurnBasedBlueprintLibrary::GetIsMyTurn(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, /*out*/ bool& bIsMyTurn)
{
	bIsMyTurn = false;

	FOnlineSubsystemBPCallHelper Helper(TEXT("GetIsMyTurn"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerController);

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr OnlineTurnBasedPtr = Helper.OnlineSub->GetTurnBasedInterface();
		if (OnlineTurnBasedPtr.IsValid())
		{
			FTurnBasedMatchPtr Match = OnlineTurnBasedPtr->GetMatchWithID(MatchID);
			if (Match.IsValid())
			{
				bIsMyTurn = Match->GetCurrentPlayerIndex() == Match->GetLocalPlayerIndex();
			}
			else
			{
				FString Message = FString::Printf(TEXT("Match ID %s not found"), *MatchID);
				FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn Based Matches not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}
}

void UTurnBasedBlueprintLibrary::GetMyPlayerIndex(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, /*out*/ int32& PlayerIndex)
{
	PlayerIndex = -1;

	FOnlineSubsystemBPCallHelper Helper(TEXT("GetMyPlayerIndex"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerController);

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr OnlineTurnBasedPtr = Helper.OnlineSub->GetTurnBasedInterface();
		if (OnlineTurnBasedPtr.IsValid())
		{
			FTurnBasedMatchPtr Match = OnlineTurnBasedPtr->GetMatchWithID(MatchID);
			if (Match.IsValid())
			{
				PlayerIndex = Match->GetLocalPlayerIndex();
			}
			else
			{
				FString Message = FString::Printf(TEXT("Match ID %s not found"), *MatchID);
				FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn Based Matches not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}
}

void UTurnBasedBlueprintLibrary::RegisterTurnBasedMatchInterfaceObject(UObject* WorldContextObject, APlayerController* PlayerController, UObject* Object)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("GetIsMyTurn"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerController);

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr OnlineTurnBasedPtr = Helper.OnlineSub->GetTurnBasedInterface();
		if (OnlineTurnBasedPtr.IsValid())
		{
			OnlineTurnBasedPtr->RegisterTurnBasedMatchInterfaceObject(Object);
		}
	}
}

void UTurnBasedBlueprintLibrary::GetPlayerDisplayName(UObject* WorldContextObject, APlayerController* PlayerController, FString MatchID, int32 PlayerIndex, /*out*/ FString& PlayerDisplayName)
{
	FOnlineSubsystemBPCallHelper Helper(TEXT("GetMyPlayerIndex"), WorldContextObject);
	Helper.QueryIDFromPlayerController(PlayerController);

	if (Helper.IsValid())
	{
		IOnlineTurnBasedPtr OnlineTurnBasedPtr = Helper.OnlineSub->GetTurnBasedInterface();
		if (OnlineTurnBasedPtr.IsValid())
		{
			FTurnBasedMatchPtr Match = OnlineTurnBasedPtr->GetMatchWithID(MatchID);
			if (Match.IsValid())
			{
				if (Match->GetPlayerDisplayName(PlayerIndex, PlayerDisplayName) == false)
				{
					FString Message = FString::Printf(TEXT("Player index %d not within bounds of player array."), PlayerIndex);
					FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
				}
			}
			else
			{
				FString Message = FString::Printf(TEXT("Match ID %s not found"), *MatchID);
				FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Turn Based Matches not supported by Online Subsystem"), ELogVerbosity::Warning);
		}
	}
}

