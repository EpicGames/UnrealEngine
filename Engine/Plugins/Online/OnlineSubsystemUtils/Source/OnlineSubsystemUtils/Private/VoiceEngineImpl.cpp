// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceEngineImpl.h"
#include "Interfaces/VoiceCodec.h"
#include "VoiceModule.h"

#include "OnlineSubsystemUtils.h"
#include "OnlineSubsystemBPCallHelper.h"
#include "VoicePacketBuffer.h"
#include "VoipListenerSynthComponent.h"

/** Largest size allowed to carry over into next buffer */
#define MAX_VOICE_REMAINDER_SIZE 4 * 1024

namespace VoiceEngineUtilities
{
	void DownmixBuffer(const float* InAudio, float* OutAudio, int32 NumFrames, int32 InNumChannels, int32 OutNumChannels)
	{
		const float Attenuation = FMath::Clamp<float>(((float)OutNumChannels) / InNumChannels, 0.0f, 1.0f);
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			for (int32 OutChannelIndex = 0; OutChannelIndex < OutNumChannels; OutChannelIndex++)
			{
				const int32 OutSampleIndex = FrameIndex * OutNumChannels + OutChannelIndex;
				OutAudio[OutSampleIndex] = Attenuation * InAudio[FrameIndex * InNumChannels];

				for (int32 InChannelIndex = 1; InChannelIndex < InNumChannels; InChannelIndex++)
				{
					const int32 InSampleIndex = FrameIndex * InNumChannels + InChannelIndex;
					OutAudio[OutSampleIndex] += Attenuation * InAudio[InSampleIndex];
				}
			}
		}
	}
}

FRemoteTalkerDataImpl::FRemoteTalkerDataImpl() :
	MaxUncompressedDataSize(0),
	MaxUncompressedDataQueueSize(0),
	CurrentUncompressedDataQueueSize(0),
	LastSeen(0.0),
	NumFramesStarved(0),
	VoipSynthComponent(nullptr),
	VoiceDecoder(nullptr),
	MicrophoneAmplitude(0.0f)
{
	int32 SampleRate = UVOIPStatics::GetVoiceSampleRate();
	int32 NumChannels = UVOIPStatics::GetVoiceNumChannels();
	VoiceDecoder = FVoiceModule::Get().CreateVoiceDecoder(SampleRate, NumChannels);
	check(VoiceDecoder.IsValid());

	// Approx 1 sec worth of data for a stereo microphone
	MaxUncompressedDataSize = UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel() * 2;
	MaxUncompressedDataQueueSize = MaxUncompressedDataSize * 5;
	{
		FScopeLock ScopeLock(&QueueLock);
		UncompressedDataQueue.Empty(MaxUncompressedDataQueueSize);
	}
}

FRemoteTalkerDataImpl::FRemoteTalkerDataImpl(const FRemoteTalkerDataImpl& Other)
{
	LastSeen = Other.LastSeen;
	NumFramesStarved = Other.NumFramesStarved;
	VoipSynthComponent = Other.VoipSynthComponent;
	VoiceDecoder = Other.VoiceDecoder;
	MaxUncompressedDataSize = Other.MaxUncompressedDataSize;
	MaxUncompressedDataQueueSize = Other.MaxUncompressedDataQueueSize;
	CurrentUncompressedDataQueueSize = Other.CurrentUncompressedDataQueueSize;
	MicrophoneAmplitude = Other.MicrophoneAmplitude;

	{
		FScopeLock ScopeLock(&Other.QueueLock);
		UncompressedDataQueue = Other.UncompressedDataQueue;
	}
}

FRemoteTalkerDataImpl::FRemoteTalkerDataImpl(FRemoteTalkerDataImpl&& Other)
{
	LastSeen = Other.LastSeen;
	Other.LastSeen = 0.0;

	NumFramesStarved = Other.NumFramesStarved;
	Other.NumFramesStarved = 0;

	VoipSynthComponent = Other.VoipSynthComponent;
	Other.VoipSynthComponent = nullptr;

	VoiceDecoder = MoveTemp(Other.VoiceDecoder);
	Other.VoiceDecoder = nullptr;

	MaxUncompressedDataSize = Other.MaxUncompressedDataSize;
	Other.MaxUncompressedDataSize = 0;

	MaxUncompressedDataQueueSize = Other.MaxUncompressedDataQueueSize;
	Other.MaxUncompressedDataQueueSize = 0;

	CurrentUncompressedDataQueueSize = Other.CurrentUncompressedDataQueueSize;
	Other.CurrentUncompressedDataQueueSize = 0;

	MicrophoneAmplitude = Other.MicrophoneAmplitude;
	Other.MicrophoneAmplitude = 0.0f;

	{
		FScopeLock ScopeLock(&Other.QueueLock);
		UncompressedDataQueue = MoveTemp(Other.UncompressedDataQueue);
	}
}

FRemoteTalkerDataImpl::~FRemoteTalkerDataImpl()
{
	VoiceDecoder = nullptr;

	Reset();
}

void FRemoteTalkerDataImpl::Reset()
{
	// Set to large number so TickTalkers doesn't come in here
	LastSeen = MAX_FLT;
	NumFramesStarved = 0;

	if (UObjectInitialized() && VoipSynthComponent.IsValid())
	{
		VoipSynthComponent->Stop();

		UAudioComponent* AudioComponent = VoipSynthComponent->GetAudioComponent();
		if (AudioComponent && AudioComponent->IsRegistered())
		{
			AudioComponent->UnregisterComponent();
		}

		//If the UVOIPTalker associated with this is still alive, notify it that this player is done talking.
		if (UVOIPStatics::IsVOIPTalkerStillAlive(CachedTalkerPtr))
		{
			CachedTalkerPtr->OnTalkingEnd();
		}

		bIsActive = false;
		
		if (VoipSynthComponent->IsRegistered())
		{
			VoipSynthComponent->UnregisterComponent();
		}

		VoipSynthComponent = nullptr;
	}

	CurrentUncompressedDataQueueSize = 0;
	MicrophoneAmplitude = 0.0f;

	{
		FScopeLock ScopeLock(&QueueLock);
		UncompressedDataQueue.Empty();
	}
}

void FRemoteTalkerDataImpl::Cleanup()
{
	if (VoipSynthComponent.IsValid())
	{
		VoipSynthComponent->Stop();
		bIsActive = false;
	}

	VoipSynthComponent = nullptr;
}

FVoiceEngineImpl ::FVoiceEngineImpl()
	: OnlineInstanceName(NAME_None)
	, VoiceCapture(nullptr)
	, VoiceEncoder(nullptr)
	, OwningUserIndex(INVALID_INDEX)
	, UncompressedBytesAvailable(0)
	, CompressedBytesAvailable(0)
	, AvailableVoiceResult(EVoiceCaptureState::UnInitialized)
	, bPendingFinalCapture(false)
	, bIsCapturing(false)
	, SerializeHelper(nullptr)
{
}

FVoiceEngineImpl::FVoiceEngineImpl(IOnlineSubsystem* InSubsystem) :
	OnlineInstanceName(NAME_None),
	VoiceCapture(nullptr),
	VoiceEncoder(nullptr),
	OwningUserIndex(INVALID_INDEX),
	UncompressedBytesAvailable(0),
	CompressedBytesAvailable(0),
	AvailableVoiceResult(EVoiceCaptureState::UnInitialized),
	bPendingFinalCapture(false),
	bIsCapturing(false),
	SerializeHelper(nullptr)
{
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FVoiceEngineImpl::OnPostLoadMap);

	if (InSubsystem)
	{
		OnlineInstanceName = InSubsystem->GetInstanceName();
	}
}

FVoiceEngineImpl::~FVoiceEngineImpl()
{
	if (bIsCapturing)
	{
		VoiceCapture->Stop();
	}

	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	VoiceCapture = nullptr;
	VoiceEncoder = nullptr;

	delete SerializeHelper;

}

void FVoiceEngineImpl::VoiceCaptureUpdate() const
{
	if (bPendingFinalCapture && VoiceCapture.IsValid())
	{
		uint32 CompressedSize;
		const EVoiceCaptureState::Type RecordingState = VoiceCapture->GetCaptureState(CompressedSize);

		// If no data is available, we have finished capture the last (post-StopRecording) half-second of voice data
		if (RecordingState == EVoiceCaptureState::NotCapturing)
		{
			UE_LOG_ONLINE_VOICEENGINE(Log, TEXT("Internal voice capture complete."));

			bPendingFinalCapture = false;

			// If a new recording session has begun since the call to 'StopRecording', kick that off
			if (bIsCapturing)
			{
				StartRecording();
			}
			else
			{
				// Marks that recording has successfully stopped
				StoppedRecording();
			}
		}
	}
}

void FVoiceEngineImpl::StartRecording() const
{
	UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("VOIP StartRecording"));
	if (VoiceCapture.IsValid())
	{
		if (!VoiceCapture->Start())
		{
			UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("Failed to start voice recording"));
		}
	}
}

void FVoiceEngineImpl::StopRecording() const
{
	UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("VOIP StopRecording"));
	if (VoiceCapture.IsValid())
	{
		VoiceCapture->Stop();
	}
}

void FVoiceEngineImpl::StoppedRecording() const
{
	UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("VOIP StoppedRecording"));
}

bool FVoiceEngineImpl::Init(int32 MaxLocalTalkers, int32 MaxRemoteTalkers)
{
	bool bSuccess = false;

	IOnlineSubsystem* OnlineSub = GetOnlineSubSystem();

	if (OnlineSub && !OnlineSub->IsDedicated())
	{
		FVoiceModule& VoiceModule = FVoiceModule::Get();
		if (VoiceModule.IsVoiceEnabled())
		{
			VoiceEncoder = VoiceModule.CreateVoiceEncoder();

			bSuccess = VoiceEncoder.IsValid();
			if (bSuccess)
			{
				CompressedVoiceBuffer.Empty(UVOIPStatics::GetMaxCompressedVoiceDataSize());
				DecompressedVoiceBuffer.Empty(UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel());

				for (int32 TalkerIdx = 0; TalkerIdx < MaxLocalTalkers; TalkerIdx++)
				{
					PlayerVoiceData[TalkerIdx].VoiceRemainderSize = 0;
					PlayerVoiceData[TalkerIdx].VoiceRemainder.Empty(MAX_VOICE_REMAINDER_SIZE);
				}
			}
			else
			{
				UE_LOGF(LogVoice, Warning, "Voice capture initialization failed!");
			}
		}
		else
		{
			UE_LOGF(LogVoice, Log, "Voice module disabled by config [Voice].bEnabled");
		}
	}

	return bSuccess;
}

uint32 FVoiceEngineImpl::StartLocalVoiceProcessing(uint32 LocalUserNum) 
{
	uint32 Return = ONLINE_FAIL;
	if (IsOwningUser(LocalUserNum))
	{
		if (!bIsCapturing)
		{
			// Update the current recording state, if VOIP data was still being read
			VoiceCaptureUpdate();

			if (!IsRecording())
			{
				StartRecording();
			}

			bIsCapturing = true;
		}

		Return = ONLINE_SUCCESS;
	}

	return Return;
}

uint32 FVoiceEngineImpl::StopLocalVoiceProcessing(uint32 LocalUserNum) 
{
	uint32 Return = ONLINE_FAIL;
	if (IsOwningUser(LocalUserNum))
	{
		if (bIsCapturing)
		{
			bIsCapturing = false;
			bPendingFinalCapture = true;

			// Make a call to begin stopping the current VOIP recording session
			StopRecording();

			// Now check/update the status of the recording session
			VoiceCaptureUpdate();
		}

		Return = ONLINE_SUCCESS;
	}

	return Return;
}

uint32 FVoiceEngineImpl::RegisterLocalTalker(uint32 LocalUserNum)
{
	if (!VoiceCapture.IsValid())
	{
		VoiceCapture = FVoiceModule::Get().CreateVoiceCapture("");

		if (!VoiceCapture.IsValid())
		{
			UE_LOG_ONLINE_VOICEENGINE(Error, TEXT("RegisterLocalTalker: Failed to create a Voice Capture Device"));
			return ONLINE_FAIL;
		}
	}

	if (OwningUserIndex == INVALID_INDEX)
	{
		OwningUserIndex = LocalUserNum;
		return ONLINE_SUCCESS;
	}

	return ONLINE_FAIL;
}

uint32 FVoiceEngineImpl::UnregisterLocalTalker(uint32 LocalUserNum)
{
	if (IsOwningUser(LocalUserNum))
	{
		OwningUserIndex = INVALID_INDEX;
		return ONLINE_SUCCESS;
	}

	return ONLINE_FAIL;
}

uint32 FVoiceEngineImpl::UnregisterRemoteTalker(const FUniqueNetId& UniqueId)
{
	FRemoteTalkerDataImpl* RemoteData = RemoteTalkerBuffers.Find(FUniqueNetIdWrapper(UniqueId.AsShared()));
	if (RemoteData != nullptr)
	{
		// Dump the whole talker
		RemoteData->Cleanup();
		RemoteTalkerBuffers.Remove(FUniqueNetIdWrapper(UniqueId.AsShared()));
		VoiceAmplitudes.Remove(FUniqueNetIdWrapper(UniqueId.AsShared()));
	}

	return ONLINE_SUCCESS;
}

uint32 FVoiceEngineImpl::GetVoiceDataReadyFlags() const
{
	// First check and update the internal state of VOIP recording
	VoiceCaptureUpdate();
	if (OwningUserIndex != INVALID_INDEX && IsRecording())
	{
		// Check if there is new data available via the Voice API
		if (AvailableVoiceResult == EVoiceCaptureState::Ok && UncompressedBytesAvailable > 0)
		{
			return 1 << OwningUserIndex;
		}
	}

	return 0;
}

uint32 FVoiceEngineImpl::ReadLocalVoiceData(uint32 LocalUserNum, uint8* Data, uint32* Size, uint64* OutSampleCount)
{
	check(*Size > 0);
	
	// Before doing anything, check/update the current recording state
	VoiceCaptureUpdate();

	// Return data even if not capturing, possibly have data during stopping
	if (IsOwningUser(LocalUserNum) && IsRecording())
	{
		DecompressedVoiceBuffer.Empty(UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel());
		CompressedVoiceBuffer.Empty(UVOIPStatics::GetMaxCompressedVoiceDataSize());

		uint32 NewVoiceDataBytes = 0;
		EVoiceCaptureState::Type VoiceResult = VoiceCapture->GetCaptureState(NewVoiceDataBytes);
		if (VoiceResult != EVoiceCaptureState::Ok && VoiceResult != EVoiceCaptureState::NoData)
		{
			UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("ReadLocalVoiceData: GetAvailableVoice failure: VoiceResult: %s"), EVoiceCaptureState::ToString(VoiceResult));
			return ONLINE_FAIL;
		}

		if (NewVoiceDataBytes == 0)
		{
			UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("ReadLocalVoiceData: No Data: VoiceResult: %s"), EVoiceCaptureState::ToString(VoiceResult));
			*Size = 0;
			return ONLINE_SUCCESS;
		}

		// Make space for new and any previously remaining data

		// Add the number of new bytes (since last time this function was called) and the number of bytes remaining that wasn't consumed last time this was called
		// This is how many bytes we would like to return
		uint32 TotalVoiceBytes = NewVoiceDataBytes + PlayerVoiceData[LocalUserNum].VoiceRemainderSize;

		// But we have a max amount we can return so clamp it to that max value if we're asking for more bytes than we're allowed
		if (TotalVoiceBytes > UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel())
		{
			UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("Exceeded uncompressed voice buffer size, clamping"))
			TotalVoiceBytes = UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel();
		}

		DecompressedVoiceBuffer.AddUninitialized(TotalVoiceBytes);

		// If there's still audio left from a previous ReadLocalData call that didn't get output, copy that first into the decompressed voice buffer
		if (PlayerVoiceData[LocalUserNum].VoiceRemainderSize > 0)
		{
			FMemory::Memcpy(DecompressedVoiceBuffer.GetData(), PlayerVoiceData[LocalUserNum].VoiceRemainder.GetData(), PlayerVoiceData[LocalUserNum].VoiceRemainderSize);
		}

		// Get new uncompressed data
		uint8* const RemainingDecompressedBufferPtr = DecompressedVoiceBuffer.GetData() + PlayerVoiceData[LocalUserNum].VoiceRemainderSize;
		const uint32 RemainingDecompressedBufferSize = DecompressedVoiceBuffer.Num() - PlayerVoiceData[LocalUserNum].VoiceRemainderSize;
		uint32 ByteWritten = 0;
		uint64 NewSampleCount = 0;
		VoiceResult = VoiceCapture->GetVoiceData(RemainingDecompressedBufferPtr, RemainingDecompressedBufferSize, ByteWritten, NewSampleCount);
		
		TotalVoiceBytes = ByteWritten + PlayerVoiceData[LocalUserNum].VoiceRemainderSize;

		if ((VoiceResult == EVoiceCaptureState::Ok || VoiceResult == EVoiceCaptureState::NoData) && TotalVoiceBytes > 0)
		{
			if (OutSampleCount != nullptr)
			{
				*OutSampleCount = NewSampleCount;
			}

			// Prepare the encoded buffer (e.g. opus)
			CompressedBytesAvailable = UVOIPStatics::GetMaxCompressedVoiceDataSize();
			CompressedVoiceBuffer.AddUninitialized(UVOIPStatics::GetMaxCompressedVoiceDataSize());

			check(((uint32) CompressedVoiceBuffer.Num()) <= UVOIPStatics::GetMaxCompressedVoiceDataSize());

			// Run the uncompressed audio through the opus decoder, note that it may not encode all data, which results in some remaining data
			PlayerVoiceData[LocalUserNum].VoiceRemainderSize =
				VoiceEncoder->Encode(DecompressedVoiceBuffer.GetData(), TotalVoiceBytes, CompressedVoiceBuffer.GetData(), CompressedBytesAvailable);

			// Save off any unencoded remainder
			if (PlayerVoiceData[LocalUserNum].VoiceRemainderSize > 0)
			{
				if (PlayerVoiceData[LocalUserNum].VoiceRemainderSize > MAX_VOICE_REMAINDER_SIZE)
				{
					UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("Exceeded voice remainder buffer size, clamping"));
					PlayerVoiceData[LocalUserNum].VoiceRemainderSize = MAX_VOICE_REMAINDER_SIZE;
				}

				PlayerVoiceData[LocalUserNum].VoiceRemainder.AddUninitialized(MAX_VOICE_REMAINDER_SIZE);
				FMemory::Memcpy(PlayerVoiceData[LocalUserNum].VoiceRemainder.GetData(), DecompressedVoiceBuffer.GetData() + (TotalVoiceBytes - PlayerVoiceData[LocalUserNum].VoiceRemainderSize), PlayerVoiceData[LocalUserNum].VoiceRemainderSize);
			}

			static double LastGetVoiceCallTime = 0.0;
			double CurTime = FPlatformTime::Seconds();
			double TimeSinceLastCall = (LastGetVoiceCallTime > 0) ? (CurTime - LastGetVoiceCallTime) : 0.0;
			LastGetVoiceCallTime = CurTime;

			UE_LOG_ONLINE_VOICEENGINE(Log, TEXT("ReadLocalVoiceData: GetVoice: Result: %s, Available: %i, LastCall: %0.3f ms"), EVoiceCaptureState::ToString(VoiceResult), CompressedBytesAvailable, TimeSinceLastCall * 1000.0);
			if (CompressedBytesAvailable > 0)
			{
				*Size = FMath::Min<int32>(*Size, CompressedBytesAvailable);
				FMemory::Memcpy(Data, CompressedVoiceBuffer.GetData(), *Size);

				UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("ReadLocalVoiceData: Size: %d"), *Size);
				return ONLINE_SUCCESS;
			}
			else
			{
				*Size = 0;
				CompressedVoiceBuffer.Empty(UVOIPStatics::GetMaxCompressedVoiceDataSize());

				UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("ReadLocalVoiceData: GetVoice failure: VoiceResult: %s"), EVoiceCaptureState::ToString(VoiceResult));
				return ONLINE_FAIL;
			}
		}
	}

	return ONLINE_FAIL;
}

uint32 FVoiceEngineImpl::SubmitRemoteVoiceData(const FUniqueNetIdWrapper& RemoteTalkerId, uint8* Data, uint32* Size, uint64& InSampleCount)
{
	UE_LOG_ONLINE_VOICEENGINE(VeryVerbose, TEXT("SubmitRemoteVoiceData(%s) Size: %d received!"), *RemoteTalkerId.ToDebugString(), *Size);
	
	FRemoteTalkerDataImpl& QueuedData = RemoteTalkerBuffers.FindOrAdd(RemoteTalkerId);

	// new voice packet.
	QueuedData.LastSeen = FPlatformTime::Seconds();

	uint32 BytesWritten = UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel();

	DecompressedVoiceBuffer.Empty(UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel());
	DecompressedVoiceBuffer.AddUninitialized(UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel());
	QueuedData.VoiceDecoder->Decode(Data, *Size, DecompressedVoiceBuffer.GetData(), BytesWritten);

	// If there is no data, return
	if (BytesWritten <= 0)
	{
		*Size = 0;
		return ONLINE_SUCCESS;
	}

	bool bAudioComponentCreated = false;
	// Generate a streaming wave audio component for voice playback
	if (!QueuedData.VoipSynthComponent.IsValid())
	{
		CreateSerializeHelper();
		
		if (QueuedData.VoipSynthComponent.IsValid())
		{
			QueuedData.VoipSynthComponent->Stop();
			QueuedData.VoipSynthComponent->ClosePacketStream();
		}

		if (GetOnlineSubSystem())
		{
			if (UWorld* World = GetWorldForOnline(GetOnlineSubSystem()->GetInstanceName()))
			{
				QueuedData.VoipSynthComponent = CreateVoiceSynthComponent(World, UVOIPStatics::GetVoiceSampleRate());
			}
		}

		if (QueuedData.VoipSynthComponent.IsValid())
		{
			//TODO, make buffer size and buffering delay runtime-controllable parameters.
			QueuedData.bIsActive = false;
			QueuedData.VoipSynthComponent->OpenPacketStream(InSampleCount, UVOIPStatics::GetNumBufferedPackets(), UVOIPStatics::GetBufferingDelay());
			QueuedData.bIsEnvelopeBound = false;
			QueuedData.VoipSynthComponent->ConnectToSplitter(AllRemoteTalkerAudio);
		}
	}

	if (QueuedData.VoipSynthComponent.IsValid())
	{
		if (!QueuedData.bIsActive)
		{
			QueuedData.bIsActive = true;
			FVoiceSettings InSettings;
			UVOIPTalker* OwningTalker = nullptr;

			OwningTalker = UVOIPStatics::GetVOIPTalkerForPlayer(RemoteTalkerId, InSettings);

			GetVoiceSettingsOverride(RemoteTalkerId, InSettings);

			ApplyVoiceSettings(QueuedData.VoipSynthComponent.Get(), InSettings);

			QueuedData.VoipSynthComponent->ResetBuffer(int32(InSampleCount), UVOIPStatics::GetBufferingDelay());
			QueuedData.VoipSynthComponent->Start();
			QueuedData.CachedTalkerPtr = OwningTalker;

			if (OwningTalker)
			{
				if (!QueuedData.bIsEnvelopeBound)
				{
					QueuedData.VoipSynthComponent->OnAudioEnvelopeValueNative.AddUObject(OwningTalker, &UVOIPTalker::OnAudioComponentEnvelopeValue);
					QueuedData.bIsEnvelopeBound = true;
				}

				OwningTalker->OnTalkingBegin(QueuedData.VoipSynthComponent->GetAudioComponent());
			}
		}

		QueuedData.VoipSynthComponent->SubmitPacket((float*)DecompressedVoiceBuffer.GetData(), BytesWritten, InSampleCount, EVoipStreamDataFormat::Int16);

		// Try to start the VoipSynthComponent if it has been killed by the audio engine.
		if (!QueuedData.VoipSynthComponent->IsPlaying())
		{
			QueuedData.VoipSynthComponent->Start();
		}
	}

	return ONLINE_SUCCESS;
}

void FVoiceEngineImpl::TickTalkers(float DeltaTime)
{
	// Remove users that are done talking.
	const double CurTime = FPlatformTime::Seconds();
	for (FRemoteTalkerData::TIterator It(RemoteTalkerBuffers); It; ++It)
	{
		FRemoteTalkerDataImpl& RemoteData = It.Value();
		double TimeSince = CurTime - RemoteData.LastSeen;

		if (RemoteData.VoipSynthComponent.IsValid() && RemoteData.VoipSynthComponent->IsIdling() && RemoteData.bIsActive)
		{
			RemoteData.Reset();
		}
		else if (TimeSince >= UVOIPStatics::GetRemoteTalkerTimeoutDuration())
		{
			// Dump the whole talker
			RemoteData.Reset();
		}
	}
}

void FVoiceEngineImpl::Tick(float DeltaTime)
{
	// Check available voice once a frame, this value changes after calling GetVoiceData()
	if (VoiceCapture.IsValid())
	{
		AvailableVoiceResult = VoiceCapture->GetCaptureState(UncompressedBytesAvailable);
	}

	TickTalkers(DeltaTime);

	// Push any buffered audio to any connected outputs.
	AllRemoteTalkerAudio.ProcessAudio();
}

void FVoiceEngineImpl::GenerateVoiceData(USoundWaveProcedural* InProceduralWave, int32 SamplesRequired, const FUniqueNetId& TalkerId)
{
	FRemoteTalkerDataImpl* QueuedData = RemoteTalkerBuffers.Find(FUniqueNetIdWrapper(TalkerId.AsShared()));
	if (QueuedData)
	{
		const int32 SampleSize = sizeof(uint16) * UVOIPStatics::GetVoiceNumChannels();

		{
			FScopeLock ScopeLock(&QueuedData->QueueLock);
			QueuedData->CurrentUncompressedDataQueueSize = QueuedData->UncompressedDataQueue.Num();
			const int32 AvailableSamples = QueuedData->CurrentUncompressedDataQueueSize / SampleSize;
			if (AvailableSamples >= SamplesRequired)
			{
				UE_LOG_ONLINE_VOICEENGINE(Verbose, TEXT("GenerateVoiceData %d / %d"), AvailableSamples, SamplesRequired);
				const int32 SamplesBytesTaken = AvailableSamples * SampleSize;
				InProceduralWave->QueueAudio(QueuedData->UncompressedDataQueue.GetData(), SamplesBytesTaken);
				QueuedData->UncompressedDataQueue.RemoveAt(0, SamplesBytesTaken, EAllowShrinking::No);
				QueuedData->CurrentUncompressedDataQueueSize -= (SamplesBytesTaken);
			}
			else
			{
				UE_LOG_ONLINE_VOICEENGINE(Verbose, TEXT("Voice underflow"));
			}
		}
	}
}

void FVoiceEngineImpl::OnAudioFinished()
{
	for (FRemoteTalkerData::TIterator It(RemoteTalkerBuffers); It; ++It)
	{
		FRemoteTalkerDataImpl& RemoteData = It.Value();
		if (RemoteData.VoipSynthComponent.IsValid() && RemoteData.VoipSynthComponent->IsIdling())
		{
			UE_LOG_ONLINE_VOICEENGINE(Log, TEXT("Removing VOIP AudioComponent for Id: %s"), *It.Key().ToDebugString());
			RemoteData.VoipSynthComponent->Stop();
			RemoteData.bIsActive = false;
			break;
		}
	}
	UE_LOG_ONLINE_VOICEENGINE(Verbose, TEXT("Audio Finished"));
}

void FVoiceEngineImpl::OnPostLoadMap(UWorld*)
{
	for (FRemoteTalkerData::TIterator It(RemoteTalkerBuffers); It; ++It)
	{
		FRemoteTalkerDataImpl& RemoteData = It.Value();
		RemoteData.Reset();
	}
}

FString FVoiceEngineImpl::GetVoiceDebugState() const
{
	FString Output;
	Output = FString::Printf(TEXT("IsRecording: %d\n DataReady: 0x%08x State:%s\n UncompressedBytes: %d\n CompressedBytes: %d\n"),
		IsRecording(), 
		GetVoiceDataReadyFlags(),
		EVoiceCaptureState::ToString(AvailableVoiceResult),
		UncompressedBytesAvailable,
		CompressedBytesAvailable
		);

	// Add remainder size
	for (int32 Idx=0; Idx < MAX_SPLITSCREEN_TALKERS; Idx++)
	{
		Output += FString::Printf(TEXT("Remainder[%d] %d\n"), Idx, PlayerVoiceData[Idx].VoiceRemainderSize);
	}

	return Output;
}

bool FVoiceEngineImpl::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("vcvbr")))
	{
		// vcvbr <true/false>
		FString VBRStr = FParse::Token(Cmd, false);
		int32 ShouldVBR = FPlatformString::Atoi(*VBRStr);
		bool bVBR = ShouldVBR != 0;
		if (VoiceEncoder.IsValid())
		{
			if (!VoiceEncoder->SetVBR(bVBR))
			{
				UE_LOGF(LogVoice, Warning, "Failed to set VBR %d", bVBR);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcbitrate")))
	{
		// vcbitrate <bitrate>
		FString BitrateStr = FParse::Token(Cmd, false);
		int32 NewBitrate = !BitrateStr.IsEmpty() ? FPlatformString::Atoi(*BitrateStr) : 0;
		if (VoiceEncoder.IsValid() && NewBitrate > 0)
		{
			if (!VoiceEncoder->SetBitrate(NewBitrate))
			{
				UE_LOGF(LogVoice, Warning, "Failed to set bitrate %d", NewBitrate);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vccomplexity")))
	{
		// vccomplexity <complexity>
		FString ComplexityStr = FParse::Token(Cmd, false);
		int32 NewComplexity = !ComplexityStr.IsEmpty() ? FPlatformString::Atoi(*ComplexityStr) : -1;
		if (VoiceEncoder.IsValid() && NewComplexity >= 0)
		{
			if (!VoiceEncoder->SetComplexity(NewComplexity))
			{
				UE_LOGF(LogVoice, Warning, "Failed to set complexity %d", NewComplexity);
			}
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("vcdump")))
	{
		if (VoiceCapture.IsValid())
		{
			VoiceCapture->DumpState();
		}

		if (VoiceEncoder.IsValid())
		{
			VoiceEncoder->DumpState();
		}

		for (FRemoteTalkerData::TIterator It(RemoteTalkerBuffers); It; ++It)
		{
			FRemoteTalkerDataImpl& RemoteData = It.Value();
			if (RemoteData.VoiceDecoder.IsValid())
			{
				RemoteData.VoiceDecoder->DumpState();
			}
		}

		bWasHandled = true;
	}

	return bWasHandled;
}

Audio::FPatchOutputStrongPtr FVoiceEngineImpl::GetMicrophoneOutput()
{
	 // NOTE: We don't mix down multiple microphones here.
	if (VoiceCapture.IsValid())
	{
		return VoiceCapture->GetMicrophoneAudio(4096 * 2, 1.0f);
	}
	else
	{
		return nullptr;
	}
}

Audio::FPatchOutputStrongPtr FVoiceEngineImpl::GetRemoteTalkerOutput()
{
	return AllRemoteTalkerAudio.AddNewOutput(4096 * 2, 1.0f);
}

float FVoiceEngineImpl::GetMicrophoneAmplitude(int32 LocalUserNum)
{
	if (VoiceCapture.IsValid())
	{
		return VoiceCapture->GetCurrentAmplitude();
	}
	else
	{
		return 0.0f;
	}
}

float FVoiceEngineImpl::GetIncomingAudioAmplitude(const FUniqueNetIdWrapper& RemoteUserId)
{
	FVoiceAmplitudeData* VoiceAmplitude = VoiceAmplitudes.Find(RemoteUserId);

	if (VoiceAmplitude != nullptr)
	{
		// Timeout and default to 0 if we haven't received data recently
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - VoiceAmplitude->LastSeen > UVOIPStatics::GetRemoteTalkerTimeoutDuration())
		{
			return 0.0f;
		}

		return VoiceAmplitude->Amplitude;
	}
	else
	{
		return -1.0f;
	}
}

uint32 FVoiceEngineImpl::SetRemoteVoiceAmplitude(const FUniqueNetIdWrapper& RemoteTalkerId, float InAmplitude)
{
	FVoiceAmplitudeData& VoiceAmplitude = VoiceAmplitudes.FindOrAdd(RemoteTalkerId);

	VoiceAmplitude.Amplitude = InAmplitude;
	VoiceAmplitude.LastSeen = FPlatformTime::Seconds();

	return 0;
}

bool FVoiceEngineImpl::PatchRemoteTalkerOutputToEndpoint(const FString& InDeviceName, bool bMuteInGameOutput /*= true*/)
{
	if (bMuteInGameOutput)
	{
		static IConsoleVariable* MuteAudioEngineOutputCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MuteAudioEngineOutput"));
		check(MuteAudioEngineOutputCVar);
		MuteAudioEngineOutputCVar->Set(1, ECVF_SetByGameSetting);
	}
	
	TUniquePtr<FVoiceEndpoint>& Endpoint = ExternalEndpoints.Emplace_GetRef(new FVoiceEndpoint(InDeviceName, float(UVOIPStatics::GetVoiceSampleRate()), UVOIPStatics::GetVoiceNumChannels()));
	Audio::FPatchOutputStrongPtr OutputPatch = AllRemoteTalkerAudio.AddNewOutput(4096 * 2, 1.0f);
	Endpoint->PatchInOutput(OutputPatch);
	return true;
}

bool FVoiceEngineImpl::PatchLocalTalkerOutputToEndpoint(const FString& InDeviceName)
{
	// Local talker patched output is always mixed down to mono.
	TUniquePtr<FVoiceEndpoint>& Endpoint = ExternalEndpoints.Emplace_GetRef(new FVoiceEndpoint(InDeviceName, float(UVOIPStatics::GetVoiceSampleRate()), 1));
	Audio::FPatchOutputStrongPtr OutputPatch = VoiceCapture->GetMicrophoneAudio(4096 * 2, 1.0f);
	Endpoint->PatchInOutput(OutputPatch);
	return true;
}

void FVoiceEngineImpl::DisconnectAllEndpoints()
{
	static IConsoleVariable* MuteAudioEngineOutputCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MuteAudioEngineOutput"));
	check(MuteAudioEngineOutputCVar);
	MuteAudioEngineOutputCVar->Set(0, ECVF_SetByGameSetting);

	ExternalEndpoints.Reset();
}

int32 FVoiceEngineImpl::GetMaxVoiceRemainderSize()
{
	return MAX_VOICE_REMAINDER_SIZE;
}

void FVoiceEngineImpl::CreateSerializeHelper()
{
	if (SerializeHelper == nullptr)
	{
		SerializeHelper = new FVoiceSerializeHelper(this);
	}
}

IOnlineSubsystem* FVoiceEngineImpl::GetOnlineSubSystem()
{
	if (UWorld* World = GetWorldForOnline(OnlineInstanceName))
	{
		return Online::GetSubsystem(World);
	}

	return nullptr;
}

FVoiceEndpoint::FVoiceEndpoint(const FString& InEndpointName, float InSampleRate, int32 InNumChannels)
	: NumChannelsComingIn(InNumChannels)
{
	check(GEngine && GEngine->GetAudioDeviceManager());

	IAudioDeviceModule* AudioModule = GEngine->GetAudioDeviceManager()->GetAudioDeviceModule();
	check(AudioModule);

	PlatformEndpoint.Reset(AudioModule->CreateAudioMixerPlatformInterface());

	if (PlatformEndpoint.IsValid())
	{
		bool Result = PlatformEndpoint->InitializeHardware();

		check(Result);

		int32 DeviceIndex = PlatformEndpoint->GetIndexForDevice(InEndpointName);

		if (DeviceIndex == INDEX_NONE)
		{
			UE_LOGF(LogVoice, Warning, "Failed to find device %ls, using default output device.", *InEndpointName);
			DeviceIndex = AUDIO_MIXER_DEFAULT_DEVICE_INDEX;
		}

		int32 NumFrames = PlatformEndpoint->GetNumFrames(1024);

		OpenParams.NumBuffers = 3;
		OpenParams.NumFrames = NumFrames;
		OpenParams.OutputDeviceIndex = DeviceIndex;
		OpenParams.SampleRate = uint32(InSampleRate);
		OpenParams.AudioMixer = this;
		OpenParams.MaxSources = 0;

		Result = PlatformEndpoint->OpenAudioStream(OpenParams);

		check(Result);

		PlatformEndpoint->PostInitializeHardware();

		PlatformDeviceInfo = PlatformEndpoint->GetPlatformDeviceInfo();

		PlatformEndpoint->StartAudioStream();
		PlatformEndpoint->FadeIn();
	}
}

FVoiceEndpoint::~FVoiceEndpoint()
{
	if (PlatformEndpoint.IsValid())
	{
		PlatformEndpoint->StopAudioStream();
		PlatformEndpoint->CloseAudioStream();
	}
}

void FVoiceEndpoint::PatchInOutput(Audio::FPatchOutputStrongPtr& InOutput)
{
	FScopeLock ScopeLock(&OutputPatchCriticalSection);
	OutputPatch = InOutput;
}

bool FVoiceEndpoint::OnProcessAudioStream(Audio::FAlignedFloatBuffer& OutputBuffer)
{
	FScopeLock ScopeLock(&OutputPatchCriticalSection);

	int32 NumFrames = OutputBuffer.Num() / PlatformDeviceInfo.NumChannels;

	if (OutputPatch.IsValid() && OutputPatch->GetNumSamplesAvailable() >= NumFrames * NumChannelsComingIn)
	{
		if (PlatformDeviceInfo.NumChannels != NumChannelsComingIn)
		{
			DownmixBuffer.Reset();
			DownmixBuffer.AddZeroed(NumFrames * NumChannelsComingIn);

			OutputPatch->PopAudio(DownmixBuffer.GetData(), NumFrames * NumChannelsComingIn, false);

			VoiceEngineUtilities::DownmixBuffer(DownmixBuffer.GetData(), OutputBuffer.GetData(), NumFrames, NumChannelsComingIn, PlatformDeviceInfo.NumChannels);
		}
		else
		{
			OutputPatch->PopAudio(OutputBuffer.GetData(), OutputBuffer.Num(), false);
		}
	}
	else
	{
		FMemory::Memzero(OutputBuffer.GetData(), OutputBuffer.Num() * sizeof(float));
	}

	return true;
}

void FVoiceEndpoint::OnAudioStreamShutdown()
{
	// Nothing to do here.
}
