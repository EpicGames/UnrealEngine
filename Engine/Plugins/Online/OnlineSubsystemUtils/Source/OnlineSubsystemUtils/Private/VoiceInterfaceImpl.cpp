// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceInterfaceImpl.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "OnlineSubsystemImpl.h"
#include "VoiceEngineImpl.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineSessionInterface.h"

FOnlineVoiceImpl::FOnlineVoiceImpl(IOnlineSubsystem* InOnlineSubsystem) :
	OnlineSubsystem(InOnlineSubsystem),
	VoiceEngine(NULL)
{
}

FOnlineVoiceImpl::~FOnlineVoiceImpl()
{
	LocalTalkers.Empty();
	RemoteTalkers.Empty();
	VoiceEngine = NULL;
}

bool FOnlineVoiceImpl::Init()
{
	bool bSuccess = false;

	if (!GConfig->GetInt(TEXT("OnlineSubsystem"),TEXT("MaxLocalTalkers"), MaxLocalTalkers, GEngineIni))
	{
		MaxLocalTalkers = MAX_SPLITSCREEN_TALKERS;
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing MaxLocalTalkers key in OnlineSubsystem of DefaultEngine.ini"));
	}
	if (!GConfig->GetInt(TEXT("OnlineSubsystem"),TEXT("MaxRemoteTalkers"), MaxRemoteTalkers, GEngineIni))
	{
		MaxRemoteTalkers = MAX_REMOTE_TALKERS;
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing MaxRemoteTalkers key in OnlineSubsystem of DefaultEngine.ini"));
	}
	if (!GConfig->GetFloat(TEXT("OnlineSubsystem"),TEXT("VoiceNotificationDelta"), VoiceNotificationDelta, GEngineIni))
	{
		VoiceNotificationDelta = 0.2f;
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing VoiceNotificationDelta key in OnlineSubsystem of DefaultEngine.ini"));
	}

	bool bHasVoiceEnabled = false;
	if (GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("bHasVoiceEnabled"), bHasVoiceEnabled, GEngineIni) && bHasVoiceEnabled)
	{
		if (OnlineSubsystem)
		{
			SessionInt = OnlineSubsystem->GetSessionInterface().Get();
			IdentityInt = OnlineSubsystem->GetIdentityInterface().Get();
			bSuccess = SessionInt && IdentityInt;
		}

		if (bSuccess)
		{
			const bool bVoiceEngineForceDisable = OnlineSubsystem->IsDedicated() || GIsBuildMachine;
			if (!bVoiceEngineForceDisable)
			{
				VoiceEngine = CreateVoiceEngine();
				bSuccess = VoiceEngine->Init(MaxLocalTalkers, MaxRemoteTalkers);
			}
			else
			{
				MaxLocalTalkers = 0;
				MaxRemoteTalkers = 0;
			}
		}

		LocalTalkers.Init(FLocalTalker(), MaxLocalTalkers);
		RemoteTalkers.Empty(MaxRemoteTalkers);

		if (!bSuccess)
		{
			// Not necessary to log here since VoiceEngine::Init() will report its own failure
			//UE_LOG_ONLINE_VOICE(Warning, TEXT("Failed to initialize voice interface"));

			LocalTalkers.Empty();
			RemoteTalkers.Empty();
			VoiceEngine = nullptr;
		}
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Voice interface disabled by config [OnlineSubsystem].bHasVoiceEnabled"));
	}

	return bSuccess;
}

void FOnlineVoiceImpl::Shutdown()
{
	VoiceData.RemotePackets.Empty();

	LocalTalkers.Empty();
	RemoteTalkers.Empty();

	VoiceEngine = nullptr;
	SessionInt = nullptr;
	IdentityInt = nullptr;
}

void FOnlineVoiceImpl::ClearVoicePackets()
{
	for (uint32 Index = 0; Index < MAX_SPLITSCREEN_TALKERS; Index++)
	{
		// Mark the local packet as processed
		VoiceData.LocalPackets[Index].ResetData();
	}
}

void FOnlineVoiceImpl::Tick(float DeltaTime)
{
	if (!OnlineSubsystem->IsDedicated())
	{
		// If we aren't in a networked match, no need to update networked voice
		if (SessionInt && SessionInt->GetNumSessions() > 0)
		{
			// Processing voice data only valid with a voice engine to capture/play
			if (VoiceEngine.IsValid())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineVoiceImpl_Tick);

				VoiceEngine->Tick(DeltaTime);

				// Queue local packets for sending via the network
				ProcessLocalVoicePackets();
				// Submit queued packets to audio system
				ProcessRemoteVoicePackets();
				// Fire off any talking notifications for hud display
				ProcessTalkingDelegates(DeltaTime);
			}
		}
	}
}

void FOnlineVoiceImpl::StartNetworkedVoice(uint8 LocalUserNum)
{
	// Validate the range of the entry
	if (LocalUserNum >= 0 && LocalUserNum < MaxLocalTalkers)
	{
		LocalTalkers[LocalUserNum].bHasNetworkedVoice = true;
		if (VoiceEngine.IsValid())
		{
			uint32 Return = VoiceEngine->StartLocalVoiceProcessing(LocalUserNum);
			UE_LOG_ONLINE_VOICE(Log, TEXT("StartLocalProcessing(%d) returned 0x%08X"), LocalUserNum, Return);
		}
		UE_LOG_ONLINE_VOICE(Log, TEXT("Starting networked voice for user: %d"), LocalUserNum);
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Invalid user specified in StartNetworkedVoice(%d)"),
			(uint32)LocalUserNum);
	}
}

void FOnlineVoiceImpl::StopNetworkedVoice(uint8 LocalUserNum)
{
	// Validate the range of the entry
	if (LocalUserNum >= 0 && LocalUserNum < MaxLocalTalkers)
	{
		if (VoiceEngine.IsValid())
		{
			uint32 Return = VoiceEngine->StopLocalVoiceProcessing(LocalUserNum);
			UE_LOG_ONLINE_VOICE(Log, TEXT("StopLocalVoiceProcessing(%d) returned 0x%08X"), LocalUserNum, Return);
		}
		LocalTalkers[LocalUserNum].bHasNetworkedVoice = false;
		UE_LOG_ONLINE_VOICE(Log, TEXT("Stopping networked voice for user: %d"), LocalUserNum);
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Invalid user specified in StopNetworkedVoice(%d)"),
			(uint32)LocalUserNum);
	}
}

bool FOnlineVoiceImpl::RegisterLocalTalker(uint32 LocalUserNum)
{
	uint32 Return = ONLINE_FAIL;
	if (LocalUserNum >= 0 && LocalUserNum < (uint32)MaxLocalTalkers)
	{
		// Get at the local talker's cached data
		FLocalTalker& Talker = LocalTalkers[LocalUserNum];
		// Make local user capable of sending voice data
		StartNetworkedVoice(uint8(LocalUserNum));
		// Don't register talkers when voice is disabled
		if (VoiceEngine.IsValid())
		{
			if (Talker.bIsRegistered == false)
			{
				// Register the talker locally
				Return = VoiceEngine->RegisterLocalTalker(LocalUserNum);
				UE_LOG_ONLINE_VOICE(Log, TEXT("RegisterLocalTalker(%d) returned 0x%08X"),	LocalUserNum, Return);
				if (Return == ONLINE_SUCCESS)
				{
					Talker.bIsRegistered = true;
					#if (0)
					{	
						// If enabled, voice capture is continuous and "push to talk" merely sends packets
						// Kick off the processing mode
						Return = VoiceEngine->StartLocalVoiceProcessing(LocalUserNum);
						UE_LOG_ONLINE_VOICE(Log, TEXT("StartLocalProcessing(%d) returned 0x%08X"), LocalUserNum, Return);
					}
					#endif
				}
			}
			else
			{
				// Just say yes, we registered fine
				Return = ONLINE_SUCCESS;
			}
			
			// @todo ONLINE - update mute list?
		}
		else
		{
			// Not properly logged in, so skip voice for them
			Talker.bIsRegistered = false;
		}
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Invalid user specified in RegisterLocalTalker(%d)"), LocalUserNum);
	}
	return Return == ONLINE_SUCCESS;
}

void FOnlineVoiceImpl::RegisterLocalTalkers()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Registering all local talkers"));
	// Loop through the 4 available players and register them
	for (uint32 Index = 0; Index < (uint32)MaxLocalTalkers ; Index++)
	{
		// Register the local player as a local talker
		RegisterLocalTalker(Index);
	}
}

bool FOnlineVoiceImpl::UnregisterLocalTalker(uint32 LocalUserNum)
{
	uint32 Return = ONLINE_SUCCESS;
	if (LocalUserNum >= 0 && LocalUserNum < (uint32)MaxLocalTalkers)
	{
		// Get at the local talker's cached data
		FLocalTalker& Talker = LocalTalkers[LocalUserNum];
		// Skip the unregistration if not registered
		if (Talker.bIsRegistered == true &&
			// Or when voice is disabled
			VoiceEngine.IsValid())
		{
			if (OnPlayerTalkingStateChangedDelegates.IsBound() && (Talker.bIsTalking || Talker.bWasTalking))
			{
				FUniqueNetIdPtr UniqueId = IdentityInt->GetUniquePlayerId(LocalUserNum);
				if (UniqueId.IsValid())
				{
					OnPlayerTalkingStateChangedDelegates.Broadcast(UniqueId.ToSharedRef(), false);
				}
				else
				{
					UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid UserId for local player %d in UnregisterLocalTalker"), LocalUserNum);
				}
			}

			// Remove them from engine too
			Return = VoiceEngine->StopLocalVoiceProcessing(LocalUserNum);
			UE_LOG_ONLINE_VOICE(Log, TEXT("StopLocalVoiceProcessing(%d) returned 0x%08X"), LocalUserNum, Return);
			Return = VoiceEngine->UnregisterLocalTalker(LocalUserNum);
			UE_LOG_ONLINE_VOICE(Log, TEXT("UnregisterLocalTalker(%d) returned 0x%08X"), LocalUserNum, Return);
			Talker.bIsTalking = false;
			Talker.bWasTalking = false;
			Talker.bIsRegistered = false;
		}
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Invalid user specified in UnregisterLocalTalker(%d)"), LocalUserNum);
	}
	return Return == ONLINE_SUCCESS;
}

void FOnlineVoiceImpl::UnregisterLocalTalkers()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Unregistering all local talkers"));
	// Loop through the 4 available players and unregister them
	for (uint32 Index = 0; Index < (uint32)MaxLocalTalkers; Index++)
	{
		// Unregister the local player as a local talker
		UnregisterLocalTalker(Index);
	}
}

bool FOnlineVoiceImpl::RegisterRemoteTalker(const FUniqueNetId& UniqueId)
{
	uint32 Return = ONLINE_FAIL;
	if (OnlineSubsystem)
	{
		// Skip this if the session isn't active
		if (SessionInt && SessionInt->GetNumSessions() > 0 &&
			// Or when voice is disabled
			VoiceEngine.IsValid())
		{
			// See if this talker has already been registered or not
			FRemoteTalker* Talker = FindRemoteTalker(UniqueId);
			if (Talker == NULL)
			{
				// Add a new talker to our list
				int32 AddIndex = RemoteTalkers.AddZeroed();
				Talker = &RemoteTalkers[AddIndex];
				// Copy the UniqueId
				Talker->TalkerId = UniqueId.AsShared();

				// Register the remote talker locally
				Return = VoiceEngine->RegisterRemoteTalker(UniqueId);
				UE_LOG_ONLINE_VOICE(Log, TEXT("RegisterRemoteTalker(%s) returned 0x%08X"),
					*UniqueId.ToDebugString(), Return);
			}
			else
			{
				UE_LOG_ONLINE_VOICE(Verbose, TEXT("Remote talker %s is being re-registered"), *UniqueId.ToDebugString());
				Return = ONLINE_SUCCESS;
			}
			
			// Update muting all of the local talkers with this remote talker
			ProcessMuteChangeNotification();
			// Now start processing the remote voices
			Return = VoiceEngine->StartRemoteVoiceProcessing(UniqueId);
			UE_LOG_ONLINE_VOICE(Log, TEXT("StartRemoteVoiceProcessing(%s) returned 0x%08X"), *UniqueId.ToDebugString(), Return);
		}
	}
	return Return == ONLINE_SUCCESS;
}

bool FOnlineVoiceImpl::UnregisterRemoteTalker(const FUniqueNetId& UniqueId)
{
	uint32 Return = ONLINE_FAIL;
	if (OnlineSubsystem)
	{
		// Skip this if the session isn't active
		if (SessionInt && SessionInt->GetNumSessions() > 0 &&
			// Or when voice is disabled
			VoiceEngine.IsValid())
		{
			// Make sure the talker is valid
			if (FindRemoteTalker(UniqueId) != NULL)
			{
				// Find them in the talkers array and remove them
				for (int32 Index = 0; Index < RemoteTalkers.Num(); Index++)
				{
					const FRemoteTalker& Talker = RemoteTalkers[Index];
					// Is this the remote talker?
					if (*Talker.TalkerId == UniqueId)
					{
						// Going to remove the talker, so if they were talking recently make sure to indicate they've stopped
						if (OnPlayerTalkingStateChangedDelegates.IsBound() && (Talker.bIsTalking || Talker.bWasTalking))
						{
							OnPlayerTalkingStateChangedDelegates.Broadcast(Talker.TalkerId.ToSharedRef(), false);
						}

						RemoteTalkers.RemoveAtSwap(Index);
						break;
					}
				}
				// Remove them from voice engine
				Return = VoiceEngine->UnregisterRemoteTalker(UniqueId);
				UE_LOG_ONLINE_VOICE(Log, TEXT("UnregisterRemoteTalker(%s) returned 0x%08X"), *UniqueId.ToDebugString(), Return);
			}
			else
			{
				UE_LOG_ONLINE_VOICE(Verbose, TEXT("Unknown remote talker (%s) specified to UnregisterRemoteTalker()"), *UniqueId.ToDebugString());
			}
		}
	}
	return Return == ONLINE_SUCCESS;
}

void FOnlineVoiceImpl::RemoveAllRemoteTalkers()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Removing all remote talkers"));
	if (VoiceEngine.IsValid())
	{
		// Work backwards through array removing the talkers
		for (int32 Index = RemoteTalkers.Num() - 1; Index >= 0; Index--)
		{
			const FRemoteTalker& Talker = RemoteTalkers[Index];

			if (OnPlayerTalkingStateChangedDelegates.IsBound() && (Talker.bIsTalking || Talker.bWasTalking))
			{
				OnPlayerTalkingStateChangedDelegates.Broadcast(Talker.TalkerId.ToSharedRef(), false);
			}

			uint32 Return = VoiceEngine->UnregisterRemoteTalker(*Talker.TalkerId);
			UE_LOG_ONLINE_VOICE(Log, TEXT("UnregisterRemoteTalker(%s) returned 0x%08X"), *Talker.TalkerId->ToDebugString(), Return);
		}
	}

	// Empty the array now that they are all unregistered
	RemoteTalkers.Empty(MaxRemoteTalkers);
}

FRemoteTalker* FOnlineVoiceImpl::FindRemoteTalker(const FUniqueNetId& UniqueId)
{
	for (int32 Index = 0; Index < RemoteTalkers.Num(); Index++)
	{
		FRemoteTalker& Talker = RemoteTalkers[Index];
		// Compare net ids to see if they match
		if (*Talker.TalkerId == UniqueId)
		{
			return &RemoteTalkers[Index];
		}
	}
	return NULL;
}

bool FOnlineVoiceImpl::IsHeadsetPresent(uint32 LocalUserNum)
{
	return VoiceEngine.IsValid() && VoiceEngine->IsHeadsetPresent(LocalUserNum);
}

bool FOnlineVoiceImpl::IsLocalPlayerTalking(uint32 LocalUserNum)
{
	return VoiceEngine.IsValid() && VoiceEngine->IsLocalPlayerTalking(LocalUserNum);
}

bool FOnlineVoiceImpl::IsRemotePlayerTalking(const FUniqueNetId& UniqueId)
{
	return VoiceEngine.IsValid() && VoiceEngine->IsRemotePlayerTalking(UniqueId);
}

bool FOnlineVoiceImpl::IsMuted(uint32 LocalUserNum, const FUniqueNetId& UniqueId) const
{
	if (LocalUserNum >= 0 && LocalUserNum < (uint32)MaxLocalTalkers)
	{
		return IsLocallyMuted(UniqueId);
	}

	return false;
}

bool FOnlineVoiceImpl::IsLocallyMuted(const FUniqueNetId& UniqueId) const
{
	int32 Index = MuteList.Find(FUniqueNetIdWrapper(UniqueId.AsShared()));
	return Index != INDEX_NONE;
}

bool FOnlineVoiceImpl::IsSystemWideMuted(const FUniqueNetId& UniqueId) const
{
	int32 Index = SystemMuteList.Find(FUniqueNetIdWrapper(UniqueId.AsShared()));
	return Index != INDEX_NONE;
}

bool FOnlineVoiceImpl::MuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide)
{
	uint32 Return = ONLINE_FAIL;
	if (LocalUserNum >= 0 && LocalUserNum < MaxLocalTalkers )
	{
		if (bIsSystemWide)
		{
			// Add them to the system wide mute list
			SystemMuteList.AddUnique(FUniqueNetIdWrapper(PlayerId.AsShared()));
			// Should update MuteList after going up to the server and coming back down
			ProcessMuteChangeNotification();
		}
		else
		{
			// Skip this if the session isn't active
			if (SessionInt && SessionInt->GetNumSessions() > 0 &&
				// Or if voice is disabled
					VoiceEngine.IsValid())
			{
				// Find the specified talker
				FRemoteTalker* Talker = FindRemoteTalker(PlayerId);
				if (Talker != NULL)
				{
					MuteList.AddUnique(FUniqueNetIdWrapper(PlayerId.AsShared()));
					Return = ONLINE_SUCCESS;
					UE_LOG_ONLINE_VOICE(Log, TEXT("Muting remote talker (%s)"), *PlayerId.ToDebugString());
				}
				else
				{
					UE_LOG_ONLINE_VOICE(Verbose, TEXT("Unknown remote talker (%s) specified to MuteRemoteTalker()"), *PlayerId.ToDebugString());
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid user specified in MuteRemoteTalker(%d)"), LocalUserNum);
	}

	return Return == ONLINE_SUCCESS;
}

bool FOnlineVoiceImpl::UnmuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide)
{
	uint32 Return = ONLINE_FAIL;
	if (LocalUserNum >= 0 && LocalUserNum < MaxLocalTalkers )
	{
		if (bIsSystemWide)
		{
			// Remove them from the system wide mute list
			SystemMuteList.RemoveSingleSwap(FUniqueNetIdWrapper(PlayerId.AsShared()));
			// Should update MuteList after going up to the server and coming back down
			ProcessMuteChangeNotification();
		}
		else
		{
			// Skip this if the session isn't active
			if (SessionInt && SessionInt->GetNumSessions() > 0 &&
				// Or if voice is disabled
				VoiceEngine.IsValid())
			{
				// Find the specified talker
				FRemoteTalker* Talker = FindRemoteTalker(PlayerId);
				if (Talker != NULL)
				{
					// Make sure there isn't a system mute
					bool bIsSystemMuted = IsSystemWideMuted(PlayerId);
					if (!bIsSystemMuted)
					{
						// Remove them from the mute list
						MuteList.RemoveSingleSwap(FUniqueNetIdWrapper(PlayerId.AsShared()));
						UE_LOG_ONLINE_VOICE(Log, TEXT("Unmuting remote talker (%s)"), *PlayerId.ToDebugString());
					}
				}
				else
				{
					UE_LOG_ONLINE_VOICE(Verbose, TEXT("Unknown remote talker (%s) specified to UnmuteRemoteTalker()"), *PlayerId.ToDebugString());
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid user specified in UnmuteRemoteTalker(%d)"), LocalUserNum);
	}

	return Return == ONLINE_SUCCESS; //-V547
}

void FOnlineVoiceImpl::ProcessMuteChangeNotification()
{
	// Nothing to update if there isn't an active session
	if (VoiceEngine.IsValid())
	{
		if (SessionInt && SessionInt->GetNumSessions() > 0)
		{
			// For each local user with voice
			for (int32 Index = 0; Index < MaxLocalTalkers; Index++)
			{
				// Use the common method of checking muting
				UpdateMuteListForLocalTalker(Index);
			}
		}
	}
}

IVoiceEnginePtr FOnlineVoiceImpl::CreateVoiceEngine()
{
	return MakeShareable(new FVoiceEngineImpl(OnlineSubsystem));
}

void FOnlineVoiceImpl::UpdateMuteListForLocalTalker(int32 TalkerIndex)
{
	// Find the very first ULocalPlayer for this ControllerId. 
	// This is imperfect and means we cannot support voice chat properly for
	// multiple UWorlds (but thats ok for the time being).
	ULocalPlayer* LP = GEngine->FindFirstLocalPlayerFromControllerId(TalkerIndex);
	APlayerController* PC = LP ? LP->PlayerController.Get() : nullptr;
	if (PC)
	{
		// For each registered remote talker
		for (int32 RemoteIndex = 0; RemoteIndex < RemoteTalkers.Num(); RemoteIndex++)
		{
			const FRemoteTalker& Talker = RemoteTalkers[RemoteIndex];

			FUniqueNetIdRepl UniqueIdRepl(Talker.TalkerId);

			// Is the remote talker on this local player's mute list?
			if (SystemMuteList.Find(FUniqueNetIdWrapper(Talker.TalkerId->AsShared())) == INDEX_NONE)
			{
				// Unmute on the server
				PC->ServerUnmutePlayer(UniqueIdRepl);
			}
			else
			{
				// Mute on the server
				PC->ServerMutePlayer(UniqueIdRepl);
			}

			// The ServerUn/MutePlayer() functions will perform the muting based
			// upon gameplay settings and other player's mute list
		}
	}
}

TSharedPtr<FVoicePacket> FOnlineVoiceImpl::SerializeRemotePacket(FArchive& Ar)
{
	TSharedPtr<FVoicePacketImpl> NewPacket = MakeShareable(new FVoicePacketImpl());
	NewPacket->Serialize(Ar);
	if (Ar.IsError() == false && NewPacket->GetBufferSize() > 0)
	{
		if (!OnlineSubsystem->IsDedicated())
		{
			if (MuteList.Find(FUniqueNetIdWrapper( NewPacket->GetSender())) == INDEX_NONE)
			{
				VoiceData.RemotePackets.Add(NewPacket);
			}
		}
		return NewPacket;
	}

	return NULL;
}

TSharedPtr<FVoicePacket> FOnlineVoiceImpl::GetLocalPacket(uint32 LocalUserNum)
{
	// duplicate the local copy of the data and set it on a shared pointer for destruction elsewhere
	if (LocalUserNum >= 0 && LocalUserNum < MAX_SPLITSCREEN_TALKERS)
	{
		FVoicePacketImpl& VoicePacket = VoiceData.LocalPackets[LocalUserNum];
		if (VoicePacket.GetBufferSize() > 0)
		{
			return MakeShareable(new FVoicePacketImpl(VoicePacket));
		}
	}

	return NULL;
}

void FOnlineVoiceImpl::ProcessTalkingDelegates(float DeltaTime)
{
	// Fire off any talker notification delegates for local talkers
	for (int32 LocalUserNum = 0; LocalUserNum < LocalTalkers.Num(); LocalUserNum++)
	{
		FLocalTalker& Talker = LocalTalkers[LocalUserNum];

		// Only check players with voice
		if (Talker.bIsRegistered)
		{
			// If the talker was not previously talking, but now is trigger the event
			bool bShouldNotify = !Talker.bWasTalking && Talker.bIsTalking;
			// If the talker was previously talking, but now isn't time delay the event
			if (!bShouldNotify && Talker.bWasTalking)
			{
				Talker.LastNotificationTime -= DeltaTime;
				if (Talker.LastNotificationTime <= 0.f)
				{
					// Clear the flag so it only activates when needed
					Talker.bIsTalking = false;
					Talker.LastNotificationTime = VoiceNotificationDelta;
					bShouldNotify = true;
				}
			}

			if (bShouldNotify)
			{
				// Skip all delegate handling if none are registered
				if (OnPlayerTalkingStateChangedDelegates.IsBound())
				{
					FUniqueNetIdPtr UniqueId = IdentityInt->GetUniquePlayerId(LocalUserNum);
					OnPlayerTalkingStateChangedDelegates.Broadcast(UniqueId.ToSharedRef(), Talker.bIsTalking);
				}

				Talker.bWasTalking = Talker.bIsTalking;
				UE_LOG_ONLINE_VOICE(Log, TEXT("Trigger Local %d %sTALKING"), LocalUserNum, Talker.bIsTalking ? TEXT("") : TEXT("NOT"));
			}
		}
	}
	// Now check all remote talkers
	for (int32 Index = 0; Index < RemoteTalkers.Num(); Index++)
	{
		FRemoteTalker& Talker = RemoteTalkers[Index];

		// If the talker was not previously talking, but now is trigger the event
		bool bShouldNotify = !Talker.bWasTalking && Talker.bIsTalking;
		// If the talker was previously talking, but now isn't time delay the event
		if (!bShouldNotify && Talker.bWasTalking && !Talker.bIsTalking)
		{
			Talker.LastNotificationTime -= DeltaTime;
			if (Talker.LastNotificationTime <= 0.f)
			{
				bShouldNotify = true;
			}
		}

		if (bShouldNotify)
		{
			// Skip all delegate handling if none are registered
			if (OnPlayerTalkingStateChangedDelegates.IsBound())
			{
				OnPlayerTalkingStateChangedDelegates.Broadcast(Talker.TalkerId.ToSharedRef(), Talker.bIsTalking);
			}

			UE_LOG_ONLINE_VOICE(Log, TEXT("Trigger Remote %s %sTALKING"), *Talker.TalkerId->ToString(), Talker.bIsTalking ? TEXT("") : TEXT("NOT"));

			// Clear the flag so it only activates when needed
			Talker.bWasTalking = Talker.bIsTalking;
			Talker.LastNotificationTime = VoiceNotificationDelta;
		}
	}
}

void FOnlineVoiceImpl::ProcessLocalVoicePackets()
{
	if (VoiceEngine.IsValid())
	{
		// Read the data from any local talkers
		uint32 DataReadyFlags = VoiceEngine->GetVoiceDataReadyFlags();
		// Skip processing if there is no data from a local talker
		if (DataReadyFlags)
		{
			// Process each talker with a bit set
			for (uint32 Index = 0; DataReadyFlags; Index++, DataReadyFlags >>= 1)
			{
				// Talkers needing processing will always be in lsb due to shifts
				if (DataReadyFlags & 1)
				{
					uint32 SpaceAvail = UVOIPStatics::GetMaxVoiceDataSize() - VoiceData.LocalPackets[Index].Length;
					// Figure out if there is space for this packet
					if (SpaceAvail > 0)
					{
						// Figure out where to append the data
						uint8* BufferStart = VoiceData.LocalPackets[Index].Buffer.GetData();
						BufferStart += VoiceData.LocalPackets[Index].Length;
						// Copy the sender info
						VoiceData.LocalPackets[Index].Sender = IdentityInt->GetUniquePlayerId(Index);

						uint64 SampleCount = VoiceData.LocalPackets[Index].SampleCount;

						// Process this user
						uint32 Result = VoiceEngine->ReadLocalVoiceData(Index, BufferStart, &SpaceAvail, &SampleCount);

						float Amplitude = VoiceEngine->GetMicrophoneAmplitude(Index);
						Amplitude = Amplitude < 0.0f ? 1.0f : Amplitude; // GetMicrophoneAmplitude returns -1 if not implemented: we set it to a sane value so we don't trigger the ensure
						if (!ensure(Amplitude >= 0.0f && Amplitude <= 1.0f))
						{
							Amplitude = 1.0f; // make sure Amplitude has a sane value in any case
						}
						// Convert to Q15:
						VoiceData.LocalPackets[Index].MicrophoneAmplitude = (int16)(Amplitude * 32767.0f);

						if (Result == ONLINE_SUCCESS)
						{
							if (LocalTalkers[Index].bHasNetworkedVoice)
							{
								// Mark the person as talking
								LocalTalkers[Index].bIsTalking = true;
								LocalTalkers[Index].LastNotificationTime = VoiceNotificationDelta;

								// Update the length based on what it copied
								VoiceData.LocalPackets[Index].Length += uint16(SpaceAvail);
								VoiceData.LocalPackets[Index].SampleCount = SampleCount;

#if VOICE_LOOPBACK
								if (OSSConsoleVariables::CVarVoiceLoopback.GetValueOnGameThread() && SpaceAvail > 0)
								{
									VoiceData.RemotePackets.Add(MakeShareable(new FVoicePacketImpl(VoiceData.LocalPackets[Index])));
								}
#endif
							}
							else
							{
								// Zero out the data since it isn't to be sent via the network
								VoiceData.LocalPackets[Index].Length = 0;
							}
						}
						else
						{
							UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("Voice data error in ReadLocalVoiceData"));
						}
					}
					else
					{
						UE_LOG_ONLINE_VOICEENGINE(Warning, TEXT("Dropping voice data due to network layer not processing fast enough"));
						// Buffer overflow, so drop previous data
						VoiceData.LocalPackets[Index].Length = 0;
					}
				}
			}
		}
	}
}

void FOnlineVoiceImpl::ProcessRemoteVoicePackets()
{
	// Clear the talking state for remote players
	for (int32 Index = 0; Index < RemoteTalkers.Num(); Index++)
	{
		RemoteTalkers[Index].bIsTalking = false;
	}

	// Now process all pending packets from the server
	for (int32 Index = 0; Index < VoiceData.RemotePackets.Num(); Index++)
	{
		TSharedPtr<FVoicePacketImpl> VoicePacket = StaticCastSharedPtr<FVoicePacketImpl>(VoiceData.RemotePackets[Index]);
		if (VoicePacket.IsValid())
		{
			// Skip local submission of voice if dedicated server or no voice
			if (VoiceEngine.IsValid())
			{
				// Get the size since it is an in/out param
				uint32 VoiceBufferSize = VoicePacket->GetBufferSize();
				uint64 VoiceSampleCounter = VoicePacket->GetSampleCounter();
				// Submit this packet to the voice engine
				uint32 Result = VoiceEngine->SubmitRemoteVoiceData(VoicePacket->Sender, VoicePacket->Buffer.GetData(), &VoiceBufferSize, VoiceSampleCounter);
				if (Result != ONLINE_SUCCESS)
				{
					UE_LOG_ONLINE_VOICEENGINE(Warning,
						TEXT("SubmitRemoteVoiceData(%s) failed with 0x%08X"),
						*VoicePacket->Sender->ToDebugString(),
						Result);
				}

				// Convert amplitude from Q15:
				const float Amplitude = ((float)VoicePacket->MicrophoneAmplitude) / 32767.0f;
				ensureAlways(Amplitude >= 0.0f && Amplitude <= 1.0f);
				Result = VoiceEngine->SetRemoteVoiceAmplitude(FUniqueNetIdWrapper(VoicePacket->Sender), Amplitude);

				if (Result != ONLINE_SUCCESS)
				{
					UE_LOG_ONLINE_VOICEENGINE(Warning,
						TEXT("SubmitRemoteVoiceData(%s) failed with 0x%08X"),
						*VoicePacket->Sender->ToDebugString(),
						Result);
				}
			}

			// Find the remote talker and mark them as talking
			for (int32 Index2 = 0; Index2 < RemoteTalkers.Num(); Index2++)
			{
				FRemoteTalker& Talker = RemoteTalkers[Index2];
				// Compare the ids
				if (*Talker.TalkerId == *VoicePacket->Sender)
				{
					// If the player is marked as muted, they can't be talking
					Talker.bIsTalking = !IsLocallyMuted(*Talker.TalkerId);
					Talker.LastNotificationTime = VoiceNotificationDelta;
				}
			}
		}
	}
	// Zero the list without causing a free/realloc
	VoiceData.RemotePackets.Reset();
}

FString FOnlineVoiceImpl::GetVoiceDebugState() const
{
	FUniqueNetIdPtr UniqueId;

	FString Output = TEXT("Voice state\n");
	Output += VoiceEngine.IsValid() ? VoiceEngine->GetVoiceDebugState() : TEXT("No Voice Engine!");

	Output += TEXT("\nLocal Talkers:\n");
	for (int32 idx=0; idx < LocalTalkers.Num(); idx++)
	{
		UniqueId = IdentityInt->GetUniquePlayerId(idx);
		
		const FLocalTalker& Talker = LocalTalkers[idx];
		Output += FString::Printf(TEXT("[%d]: %s\n Registered: %d\n Networked: %d\n Talking: %d\n WasTalking: %d\n Last:%0.2f\n"),
			idx,
			UniqueId.IsValid() ? *UniqueId->ToDebugString() : TEXT("NULL"), 
			Talker.bIsRegistered,
			Talker.bHasNetworkedVoice,
			Talker.bIsTalking,
			Talker.bWasTalking,
			Talker.LastNotificationTime);
	}

	Output += TEXT("\nRemote Talkers:\n");
	for (int32 idx=0; idx < RemoteTalkers.Num(); idx++)
	{
		const FRemoteTalker& Talker = RemoteTalkers[idx];
		Output += FString::Printf(TEXT("[%d]: %s\n Talking: %d\n WasTalking: %d\n Muted: %s\n Last:%0.2f\n"),
			idx,
			*Talker.TalkerId->ToDebugString(), 
			Talker.bIsTalking,
			Talker.bWasTalking,
			IsLocallyMuted(*Talker.TalkerId) ? TEXT("1") : TEXT("0"),
			Talker.LastNotificationTime);

	}

	Output += TEXT("\nRaw SystemMutelist:\n");
	for (int32 idx=0; idx < SystemMuteList.Num(); idx++)
	{
		Output += FString::Printf(TEXT("[%d]=%s\n"),
			idx,
			*SystemMuteList[idx].ToString());
	}

	Output += TEXT("\nRaw Mutelist:\n");
	for (int32 idx=0; idx < MuteList.Num(); idx++)
	{
		Output += FString::Printf(TEXT("[%d]=%s\n"),
			idx,
			*MuteList[idx].ToString());
	}

	return Output;
}

Audio::FPatchOutputStrongPtr FOnlineVoiceImpl::GetMicrophoneOutput()
{
	if (VoiceEngine.IsValid())
	{
		return VoiceEngine->GetMicrophoneOutput();
	}
	else
	{
		return nullptr;
	}
}

Audio::FPatchOutputStrongPtr FOnlineVoiceImpl::GetRemoteTalkerOutput()
{
	if (VoiceEngine.IsValid())
	{
		return VoiceEngine->GetRemoteTalkerOutput();
	}
	else
	{
		return nullptr;
	}
}

float FOnlineVoiceImpl::GetAmplitudeOfRemoteTalker(const FUniqueNetId& PlayerId)
{
	if (VoiceEngine.IsValid())
	{
		return VoiceEngine->GetIncomingAudioAmplitude(FUniqueNetIdWrapper(PlayerId.AsShared()));
	}
	else
	{
		return 0.0f;
	}
}

bool FOnlineVoiceImpl::PatchRemoteTalkerOutputToEndpoint(const FString& InDeviceName, bool bMuteInGameOutput /*= true*/)
{
	if (VoiceEngine.IsValid())
	{
		return VoiceEngine->PatchRemoteTalkerOutputToEndpoint(InDeviceName, bMuteInGameOutput);
	}
	else
	{
		return false;
	}
}

bool FOnlineVoiceImpl::PatchLocalTalkerOutputToEndpoint(const FString& InDeviceName)
{
	if (VoiceEngine.IsValid())
	{
		return VoiceEngine->PatchLocalTalkerOutputToEndpoint(InDeviceName);
	}
	else
	{
		return false;
	}
}

void FOnlineVoiceImpl::DisconnectAllEndpoints()
{
	if (VoiceEngine.IsValid())
	{
		return VoiceEngine->DisconnectAllEndpoints();
	}
}
