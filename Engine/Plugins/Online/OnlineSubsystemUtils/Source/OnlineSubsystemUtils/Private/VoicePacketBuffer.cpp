// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoicePacketBuffer.h"
#include "Audio.h"
#include "Trace/Trace.inl"

#define DEBUG_SORTING 0
#define DEBUG_POPPING 0
#define DEBUG_SKIPSILENCE 1

FVoicePacketBuffer::FVoicePacketBuffer(int32 BufferSize, int32 InNumSamplesUntilIdling, uint64 InStartSample)
	: ListHead(nullptr)
	, SampleCounter(InStartSample)
	, IdleSampleCounter(0)
	, NumSamplesUntilIdling(InNumSamplesUntilIdling)
{
	PacketBuffer.AddDefaulted(BufferSize);
	for (int32 Index = 0; Index < BufferSize; Index++)
	{
		PacketBuffer[Index].IndexInPacketBuffer = Index;
		FreedIndicesQueue.Enqueue(Index);
	}
}

int32 FVoicePacketBuffer::PopAudio(float* DestBuffer, uint32 RequestedSamples)
{
#if DEBUG_POPPING
	UE_LOGF(LogAudio, Log, "Beginning to pop %d samples...", RequestedSamples);
#endif

#if SCOPELOCK_VOICE_PACKET_BUFFER
	FScopeLock ScopeLock(&ListHeadCriticalSection);
#endif

	uint32 DestBufferSampleIndex = 0;
	// start churning through packets.
	while (DestBufferSampleIndex < RequestedSamples)
	{
#if DEBUG_POPPING
		UE_LOGF(LogAudio, Log, "  Loop start, index %d ", DestBufferSampleIndex);
#endif
		//If we don't have any data, output silence and short circuit.
		if (ListHead == nullptr)
		{
			FMemory::Memset(&(DestBuffer[DestBufferSampleIndex]), 0, (RequestedSamples - DestBufferSampleIndex) * sizeof(float));
			IdleSampleCounter += RequestedSamples;
			DestBufferSampleIndex += RequestedSamples;
			if (IdleSampleCounter > NumSamplesUntilIdling)
			{
#if DEBUG_POPPING
				UE_LOGF(LogAudio, Warning, "Voip listener now idle.");
#endif
				bIsIdle = true;
			}

			return 0;
		}
		// If our current packet is more than 0 samples later than our sample counter,
		// fill silence until we get to the sample index.
		else if (ListHead->StartSample > SampleCounter)
		{
#if DEBUG_SKIPSILENCE
			SampleCounter = ListHead->StartSample;
#else
			const int32 SampleOffset = ListHead->StartSample - SampleCounter;
			const int32 NumSilentSamples = FMath::Min(SampleOffset, (int32) (RequestedSamples - DestBufferSampleIndex));
#if DEBUG_POPPING
			UE_LOGF(LogAudio, Log, "    Sample counter %d behind next packet. Injecting %d samples of silence.", SampleOffset, NumSilentSamples);
#endif
			FMemory::Memset(&(DestBuffer[DestBufferSampleIndex]), 0, NumSilentSamples * sizeof(float));
			DestBufferSampleIndex += NumSilentSamples;
			SampleCounter += NumSilentSamples;
			IdleSampleCounter += NumSilentSamples;
			if (IdleSampleCounter > NumSamplesUntilIdling)
			{
				UE_LOGF(LogAudio, Warning, "Voip listener now idle.");
				bIsIdle = true;
			}
#endif
		}
		// If we have less samples in the current packet than we need to churn through,
		// copy over the rest of the samples in this packet and move on to the next packet.
		else if (ListHead->SamplesLeft <=  (int32) (RequestedSamples - DestBufferSampleIndex))
		{
			bIsIdle = false;
			const uint32 NumSamplesLeftInPacketBuffer = ListHead->SamplesLeft;
			const int32 PacketSampleIndex = ListHead->BufferNumSamples - NumSamplesLeftInPacketBuffer;
			const float* AudioBuffer = ListHead->AudioBufferMem.GetData();
			const float* PacketBufferPtr = &(AudioBuffer[PacketSampleIndex]);
			FMemory::Memcpy(&(DestBuffer[DestBufferSampleIndex]), PacketBufferPtr, NumSamplesLeftInPacketBuffer * sizeof(float));
			SampleCounter += NumSamplesLeftInPacketBuffer;
			DestBufferSampleIndex += NumSamplesLeftInPacketBuffer;
#if DEBUG_POPPING
			check((ListHead->SamplesLeft - NumSamplesLeftInPacketBuffer) == 0);
			UE_LOGF(LogAudio, Log, "    Copied %d samples from packet and reached end of packet.", NumSamplesLeftInPacketBuffer);
#endif
			//Delete this packet and move on to the next one.
			FreedIndicesQueue.Enqueue(ListHead->IndexInPacketBuffer);
			ListHead = ListHead->NextPacket;
		}
		// If we have more samples in the current packet than we need to churn through,
		// copy over samples and update the state in the packet node.
		else
		{
			int32 SamplesLeft = RequestedSamples - DestBufferSampleIndex;
			const int32 PacketSampleIndex = ListHead->BufferNumSamples - ListHead->SamplesLeft;
			const float* AudioBuffer = ListHead->AudioBufferMem.GetData();
			const float* PacketBufferPtr = &(AudioBuffer[PacketSampleIndex]);
			FMemory::Memcpy(&(DestBuffer[DestBufferSampleIndex]), PacketBufferPtr, SamplesLeft * sizeof(float));
#if DEBUG_POPPING
			check(DestBufferSampleIndex + SamplesLeft == RequestedSamples);
			UE_LOGF(LogAudio, Log, "    Copied %d samples from packet and reached end of buffer.", SamplesLeft);
#endif
			ListHead->SamplesLeft -= SamplesLeft;
			SampleCounter += SamplesLeft;
			DestBufferSampleIndex += SamplesLeft;
			IdleSampleCounter = 0;
			bIsIdle = false;
		}
#if DEBUG_POPPING
		UE_LOGF(LogAudio, Log, "  Ended loop with %d samples popped to buffer.", DestBufferSampleIndex);
#endif
	}
#if DEBUG_POPPING
	check(DestBufferSampleIndex == RequestedSamples);
	UE_LOGF(LogAudio, Log, "Finished PopAudio.");
#endif

	const int32 SamplesPopped = RequestedSamples - IdleSampleCounter;
	NumBufferedSamples.Subtract(SamplesPopped);

	return SamplesPopped;
}

int32 FVoicePacketBuffer::GetNumBufferedSamples()
{
	return NumBufferedSamples.GetValue();
}

uint32 FVoicePacketBuffer::DropOldestAudio(uint32 InNumSamples)
{
#if DEBUG_POPPING
	UE_LOGF(LogAudio, Log, "Beginning to seek forward %d samples...", InNumSamples);
#endif

#if SCOPELOCK_VOICE_PACKET_BUFFER
	FScopeLock ScopeLock(&ListHeadCriticalSection);
#endif

	uint32 NumSamplesSeeked = 0;

	// start churning through packets.
	while (NumSamplesSeeked < InNumSamples)
	{
#if DEBUG_POPPING
		UE_LOGF(LogAudio, Log, "  Loop start, index %d ", NumSamplesSeeked);
#endif

		if (ListHead == nullptr)
		{
			return NumSamplesSeeked;
		}

		// If our current packet is more than 0 samples later than our sample counter,
		// fill silence until we get to the sample index.
		else if (ListHead->StartSample > SampleCounter)
		{
#if DEBUG_SKIPSILENCE
			SampleCounter = ListHead->StartSample;
#else
			
#if DEBUG_POPPING
			const int32 SampleOffset = ListHead->StartSample - SampleCounter;
			const int32 NumSilentSamples = FMath::Min(SampleOffset, (int32)(RequestedSamples - DestBufferSampleIndex));
			UE_LOGF(LogAudio, Log, "    Sample counter %d behind next packet. Injecting %d samples of silence.", SampleOffset, NumSilentSamples);
#endif
#endif
		}
		// If we have less samples in the current packet than we need to churn through,
		// copy over the rest of the samples in this packet and move on to the next packet.
		else if (ListHead->SamplesLeft <= (int32)(InNumSamples - NumSamplesSeeked))
		{
			bIsIdle = false;
			const uint32 NumSamplesLeftInPacketBuffer = ListHead->SamplesLeft;
			SampleCounter += NumSamplesLeftInPacketBuffer;
			NumSamplesSeeked += NumSamplesLeftInPacketBuffer;
#if DEBUG_POPPING
			check((ListHead->SamplesLeft - NumSamplesLeftInPacketBuffer) == 0);
			UE_LOGF(LogAudio, Log, "    Copied %d samples from packet and reached end of packet.", NumSamplesLeftInPacketBuffer);
#endif
			//Delete this packet and move on to the next one.
			FreedIndicesQueue.Enqueue(ListHead->IndexInPacketBuffer);
			ListHead = ListHead->NextPacket;
		}
		// If we have more samples in the current packet than we need to churn through,
		// copy over samples and update the state in the packet node.
		else
		{
			int32 SamplesLeft = InNumSamples - NumSamplesSeeked;
#if DEBUG_POPPING
			check(NumSamplesSeeked + SamplesLeft == InNumSamples);
			UE_LOGF(LogAudio, Log, "    Copied %d samples from packet and reached end of buffer.", SamplesLeft);
#endif
			ListHead->SamplesLeft -= SamplesLeft;
			SampleCounter += SamplesLeft; 
			NumSamplesSeeked += SamplesLeft;
			IdleSampleCounter = 0;
			bIsIdle = false;
		}
#if DEBUG_POPPING
		UE_LOGF(LogAudio, Log, "  Ended loop with %d samples popped to buffer.", DestBufferSampleIndex);
#endif
	}

#if DEBUG_POPPING
	check(NumSamplesSeeked == InNumSamples);
	UE_LOGF(LogAudio, Log, "Finished PopAudio.");
#endif

	const int32 SamplesPopped = InNumSamples - IdleSampleCounter;
	NumBufferedSamples.Subtract(SamplesPopped);

	return SamplesPopped;
}

void FVoicePacketBuffer::Reset(int32 InStartSample)
{
	FreedIndicesQueue.Empty();
	for (int32 Index = 0; Index < PacketBuffer.Num(); Index++)
	{
		PacketBuffer[Index].NextPacket = nullptr;
		FreedIndicesQueue.Enqueue(Index);
	}
	ListHead = nullptr;
	IdleSampleCounter = 0;
	bIsIdle = false;
	SampleCounter = InStartSample;

}

void FVoicePacketBuffer::PushPacket(const void* InBuffer, int32 NumBytes, uint64 InStartSample, EVoipStreamDataFormat Format)
{
	if (SampleCounter > InStartSample + NumBytes)
	{
		SampleCounter = InStartSample;
	}

	int32 Index = INDEX_NONE;

	// Add Packet to buffer. If we have any used up packets, overwrite them.
	// Otherwise, add a new packet to the end of the buffer, if we haven't filled our preallocated buffer.
	if (FreedIndicesQueue.Dequeue(Index))
	{
		PacketBuffer[Index].Initialize(InBuffer, NumBytes, InStartSample, Format);
	}
	else
	{
		// In order to prevent log spam, we only call log this every 128 times it happens.
		static const int32 NumPacketsDroppedPerLog = 128;
		static int32 PacketDropCounter = 0;

		if (PacketDropCounter == 0)
		{
			UE_LOGF(LogAudio, Warning, "Voice packet buffer filled to capacity of %d packets; packet dropped.", PacketBuffer.Num());
		}

		PacketDropCounter = (PacketDropCounter + 1) % NumPacketsDroppedPerLog;
		
		return;
	}

	check(Index != INDEX_NONE);

	// The packet needs to know where it is in the buffer, so it can enqueue it's index
	// so that it can be overwritten after it's used.
	PacketBuffer[Index].IndexInPacketBuffer = Index;
	FSortedVoicePacketNode* PacketNodePtr = &(PacketBuffer[Index]);

#if SCOPELOCK_VOICE_PACKET_BUFFER
	FScopeLock ScopeLock(&ListHeadCriticalSection);
#endif

	// If our buffer is empty, we can just place this packet at the beginning.
	// Otherwise, we need to traverse our sorted list and insert the packet at
	// the correct place based on StartSample.
	if (ListHead == nullptr)
	{
		ListHead = PacketNodePtr;
		SampleCounter = InStartSample;
	}
	// If the packet coming in is earlier than the current packet,
	// insert this packet at the beginning.
	else if (InStartSample < ListHead->StartSample)
	{
		PacketNodePtr->NextPacket = ListHead;
		ListHead = PacketNodePtr;
	}
	// Otherwise, we have to scan through our packet list to insert this packet.
	else
	{
		FSortedVoicePacketNode* CurrentPacket = ListHead;
		FSortedVoicePacketNode* NextPacket = CurrentPacket->NextPacket;

		//Scan through list and insert this packet in the correct place.
		while (NextPacket != nullptr)
		{
			if (NextPacket->StartSample > InStartSample)
			{
				// The packet after this comes after this packet. Insert this packet.
				CurrentPacket->NextPacket = PacketNodePtr;
				CurrentPacket->NextPacket->NextPacket = NextPacket;
				return;
			}
			else
			{
				// Step through to the next packet.
				CurrentPacket = NextPacket;
				NextPacket = NextPacket->NextPacket;
			}
		}

		// If we've made it to this point, it means the pushed packet is the last packet chronologically.
		// Thus, we insert this packet at the end of the list.
		CurrentPacket->NextPacket = PacketNodePtr;
	}

	NumBufferedSamples.Add(NumBytes / SizeOfSample(Format));

#if DEBUG_SORTING
	UE_LOGF(LogAudio, Log, "Packet sort order: ");
	FSortedVoicePacketNode* PacketPtr = ListHead;
	int32 LoggedPatcketIndex = 0;
	uint64 CachedSampleStart = 0;
	while (PacketPtr != nullptr)
	{
		UE_LOGF(LogAudio, Log, "    Packet %d: Sample Start %d", ++LoggedPatcketIndex, PacketPtr->StartSample);
		check(PacketPtr->StartSample > CachedSampleStart);
		CachedSampleStart = PacketPtr->StartSample;
		PacketPtr = PacketPtr->NextPacket;
	}
	if (LoggedPatcketIndex > 6)
	{
		//Put a breakpoint here to check out sample counts.
		UE_LOGF(LogAudio, Log, "Several Packets buffered.");
	}
	UE_LOGF(LogAudio, Log, "End of packet list.");
#endif
}
