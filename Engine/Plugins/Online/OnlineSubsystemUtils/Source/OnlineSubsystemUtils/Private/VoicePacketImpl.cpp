// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoicePacketImpl.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineIdentityInterface.h"

FVoicePacketImpl::FVoicePacketImpl(const FVoicePacketImpl& Other) :
	FVoicePacket(Other)
{
	Sender = Other.Sender;
	Length = Other.Length;
	SampleCount = Other.SampleCount;
	MicrophoneAmplitude = Other.MicrophoneAmplitude;

	// Copy the contents of the voice packet
	Buffer.Empty(Other.Length);
	Buffer.AddUninitialized(Other.Length);
	FMemory::Memcpy(Buffer.GetData(), Other.Buffer.GetData(), Other.Length);
}

uint16 FVoicePacketImpl::GetTotalPacketSize()
{
#if DEBUG_VOICE_PACKET_ENCODING
	return Sender->GetSize() + sizeof(Length) + Length + sizeof(uint32);
#else
	return Sender->GetSize() + sizeof(Length) + Length;
#endif
}

uint16 FVoicePacketImpl::GetBufferSize()
{
	return Length;
}

FUniqueNetIdPtr FVoicePacketImpl::GetSender()
{
	return Sender;
}

void FVoicePacketImpl::Serialize(class FArchive& Ar)
{
	// Make sure not to overflow the buffer by reading an invalid amount
	if (Ar.IsLoading())
	{
		FString SenderStr;
		Ar << SenderStr;

		Ar << MicrophoneAmplitude;

		// Don't need to distinguish OSS interfaces here with world because we just want the create function below
		IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
		IOnlineIdentityPtr IdentityInt = OnlineSub->GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			Sender = IdentityInt->CreateUniquePlayerId(SenderStr);
		}
		
		Ar << Length;
		Ar << SampleCount;
		

		// Verify the packet is a valid size
		if (Length <= UVOIPStatics::GetMaxVoiceDataSize())
		{
			Buffer.Empty(Length);
			Buffer.AddUninitialized(Length);
			Ar.Serialize(Buffer.GetData(), Length);

#if DEBUG_VOICE_PACKET_ENCODING
			uint32 CRC = 0;
			Ar << CRC;
			if (CRC != FCrc::MemCrc32(Buffer.GetData(), Length))
			{
				UE_LOGF(LogVoice, Warning, "CRC Mismatch in voice packet");
				Length = 0;
			}
#endif
		}
		else
		{
			Length = 0;
		}
	}
	else
	{
		check(Sender.IsValid());
		FString SenderStr = Sender->ToString();
		Ar << SenderStr;
		
		Ar << MicrophoneAmplitude;

		Ar << Length;
		Ar << SampleCount;
		
		// Always safe to save the data as the voice code prevents overwrites
		Ar.Serialize(Buffer.GetData(), Length);

#if DEBUG_VOICE_PACKET_ENCODING
		uint32 CRC = FCrc::MemCrc32(Buffer.GetData(), Length);
		Ar << CRC;
#endif
	}
}

void FVoicePacketImpl::ResetData()
{
	Length = 0;
	MicrophoneAmplitude = 0;
}
