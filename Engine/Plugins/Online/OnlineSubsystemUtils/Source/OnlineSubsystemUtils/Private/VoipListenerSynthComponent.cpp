// Copyright Epic Games, Inc. All Rights Reserved.
#include "VoipListenerSynthComponent.h"
#include "VoicePacketBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoipListenerSynthComponent)

static float NumSecondsUntilIdling = 0.4f;

#if DEBUG_BUFFERING
static int64 StartingSample = 0;
static int16 DebugBuffer[2048];
#endif

#define DEBUG_NOISE 0

static int32 DefaultPatchBufferSizeCVar = 4096;
FAutoConsoleVariableRef CVarDefaultPatchBufferSize(
	TEXT("voice.DefaultPatchBufferSize"),
	DefaultPatchBufferSizeCVar,
	TEXT("Changes the amount of audio we buffer for VOIP patching, in samples.\n"),
	ECVF_Default);

static float DefaultPatchGainCVar = 1.0f;
FAutoConsoleVariableRef CVarDefaultPatchGain(
	TEXT("voice.DefaultPatchGain"),
	DefaultPatchGainCVar,
	TEXT("Changes the default gain of audio patches, in linear gain.\n"),
	ECVF_Default);

static int32 ShouldResyncCVar = 1;
FAutoConsoleVariableRef CVarShouldResync(
	TEXT("voice.playback.ShouldResync"),
	ShouldResyncCVar,
	TEXT("If set to 1, we will resync VOIP audio once it's latency goes beyond voice.playback.ResyncThreshold.\n"),
	ECVF_Default);

static int32 MuteAudioEngineOutputCVar = 0;
FAutoConsoleVariableRef CVarMuteAudioEngineOutput(
	TEXT("voice.MuteAudioEngineOutput"),
	MuteAudioEngineOutputCVar,
	TEXT("When set to a nonzero value, the output for the audio engine will be muted..\n"),
	ECVF_Default);

static float ResyncThresholdCVar = 0.3f;
FAutoConsoleVariableRef CVarResyncThreshold(
	TEXT("voice.playback.ResyncThreshold"),
	ResyncThresholdCVar,
	TEXT("If the amount of audio we have buffered is greater than this value, we drop the oldest audio we have and sync to have voice.JitterDelay worth of buffered audio.\n"),
	ECVF_Default);



bool UVoipListenerSynthComponent::Init(int32& SampleRate)
{
	NumChannels = UVOIPStatics::GetVoiceNumChannels();
	SampleRate = UVOIPStatics::GetVoiceSampleRate();
	MySampleRate = float(SampleRate);

#if DEBUG_BUFFERING
	FMToneGenerator = FDebugFMTone(MySampleRate, 660.0, 0.2, 0.8, 4.0);
#endif

	return true;
}

int32 UVoipListenerSynthComponent::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	if (PreDelaySampleCounter > 0)
	{
		//Pre delay sample counter is pretty loose, so we kind of round to the nearest buffer here.
		PreDelaySampleCounter -= NumSamples;
	}
	else if (PacketBuffer.IsValid())
	{
		FScopeLock ScopeLock(&PacketBufferCriticalSection);
		// Handle resync, if neccessary.
		if (ShouldResyncCVar)
		{
			ForceResync();
		}

		PacketBuffer->PopAudio(OutAudio, NumSamples);
	}

#if DEBUG_NOISE
	for (int32 Index = 0; Index < NumSamples; Index++)
	{
		OutAudio[Index] = 0.5 * OutAudio[Index] + 0.5 * FMath::FRand();
	}
#endif

	// Push audio to the patch splitter.
	if (ExternalSend.IsOutputStillActive())
	{
		ExternalSend.PushAudio(OutAudio, NumSamples);
	}
	
	if (MuteAudioEngineOutputCVar)
	{
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
	}

	return NumSamples;
}

UVoipListenerSynthComponent::~UVoipListenerSynthComponent()
{
	ClosePacketStream();
}

void UVoipListenerSynthComponent::OpenPacketStream(uint64 BeginningSampleCount, int32 BufferSize, float JitterDelay)
{
	ClosePacketStream();
	PacketBuffer.Reset(new FVoicePacketBuffer(BufferSize, int32(NumSecondsUntilIdling * MySampleRate), BeginningSampleCount));
	
	JitterDelayInSeconds = JitterDelay;
	PreDelaySampleCounter = int32(JitterDelay * MySampleRate);
}

void UVoipListenerSynthComponent::ClosePacketStream()
{
	FScopeLock ScopeLock(&PacketBufferCriticalSection);
	PacketBuffer.Reset();
}

void UVoipListenerSynthComponent::ResetBuffer(int32 InStartSample, float JitterDelay)
{
	if (PacketBuffer.IsValid())
	{
		PacketBuffer->Reset(InStartSample);
		PreDelaySampleCounter = int32(JitterDelay * MySampleRate);
	}
}

void UVoipListenerSynthComponent::SubmitPacket(void* InBuffer, int32 NumBytes, int64 InStartSample, EVoipStreamDataFormat DataFormat)
{
	if (PacketBuffer.IsValid())
	{
#if DEBUG_BUFFERING
		FMToneGenerator.Generate(DebugBuffer, NumBytes / sizeof(int16));
		StartingSample += NumBytes / sizeof(int16);
		PacketBuffer->PushPacket(DebugBuffer, NumBytes, StartingSample, DataFormat);
#else
		static int64 StartSample = 0;
		StartSample += NumBytes / sizeof(int16);
		PacketBuffer->PushPacket(InBuffer, NumBytes, StartSample, DataFormat);
#endif
	}
	else
	{
		UE_LOGF(LogTemp, Error, "SubmitPacket called before OpenPacketStream was.");
	}
}

void UVoipListenerSynthComponent::ConnectToSplitter(Audio::FPatchMixerSplitter& InSplitter)
{
	ExternalSend = InSplitter.AddNewInput(DefaultPatchBufferSizeCVar, DefaultPatchGainCVar);
}

bool UVoipListenerSynthComponent::IsIdling()
{
	if (PacketBuffer.IsValid())
	{
		return PacketBuffer->IsIdle();
	}
	else
	{
		return true;
	}
}

uint64 UVoipListenerSynthComponent::GetSampleCounter()
{
	if (PacketBuffer.IsValid())
	{
		return PacketBuffer->GetCurrentSample();
	}
	else
	{
		return 0;
	}
}


void UVoipListenerSynthComponent::BeginDestroy()
{
	Super::BeginDestroy();
	
	ClosePacketStream();
}

void UVoipListenerSynthComponent::ForceResync()
{
	check(PacketBuffer.IsValid());

	const int32 TargetLatencyInSamples = FMath::FloorToInt(JitterDelayInSeconds * NumChannels * MySampleRate);
	const int32 CurrentLatencyInSamples = PacketBuffer->GetNumBufferedSamples();
	const int32 ResyncThresholdInSamples = FMath::FloorToInt(ResyncThresholdCVar * NumChannels * MySampleRate);

	int32 AmountToSkip = CurrentLatencyInSamples - TargetLatencyInSamples;

	if (AmountToSkip > ResyncThresholdInSamples)
	{
		PacketBuffer->DropOldestAudio(AmountToSkip);
	}
}

#if DEBUG_BUFFERING
FDebugFMTone::FDebugFMTone(float InSampleRate, float InCarrierFreq, float InModFreq, float InCarrierAmp, float InModAmp)
	: SampleRate(InSampleRate)
	, CarrierFreq(InCarrierFreq)
	, CarrierAmp(InCarrierAmp)
	, ModFreq(InModFreq)
	, ModAmp(InModAmp)
{
}

void FDebugFMTone::Generate(int16* BufferPtr, int32 NumSamples)
{
	for (int32 Index = 0; Index < NumSamples; Index++)
	{
		float OutSample = CarrierAmp * FMath::Sin(2 * PI * (CarrierFreq + ModAmp * FMath::Sin(2 * PI * ModFreq * n / SampleRate)) * n / SampleRate);
		BufferPtr[Index] = (int16)(OutSample * 32767.0f);
		n++;
	}
}
#endif
