// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Auth.h"
#include "Online/OnlineServices.h"

class UWorld;
struct FWorldContext;
struct FPIELoginSettingsInternal;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnExternalUIChange, bool);
typedef FOnExternalUIChange::FDelegate FOnExternalUIChangeDelegate;

namespace UE::Online {

/**
 * Interface class for various online utility functions for Engine
 */
class IOnlineServicesEngineUtils
{
protected:
	/** Hidden on purpose */
	IOnlineServicesEngineUtils() {}
public:

	virtual ~IOnlineServicesEngineUtils() = default;

	/**
	 * Gets an FName that uniquely identifies an instance of a world to differentiate online services for PIE
	 *
	 * @param WorldContext the worldcontext
	 * @return an FName that can be used with online services
	 */
	virtual FName GetOnlineIdentifier(const FWorldContext& WorldContext) const = 0;

	/**
	 * Gets an FName that uniquely identifies an instance of a world to differentiate online services for PIE
	 *
	 * @param World the world to use for context
	 * @return an FName that can be used with Online Services
	 */
	virtual FName GetOnlineIdentifier(const UWorld* World) const = 0;

	/**
	 * Bind a notification delegate when any subsystem external UI is opened/closed
	 * *NOTE* there is only meant to be one delegate needed for this, game code should bind manually
	 *
	 * @param OnExternalUIChangeDelegate delegate fired when the external UI is opened/closed
	 */
	virtual void SetEngineExternalUIBinding(const FOnExternalUIChangeDelegate& OnExternalUIChangeDelegate) = 0;

#if WITH_EDITOR
	/**
	 * Play in Editor settings
	 */

	/** @return true if the default platform supports logging in for Play In Editor (PIE) */
	virtual bool SupportsOnlinePIE() const = 0;
	/** Enable/Disable online PIE at runtime */
	virtual void SetShouldTryOnlinePIE(bool bShouldTry) = 0;
	/** @return true if the user has enabled logging in for Play In Editor (PIE) */
	virtual bool IsOnlinePIEEnabled() const = 0;
	/** @return the number of logins the user has setup for Play In Editor (PIE) */
	virtual int32 GetNumPIELogins() const = 0;
	/** @return the array of valid credentials the user has setup for Play In Editor (PIE) */
	virtual void GetPIELogins(TArray<FAuthLogin::Params>& Logins) const = 0;
#endif // WITH_EDITOR
};

/** @return the single instance of the online services utils interface */
ONLINESUBSYSTEMUTILS_API IOnlineServicesEngineUtils* GetServicesEngineUtils();

inline FName GetServicesInstanceName(const UWorld* World)
{
	FName Identifier;
#if UE_EDITOR // at present, multiple worlds are only possible in the editor
	if (World != nullptr)
	{
		IOnlineServicesEngineUtils* Utils = GetServicesEngineUtils();
		check(Utils);
		Identifier = Utils->GetOnlineIdentifier(World);
	}
#endif
	return Identifier;
}

inline IOnlineServicesPtr GetServices(const UWorld* World, EOnlineServices OnlineServices = EOnlineServices::Default)
{
	return GetServices(OnlineServices, GetServicesInstanceName(World));
}

/* UE::Online */ }
