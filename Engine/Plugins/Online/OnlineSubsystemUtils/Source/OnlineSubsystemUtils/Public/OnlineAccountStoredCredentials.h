// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "OnlineAccountStoredCredentials.generated.h"

/**
 * Stores online account login credentials (for editor config data).
 * @note ONLY use this in trusted environments (like a local config file) and NOT for anything that requires actual security/strong encryption.
 * @see FOnlineAccountCredentials.
 */
USTRUCT()
struct FOnlineAccountStoredCredentials
{
public:

	GENERATED_BODY()

	/** Id of the user logging in (email, display name, facebook id, etc) */
	UPROPERTY(EditAnywhere, Category = "Logins", meta = (DisplayName = "User Id", Tooltip = "Id of the user logging in (email, display name, facebook id, etc)"))
	FString Id;
	/** Credentials of the user logging in (password or auth token) */
	UPROPERTY(EditAnywhere, Transient, Category = "Logins", meta = (DisplayName = "Password", Tooltip = "Credentials of the user logging in (password or auth token)", PasswordField = true))
	FString Token;
	/** Type of account. Needed to identity the auth method to use (epic, internal, facebook, etc) */
	UPROPERTY(EditAnywhere, Category = "Logins", meta = (DisplayName = "Type", Tooltip = "Type of account. Needed to identity the auth method to use (epic, internal, facebook, etc)"))
	FString Type;
	/** Token stored as an array of bytes, encrypted */
	UPROPERTY()
	TArray<uint8> TokenBytes;

	/** @return true if the credentials are valid, false otherwise */
	bool IsValid() const
	{
		return !Id.IsEmpty() && !Token.IsEmpty() && !Type.IsEmpty();
	}

	/**
	 * Encrypt the Token field into the TokenBytes field
	 */
	ONLINESUBSYSTEMUTILS_API void Encrypt();
	
	/**
	 * Decrypt the TokenBytes field into the Token field
	 */
	ONLINESUBSYSTEMUTILS_API void Decrypt();
};
