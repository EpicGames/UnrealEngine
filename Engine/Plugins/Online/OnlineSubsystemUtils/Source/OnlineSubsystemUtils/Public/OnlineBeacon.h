// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/NetworkDelegates.h"
#include "GameFramework/Actor.h"
#include "Engine/NetDriver.h"
#include "OnlineBeacon.generated.h"

namespace EEndPlayReason { enum Type : int; }
namespace ENetworkFailure { enum Type : int; }

class FInBunch;
class UChannel;
class UNetConnection;

ONLINESUBSYSTEMUTILS_API DECLARE_LOG_CATEGORY_EXTERN(LogBeacon, Log, All);

/** States that a beacon can be in */
namespace EBeaconState
{
	enum Type
	{
		AllowRequests,
		DenyRequests
	};
}

enum class EBeaconFailureReason : uint8
{
	Unknown = 0,
	TransportError = 1,   
	VersionMismatch = 2,
	InitFailure = 3,
	EncryptionFailure = 4,
	ApplicationError = 5,
	TimeOut
};

inline const TCHAR* LexToString(EBeaconFailureReason Value)
{
	switch (Value)
	{
	case EBeaconFailureReason::Unknown:
		return TEXT("Unknown");
	case EBeaconFailureReason::TransportError:
		return TEXT("TransportError");
	case EBeaconFailureReason::VersionMismatch:
		return TEXT("VersionMismatch");
	case EBeaconFailureReason::InitFailure:
		return TEXT("FailedToInitBeacon");
	case EBeaconFailureReason::EncryptionFailure:
		return TEXT("EncryptionFailure");
	case EBeaconFailureReason::ApplicationError:
		return TEXT("ApplicationError");
	case EBeaconFailureReason::TimeOut:
		return TEXT("TimeOut");
	default:
		return TEXT("Unknown");
	}
}

/**
 * Base class for beacon communication (Unreal Networking, but outside normal gameplay traffic)
 */
UCLASS(MinimalApi, transient, config=Engine, notplaceable)
class AOnlineBeacon : public AActor, public FNetworkNotify
{
	GENERATED_UCLASS_BODY()

	//~ Begin AActor Interface
	ONLINESUBSYSTEMUTILS_API virtual void OnActorChannelOpen(FInBunch& InBunch, UNetConnection* Connection) override;
	ONLINESUBSYSTEMUTILS_API virtual bool IsRelevancyOwnerFor(const AActor* ReplicatedActor, const AActor* ActorOwner, const AActor* ConnectionActor) const override;
	ONLINESUBSYSTEMUTILS_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;
	virtual const AActor* GetNetOwner() const override { return nullptr; }
	virtual UNetConnection* GetNetConnection() const override { return nullptr; }
	virtual bool IsLevelBoundsRelevant() const override { return false; }
	ONLINESUBSYSTEMUTILS_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason);
	//~ End AActor Interface

	//~ Begin FNetworkNotify Interface
	ONLINESUBSYSTEMUTILS_API virtual EAcceptConnection::Type NotifyAcceptingConnection() override;
	ONLINESUBSYSTEMUTILS_API virtual void NotifyAcceptedConnection(UNetConnection* Connection) override;
	ONLINESUBSYSTEMUTILS_API virtual bool NotifyAcceptingChannel(UChannel* Channel) override;
	ONLINESUBSYSTEMUTILS_API virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch) override;
	//~ End FNetworkNotify Interface
	
    /**
	 * Get the current state of the beacon
	 *
	 * @return state of the beacon
	 */
	EBeaconState::Type GetBeaconState() const { return BeaconState; }

	/**
	 * Notification of network error messages, allows a beacon to handle the failure
	 *
	 * @param	World associated with failure
	 * @param	NetDriver associated with failure
	 * @param	FailureType	the type of error
	 * @param	ErrorString	additional string detailing the error
	 */
	ONLINESUBSYSTEMUTILS_API virtual void HandleNetworkFailure(UWorld* World, UNetDriver *NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString);

	/**
	 * Set Beacon state 
	 *
	 * @bPause should the beacon stop accepting requests
	 */
	void PauseBeaconRequests(bool bPause)
	{
		if (!NetDriver)
		{
			return;
		}

		if (bPause)
		{
			UE_LOGF(LogBeacon, Verbose, "All Beacon Requests Paused.");
			NetDriver->SetWorld(nullptr);
			NetDriver->Notify = this;
			BeaconState = EBeaconState::DenyRequests;
		}
		else
		{
			UE_LOGF(LogBeacon, Verbose, "All Beacon Requests Resumed.");
			NetDriver->SetWorld(GetWorld());
			NetDriver->Notify = this;
			BeaconState = EBeaconState::AllowRequests;
		}
	}

	/** Beacon cleanup and net driver destruction */
	ONLINESUBSYSTEMUTILS_API virtual void DestroyBeacon();

protected:

	ONLINESUBSYSTEMUTILS_API void CleanupNetDriver();

	/** Time beacon will wait to establish a connection with the beacon host */
	UPROPERTY(Config)
	float BeaconConnectionInitialTimeout;
	/** Time beacon will wait for packets after establishing a connection before giving up */
	UPROPERTY(Config)
	float BeaconConnectionTimeout;

	/** Net driver routing network traffic */
	UPROPERTY()
	TObjectPtr<UNetDriver> NetDriver;

	/** State of beacon */
	EBeaconState::Type BeaconState;
	/** Handle to the registered HandleNetworkFailure delegate */
	FDelegateHandle HandleNetworkFailureDelegateHandle;

	/** Name of definition to use when creating the net driver */
	FName NetDriverDefinitionName;

	/** Common initialization for all beacon types */
	ONLINESUBSYSTEMUTILS_API virtual bool InitBase();

	/** Notification that failure needs to be handled */
	UE_DEPRECATED(5.7, "Deprecated - Move to OnFailure(FBeaconFailureContext).")
	ONLINESUBSYSTEMUTILS_API virtual void OnFailure();

	ONLINESUBSYSTEMUTILS_API virtual void OnFailure(EBeaconFailureReason Reason, FStringView ErrorMessage);

	/** overridden to return that player controllers are capable of RPCs */
	ONLINESUBSYSTEMUTILS_API virtual bool HasNetOwner() const override;
};
