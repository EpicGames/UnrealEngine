// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeacon.h"
#include "OnlineBeaconClient.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FEncryptionKeyResponse;
struct FURL;

class AOnlineBeaconHostObject;
class FInBunch;
class UNetConnection;
struct FUniqueNetIdRepl;

/**
 * State of a connection.
 */
UENUM()
enum class EBeaconConnectionState : uint8
{
	Invalid = 0,// Connection is invalid, possibly uninitialized.
	Closed = 1,	// Connection permanently closed.
	Pending = 2,// Connection is awaiting connection.
	Open = 3,	// Connection is open.
};

/**
 * Parameters used when initializing AOnlineBeaconClient.
 */
struct FOnlineBeaconClientInitParams
{
	FURL URL;
	bool bReplicateTransactionally = true;
};

/**
 * Delegate triggered on failures to connect to a host beacon
 */
DECLARE_DELEGATE(FOnHostConnectionFailure);

/**
 * Base class for any unique beacon connectivity, paired with an AOnlineBeaconHostObject implementation 
 *
 * This is the actual actor that replicates across client/server and where all RPCs occur
 * On the host, the life cycle is managed by an AOnlineBeaconHostObject
 * On the client, the life cycle is managed by the game 
 */
UCLASS(MinimalAPI, transient, notplaceable, config=Engine)
class AOnlineBeaconClient : public AOnlineBeacon
{
	GENERATED_UCLASS_BODY()

	//~ Begin AActor Interface
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual bool UseShortConnectTimeout() const override;
	UE_API virtual void OnNetCleanup(UNetConnection* Connection) override;
	UE_API virtual const AActor* GetNetOwner() const override;
	UE_API virtual UNetConnection* GetNetConnection() const override;
	UE_API virtual bool DestroyNetworkActorHandled() override;
	//~ End AActor Interface

	//~ Begin FNetworkNotify Interface
	UE_API virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch) override;
	//~ End FNetworkNotify Interface

	//~ Begin OnlineBeacon Interface
	UE_API virtual void OnFailure() override;
	UE_API virtual void DestroyBeacon() override;
	//~ End OnlineBeacon Interface
	
	/**
	 * Initialize the client beacon with connection endpoint
	 *	Creates the net driver and attempts to connect with the destination
	 *
	 * @param Params Parameters used for initialization of the client.
	 *
	 * @return true if connection is being attempted, false otherwise
	 */
	UE_API bool InitClient(const FOnlineBeaconClientInitParams& Params);

	/**
	 * Initialize the client beacon with connection endpoint
	 *	Creates the net driver and attempts to connect with the destination
	 *
	 * @param URL destination
	 *
	 * @return true if connection is being attempted, false otherwise
	 */
	UE_API bool InitClient(FURL& URL);

	/**
	 * Sets the encryption data that will be used for server connections.
	 *
	 * @param InEncryptionData the encryption data to use, including key and identifier
	 */
	UE_API void SetEncryptionData(const FEncryptionData& InEncryptionData);

	/**
	 * Send the packet for triggering the initial join
	 */
	UE_API void SendInitialJoin();

	/**
	 * Each beacon must have a unique type identifier
	 *
	 * @return string representing the type of beacon 
	 */
	UE_API FString GetBeaconType() const;
	
	/**
	 * A connection has been made and RPC/replication can begin
	 */
	virtual void OnConnected() {};

	/**
	 * Delegate triggered on failures to connect to a host beacon
	 */
	FOnHostConnectionFailure& OnHostConnectionFailure() { return HostConnectionFailure; }

	/**
	 * Set the connection state
	 * Higher level than the net connection because of the handshaking of the actors
	 *
	 * @return connection state of the beacon
	 */
	UE_API void SetConnectionState(EBeaconConnectionState NewConnectionState);

	/**
	 * Get the unique id of the user on this connection (server side only)
	 *
	 * @return unique id of the user on this connection
	 */
	UE_API const FUniqueNetIdRepl& GetUniqueId() const;

	/**
	 * Get the login options to send the server during login.
	 * Typically this is expected to be in the format that UGameplayStatics::ParseOption can parse, i.e. Key1=Value1?Key2=Value2?Key3=Value3
	 *
	 * @param PlayerId unique id of the user on this connection
	 * @return Login options value to send to server
	 */
	UE_API virtual FString GetLoginOptions(const FUniqueNetIdRepl& PlayerId);

	/**
	 * Get the connection state
	 * Higher level than the net connection because of the handshaking of the actors
	 *
	 * @return connection state of the beacon
	 */
	UE_API EBeaconConnectionState GetConnectionState() const;

	/**
	 * Get the owner of this beacon actor, some host that is listening for connections
	 * (server side only, clients have no access)
	 *
	 * @return owning host of this actor
	 */
	UE_API AOnlineBeaconHostObject* GetBeaconOwner() const;
	
	/**
	 * Set the owner of this beacon actor, some host that is listening for connections
	 * (server side only, clients have no access)
	 *
	 * @return owning host of this actor
	 */
	UE_API void SetBeaconOwner(AOnlineBeaconHostObject* InBeaconOwner);

	/**
	 * Associate this beacon with a network connection
	 *
	 * @param NetConnection connection that the beacon will communicate over
	 */
	virtual void SetNetConnection(UNetConnection* NetConnection)
	{
		BeaconConnection = NetConnection;
	}

protected:

	/** Owning beacon host of this beacon actor (server only) */
	UPROPERTY()
	TObjectPtr<AOnlineBeaconHostObject> BeaconOwner;

	/** Network connection associated with this beacon client instance */
	UPROPERTY()
	TObjectPtr<UNetConnection> BeaconConnection;

	/** State of the connection */
	UPROPERTY()
	EBeaconConnectionState ConnectionState;

	/** Delegate for host beacon connection failures */
	FOnHostConnectionFailure HostConnectionFailure;

	/** Handle for efficient management of OnFailure timer */
	FTimerHandle TimerHandle_OnFailure;

private:

	/** Get the auth ticket from OnlineSubsystem for the specified player id */
	UE_API FString GetAuthTicketInternal(const FUniqueNetIdRepl& PlayerId);

	/** Encryption data used when connecting to servers. */
	FEncryptionData EncryptionData;

	/**
	 * Setup the connection for encryption with a given key
	 * All future packets are expected to be encrypted
	 *
	 * @param Response response from the game containing its encryption key or an error message
	 * @param WeakConnection the connection related to the encryption request
	 */
	UE_API void FinalizeEncryptedConnection(const FEncryptionKeyResponse& Response, TWeakObjectPtr<UNetConnection> WeakConnection);

	/**
	 * Called on the server side to open up the actor channel that will allow RPCs to occur
	 * (DO NOT OVERLOAD, implement OnConnected() instead)
	 */
	UFUNCTION(client, reliable)
	UE_API void ClientOnConnected();

	friend class AOnlineBeaconHost;
	friend class AOnlineBeaconHostObject;
};

inline const TCHAR* ToString(EBeaconConnectionState Value)
{
	switch (Value)
	{
	case EBeaconConnectionState::Invalid:
	{
		return TEXT("Invalid");
	}
	case EBeaconConnectionState::Closed:
	{
		return TEXT("Closed");
	}
	case EBeaconConnectionState::Pending:
	{
		return TEXT("Pending");
	}
	case EBeaconConnectionState::Open:
	{
		return TEXT("Open");
	}
	}
	return TEXT("");
}

#undef UE_API
