// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeacon.h"
#include "Net/Core/Connection/NetCloseResult.h"
#include "OnlineBeaconHost.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class FUniqueNetId;

class AOnlineBeaconClient;
class AOnlineBeaconHostObject;
class FInBunch;
class UNetConnection;
struct FEncryptionKeyResponse;
struct FOnlineError;

/**
 * Main actor that listens for side channel communication from another Unreal Engine application
 *
 * The AOnlineBeaconHost listens for connections to route to a registered AOnlineBeaconHostObject 
 * The AOnlineBeaconHostObject is responsible for spawning the server version of the AOnlineBeaconClient
 * The AOnlineBeaconHost pairs the two client actors, verifies the validity of the exchange, and accepts/continues the connection
 */
UCLASS(MinimalAPI, transient, notplaceable, config=Engine)
class AOnlineBeaconHost : public AOnlineBeacon
{
	using FNetCloseResult = UE::Net::FNetCloseResult;

	GENERATED_UCLASS_BODY()

public:
	/** Configured listen port for this beacon host */
	UPROPERTY(Config)
	int32 ListenPort;

	/**
	 * Whether to configure the listening socket to allow reuse of the address and port. If this is true, be sure no other
	 * servers can run on the same port, otherwise this can lead to undefined behavior since packets will go to two servers.
	 */
	UPROPERTY(Config)
	bool bReuseAddressAndPort = false;

	//~ Begin AActor Interface
	UE_API virtual void OnNetCleanup(UNetConnection* Connection) override;
	//~ End AActor Interface

	//~ Begin OnlineBeacon Interface
	UE_API virtual void HandleNetworkFailure(UWorld* World, UNetDriver* NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString) override;
	//~ End OnlineBeacon Interface

	//~ Begin FNetworkNotify Interface
	UE_API virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch) override;
	//~ End FNetworkNotify Interface

	/**
	 * Initialize the host beacon on a specified port
	 *	Creates the net driver and begins listening for connections
	 *
	 * @return true if host was setup correctly, false otherwise
	 */
	UE_API virtual bool InitHost();

	/**
	 * Get the listen port for this beacon
	 *
	 * @return beacon listen port
	 */
	virtual int32 GetListenPort() { return ListenPort; }

	/**
	 * Register a beacon host and its client actor factory
	 *
	 * @param NewHostObject new 
	 */
	UE_API virtual void RegisterHost(AOnlineBeaconHostObject* NewHostObject);

	/**
	 * Unregister a beacon host, making future connections of a given type unresponsive
	 *
	 * @param BeaconType type of beacon host to unregister
	 */
	UE_API virtual void UnregisterHost(const FString& BeaconType);

	/**
	 * Get the host responsible for a given beacon type
	 *
	 * @param BeaconType type of beacon host 
	 * 
	 * @return BeaconHost for the given type or NULL if that type is not registered
	 */
	UE_API AOnlineBeaconHostObject* GetHost(const FString& BeaconType);

	/**
	 * Disconnect a given client from the host
	 *
	 * @param ClientActor the beacon client to disconnect
	 */
	UE_API void DisconnectClient(AOnlineBeaconClient* ClientActor);

	/**
	 * Get a client beacon actor for a given connection
	 *
	 * @param Connection connection of interest
	 *
	 * @return client beacon actor that owns the connection
	 */
	UE_API virtual AOnlineBeaconClient* GetClientActor(UNetConnection* Connection);

	/**
	 * Remove a client beacon actor from the list of active connections
	 *
	 * @param ClientActor client beacon actor to remove
	 */
	UE_API virtual void RemoveClientActor(AOnlineBeaconClient* ClientActor);

protected:
	/** Set this to true if you require clients to negotiate auth prior to joining the beacon */
	UPROPERTY(Config)
	bool bAuthRequired = false;

	UPROPERTY(Config)
	uint32 MaxAuthTokenSize = 1024;

	/**
	 * Delegate executed when user authentication has completed.
	 *
	 * @param OnlineError the result of the operation
	 */
	DECLARE_DELEGATE_OneParam(FOnAuthenticationVerificationCompleteDelegate, const FOnlineError& /*OnlineError*/);

	/**
	 * Start verifying an authentication token for a connection.
	 * OnAuthenticationVerificationComplete must be called to complete authentication verification.
	 *
	 * @param PlayerId net id of player to authenticate.
	 * @Param LoginOptions all options passed as part of the Login request.
	 * @param AuthenticationToken token to use for verification.
	 * @Param OnComplete delegate to call once the request for authentication has completed
	 */
	UE_API virtual bool StartVerifyAuthentication(const FUniqueNetId& PlayerId, const FString& LoginOptions, const FString& AuthenticationToken, const FOnAuthenticationVerificationCompleteDelegate& OnComplete);

	/**
	 * Verify user authentication once the beacon type is known.
	 *
	 * @param PlayerId net id of player to authenticate.
	 * @Param BeaconType type of beacon to verify the authentication for.
	 */
	UE_API virtual bool VerifyJoinForBeaconType(const FUniqueNetId& PlayerId, const FString& BeaconType);

private:
	/**
	 * Event which must be signaled to complete an authentication verification request.
	 *
	 * @param NetConnection network connection associated with the authentication challenge.
	 * @param Error result of the operation.
	 */
	UE_API void OnAuthenticationVerificationComplete(UNetConnection* Connection, const FOnlineError& Error);

protected:
	UE_API FString GetDebugName(UNetConnection* Connection = nullptr);

private:
	/** Connection states used to check against misbehaving connections. */
	struct FConnectionState
	{
		FConnectionState(const AOnlineBeaconHost& InBeaconHost) :
			BeaconHost(InBeaconHost)
		{
		}

		~FConnectionState();

		const AOnlineBeaconHost& BeaconHost;
		FTimerHandle FinishHandshakeTimerHandle;
		bool bHasSentHello = false;
		bool bHasSentChallenge = false;
		bool bHasSentLogin = false;
		bool bHasSentWelcome = false;
		bool bHasSetNetspeed = false;
		bool bHasAuthenticated = false;
		bool bHasJoined = false;
		bool bHasCompletedAck = false;
	};
	TMap<UNetConnection*,FConnectionState> ConnectionState;

	/** List of all client beacon actors with active connections */
	UPROPERTY()
	TArray<TObjectPtr<AOnlineBeaconClient>> ClientActors;

	/** Sends the welcome control message to the client. Used as a delegate if encryption is being negotiated. */
	UE_API void OnHelloSequenceComplete(UNetConnection* Connection);
	UE_API void OnEncryptionResponse(const FEncryptionKeyResponse& Response, TWeakObjectPtr<UNetConnection> WeakConnection);
	UE_API void OnConnectionClosed(UNetConnection* Connection);

	UE_API bool HandleControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch);
	UE_API void FinishHandshake(UNetConnection* Connection, FString BeaconType);
	UE_API void SendFailurePacket(UNetConnection* Connection, FNetCloseResult&& CloseReason, const FText& ErrorText);

	UE_API void CloseHandshakeConnection(UNetConnection* Connection);

	UE_API bool GetConnectionDataForUniqueNetId(const FUniqueNetId& UniqueNetId, UNetConnection*& OutConnection, FConnectionState*& OutConnectionState);

	/** Delegate to route a connection attempt to the appropriate beacon host, by type */
	DECLARE_DELEGATE_RetVal_OneParam(AOnlineBeaconClient*, FOnBeaconSpawned, UNetConnection*);
	UE_API FOnBeaconSpawned& OnBeaconSpawned(const FString& BeaconType);

	/** Mapping of beacon types to the OnBeaconSpawned delegates */
	TMap<FString, FOnBeaconSpawned> OnBeaconSpawnedMapping;

	/** Delegate to route a connection event to the appropriate beacon host, by type */
	DECLARE_DELEGATE_TwoParams(FOnBeaconConnected, AOnlineBeaconClient*, UNetConnection*);
	UE_API FOnBeaconConnected& OnBeaconConnected(const FString& BeaconType);

	/** Mapping of beacon types to the OnBeaconConnected delegates */
	TMap<FString, FOnBeaconConnected> OnBeaconConnectedMapping;
};

#undef UE_API
