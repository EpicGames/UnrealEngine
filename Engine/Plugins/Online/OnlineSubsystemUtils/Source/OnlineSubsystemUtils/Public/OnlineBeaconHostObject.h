// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeacon.h"
#include "OnlineBeaconHostObject.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class AOnlineBeaconClient;
class UNetConnection;

/**
 * Base class for any unique beacon connectivity, paired with an AOnlineBeaconClient implementation 
 *
 * By defining a beacon type and implementing the ability to spawn unique AOnlineBeaconClients, any two instances of the engine
 * can communicate with each other without officially connecting through normal Unreal networking
 */
UCLASS(MinimalAPI, transient, config=Engine, notplaceable)
class AOnlineBeaconHostObject : public AActor
{
	GENERATED_UCLASS_BODY()

	/** @return the name of the net driver associated with this object */
	UE_API FName GetNetDriverName() const;

	/** Get the state of the beacon (accepting/rejecting connections) */
	UE_API EBeaconState::Type GetBeaconState() const;

	/** Get the type of beacon supported by this host */
	const FString& GetBeaconType() const { return BeaconTypeName; }

	/** Simple accessor for client beacon actor class */
	TSubclassOf<AOnlineBeaconClient> GetClientBeaconActorClass() const { return ClientBeaconActorClass; }

	/**
	 * Each beacon host must be able to spawn the appropriate client beacon actor to communicate with the initiating client
	 *
	 * @return new client beacon actor that this beacon host knows how to communicate with
	 */
	UE_API virtual AOnlineBeaconClient* SpawnBeaconActor(UNetConnection* ClientConnection);

	/**
	 * Delegate triggered when a new client connection is made
	 *
	 * @param NewClientActor new client beacon actor
	 * @param ClientConnection new connection established
	 */
	UE_API virtual void OnClientConnected(AOnlineBeaconClient* NewClientActor, UNetConnection* ClientConnection);

	/**
	 * Disconnect a given client from the host
	 *
	 * @param ClientActor the beacon client to disconnect
	 */
	UE_API virtual void DisconnectClient(AOnlineBeaconClient* ClientActor);

	/**
	 * Notification that a client has been disconnected from the host in some way (timeout, client initiated, etc)
	 *
	 * @param LeavingClientActor actor that has disconnected
	 */
	UE_API virtual void NotifyClientDisconnected(AOnlineBeaconClient* LeavingClientActor);

	/**
	 * Called when this class is unregistered by the beacon host 
	 * Do any necessary cleanup.
	 */
	UE_API virtual void Unregister();

	/**
	 * @return the number of connections currently with this beacon host
	 */
	int32 GetNumClientActors() const { return ClientActors.Num(); }

protected:

	/** Custom name for this beacon */
	UPROPERTY(Transient)
	FString BeaconTypeName;

	/** Class reference for spawning client beacon actor */
	UPROPERTY()
	TSubclassOf<AOnlineBeaconClient> ClientBeaconActorClass;

	/** List of all client beacon actors with active connections */
	UPROPERTY()
	TArray<TObjectPtr<AOnlineBeaconClient>> ClientActors;
};

#undef UE_API
