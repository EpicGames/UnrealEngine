// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/OnlineReplStructs.h"
#include "OnlineBeaconReservation.generated.h"

/** A single player reservation */
USTRUCT()
struct FPlayerReservation
{
	GENERATED_USTRUCT_BODY()

		FPlayerReservation()
		: bAllowCrossplay(false)
		, ElapsedTime(0.0f)
	{}

	/** Unique id for this reservation */
	UPROPERTY(Transient)
		FUniqueNetIdRepl UniqueId;

	/** Info needed to validate user credentials when joining a server */
	UPROPERTY(Transient)
		FString ValidationStr;

	/** Platform this user is on */
	UPROPERTY(Transient)
		FString Platform;

	/** Does this player opt in to crossplay */
	UPROPERTY(Transient)
		bool bAllowCrossplay;

	/** Elapsed time since player made reservation and was last seen */
	UPROPERTY(Transient)
		float ElapsedTime;
};
