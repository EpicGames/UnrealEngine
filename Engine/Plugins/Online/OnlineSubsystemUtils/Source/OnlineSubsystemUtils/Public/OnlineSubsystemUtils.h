// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "OnlineSubsystem.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemUtilsModule.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Features/IModularFeatures.h"
#include "Interfaces/OnlineTitleFileInterface.h"
#include "Interfaces/OnlineUserCloudInterface.h"
#include "Interfaces/VoiceInterface.h"

class UWorld;
class UAudioComponent;
class UVoipListenerSynthComponent;
struct FVoiceSettings;

#ifdef ONLINESUBSYSTEMUTILS_API

/** @return an initialized audio component specifically for use with VoIP */
ONLINESUBSYSTEMUTILS_API UAudioComponent* CreateVoiceAudioComponent(uint32 SampleRate, int32 NumChannels);

/** @return an initialized Synth component specifically for use with VoIP */
ONLINESUBSYSTEMUTILS_API UVoipListenerSynthComponent* CreateVoiceSynthComponent(UWorld* World, uint32 SampleRate);

/** Updates InSynthComponent based on InSettings. */
ONLINESUBSYSTEMUTILS_API void ApplyVoiceSettings(UVoipListenerSynthComponent* InSynthComponent, const FVoiceSettings& InSettings);

/** @return the world associated with a named online subsystem instance */
ONLINESUBSYSTEMUTILS_API UWorld* GetWorldForOnline(FName InstanceName);

/**
 * Try to retrieve the active listen port for a server session
 *
 * @param InstanceName online subsystem instance to query
 *
 * @return the port number currently associated with the GAME net driver
 */
ONLINESUBSYSTEMUTILS_API int32 GetPortFromNetDriver(FName InstanceName);

ONLINESUBSYSTEMUTILS_API int32 GetClientPeerIp(FName InstanceName, const FUniqueNetId& UserId);

#if WITH_ENGINE
/**
 * Get a 64bit bit base id for a chat room 
 * <32bit IP Addr> | <EmptySpace> | <24bit ProcessId>
 *
 * @param World world for context
 *
 * @return 64bit base id for a voice chat room
 */
ONLINESUBSYSTEMUTILS_API uint64 GetBaseVoiceChatTeamId(const UWorld* World);
/**
 * Get a 64bit bit final id for a chat room 
 * <32bit IP Addr> | <8bit team index> | <24bit ProcessId>
 *
 * @param VoiceChatIdBase previously retrieved base id
 * @param TeamIndex index for a given team needing a voice chat id
 *
 * @return 64bit id for a voice chat room
 */
ONLINESUBSYSTEMUTILS_API uint64 GetVoiceChatTeamId(uint64 VoiceChatIdBase, uint8 TeamIndex);
#endif

#endif

/**
 * Interface class for various online utility functions
 */
class IOnlineSubsystemUtils 
{
protected:
	/** Hidden on purpose */
	IOnlineSubsystemUtils() {}

public:

	virtual ~IOnlineSubsystemUtils() = default;

	/**
	 * Gets an FName that uniquely identifies an instance of OSS
	 *
	 * @param WorldContext the worldcontext associated with a particular subsystem
	 * @param Subsystem the name of the subsystem
	 * @return an FName of format Subsystem:Context_Id in PlayInEditor or Subsystem everywhere else
	 */
	virtual FName GetOnlineIdentifier(const FWorldContext& WorldContext, const FName Subsystem = NAME_None) const = 0;

	/**
	 * Gets an FName that uniquely identifies an instance of OSS
	 *
	 * @param World the world to use for context
	 * @param Subsystem the name of the subsystem
	 * @return an FName of format Subsystem:Context_Id in PlayInEditor or Subsystem everywhere else
	 */
	virtual FName GetOnlineIdentifier(const UWorld* World, const FName Subsystem = NAME_None) const = 0;

	/**
	 * Create a TRANSPORT LAYER unique id
	 * NOTE: Do NOT Use this for anything other than replication to non native platforms
	 * This is NOT a shortcut for creating unique ids
	 * 
	 * @param Str string form an opaque unique net id
	 * @param Type name of the online subsystem this unique id belongs to
	 *
	 * @return unique net id in "transport" format
	 */
	virtual FUniqueNetIdPtr CreateForeignUniqueNetId(const FString& Str, FName Type) const = 0;

	/** 
	 * Return the replication hash for a given subsystem
	 *
	 * @param InSubsystemName name of subsystem to retrieve hash from
	 *
	 * @return replication hash, or 0 if invalid/unknown
	 */
	virtual uint8 GetReplicationHashForSubsystem(FName SubsystemName) const = 0;

	/**
	 * Return the name of the online subsystem associated with this hash
	 *
	 * @param InHash replication hash for an online subsystem
	 *
	 * @return name of subsystem this hash belongs to
	 */
	virtual FName GetSubsystemFromReplicationHash(uint8 InHash) const = 0;

	/**
	 * Bind a notification delegate when any subsystem external UI is opened/closed
	 * *NOTE* there is only meant to be one delegate needed for this, game code should bind manually
	 *
	 * @param OnExternalUIChangeDelegate delegate fired when the external UI is opened/closed
	 */
	virtual void SetEngineExternalUIBinding(const FOnExternalUIChangeDelegate& OnExternalUIChangeDelegate) = 0;

#if WITH_EDITOR
	/**
	 * Play in Editor settings
	 */

	/** @return true if the default platform supports logging in for Play In Editor (PIE) */
	virtual bool SupportsOnlinePIE() const = 0;
	/** Enable/Disable online PIE at runtime */
	virtual void SetShouldTryOnlinePIE(bool bShouldTry) = 0;
	/** @return true if the user has enabled logging in for Play In Editor (PIE) */
	virtual bool IsOnlinePIEEnabled() const = 0;
	/** @return the number of logins the user has setup for Play In Editor (PIE) */
	virtual int32 GetNumPIELogins() const = 0;
	/** @return the array of valid credentials the user has setup for Play In Editor (PIE) */
	virtual void GetPIELogins(TArray<FOnlineAccountCredentials>& Logins) = 0;

	/**
	 * Static check for whether online PIE is enabled. Safe to call at any point during startup —
	 * uses CDO if UObjectInitialized(), falls back to reading INI + command line auth credentials.
	 */
	static ONLINESUBSYSTEMUTILS_API bool IsOnlinePIEEnabledStatic();
#endif // WITH_EDITOR
};

/** Macro to handle the boilerplate of accessing the proper online subsystem and getting the requested interface (UWorld version) */
#define IMPLEMENT_GET_INTERFACE(InterfaceType) \
static IOnline##InterfaceType##Ptr Get##InterfaceType##Interface(const UWorld* World, const FName SubsystemName = NAME_None) \
{ \
	IOnlineSubsystem* OSS = Online::GetSubsystem(World, SubsystemName); \
	return (OSS == NULL) ? NULL : OSS->Get##InterfaceType##Interface(); \
} \
static IOnline##InterfaceType##Ptr Get##InterfaceType##InterfaceChecked(const UWorld* World, const FName SubsystemName = NAME_None) \
{ \
	IOnlineSubsystem* OSS = Online::GetSubsystem(World, SubsystemName); \
	check(OSS); \
	IOnline##InterfaceType##Ptr InterfacePtr = OSS->Get##InterfaceType##Interface(); \
	check(InterfacePtr.IsValid()); \
	return InterfacePtr; \
} 

namespace Online
{
	/** @return the single instance of the online subsystem utils interface */
	static IOnlineSubsystemUtils* GetUtils()
	{
		static const FName OnlineSubsystemModuleName = TEXT("OnlineSubsystemUtils");
		FOnlineSubsystemUtilsModule* OSSUtilsModule = FModuleManager::GetModulePtr<FOnlineSubsystemUtilsModule>(OnlineSubsystemModuleName);
		if (OSSUtilsModule != nullptr)
		{
			return OSSUtilsModule->GetUtils();
		}

		return nullptr;
	}

	/**
	 * Wrapper for IModularFeatures::IsModularFeatureAvailable and IModularFeatures::GetModularFeature
	 * @param Type name of the modular feature
	 * @return pointer to the modular feature if it is available
	 */
	template< typename TModularFeature >
	inline TModularFeature* GetModularFeature( const FName Type )
	{
		TModularFeature* Feature = nullptr;
		if (IModularFeatures::Get().IsModularFeatureAvailable(Type))
		{
			Feature = &IModularFeatures::Get().GetModularFeature<TModularFeature>(Type);
		}
		return Feature;
	}

	/** 
	 * Get the online subsystem for a given service
	 *
	 * @param World the world to use for context
	 * @param SubsystemName - Name of the requested online service
	 *
	 * @return pointer to the appropriate online subsystem
	 */
	static IOnlineSubsystem* GetSubsystem(const UWorld* World, const FName& SubsystemName = NAME_None)
	{
#if UE_EDITOR // at present, multiple worlds are only possible in the editor
		FName Identifier = SubsystemName; 
		if (World != NULL)
		{
			IOnlineSubsystemUtils* Utils = GetUtils();
			Identifier = Utils->GetOnlineIdentifier(World, SubsystemName);
		}

		return IOnlineSubsystem::Get(Identifier); 
#else
		return IOnlineSubsystem::Get(SubsystemName); 
#endif
	}

	/** 
	 * Determine if the subsystem module for a given interface is already loaded.  Note that modules do not typically initialize the subsystem when the module is loaded.
	 *
	 * @param World the world to use for context
	 * @param SubsystemName name of the requested online service
	 *
	 * @return true if module for the subsystem is loaded
	 */
	static bool IsLoaded(const UWorld* World, const FName& SubsystemName = NAME_None)
	{
#if UE_EDITOR // at present, multiple worlds are only possible in the editor
		FName Identifier = SubsystemName;
		if (World != NULL)
		{
			FWorldContext& CurrentContext = GEngine->GetWorldContextFromWorldChecked(World);
			if (CurrentContext.WorldType == EWorldType::PIE)
			{
				Identifier = FName(*FString::Printf(TEXT("%s:%s"), SubsystemName != NAME_None ? *SubsystemName.ToString() : TEXT(""), *CurrentContext.ContextHandle.ToString()));
			}
		}

		return IOnlineSubsystem::IsLoaded(SubsystemName);
#else
		return IOnlineSubsystem::IsLoaded(SubsystemName);
#endif
	}
	
	/**
	 * Determine if the subsystem for a given interface has been instanced
	 *
	 * @param World the world to use for context
	 * @param SubsystemName name of the requested online service
	 *
	 * @return true if the subsystem has been instanced
	 */
	static bool DoesInstanceExist(const UWorld* World, const FName& SubsystemName = NAME_None)
	{
#if UE_EDITOR // at present, multiple worlds are only possible in the editor
		FName Identifier = SubsystemName;
		if (World != NULL)
		{
			FWorldContext& CurrentContext = GEngine->GetWorldContextFromWorldChecked(World);
			if (CurrentContext.WorldType == EWorldType::PIE)
			{
				Identifier = FName(*FString::Printf(TEXT("%s:%s"), SubsystemName != NAME_None ? *SubsystemName.ToString() : TEXT(""), *CurrentContext.ContextHandle.ToString()));
			}
		}
		
		return IOnlineSubsystem::DoesInstanceExist(SubsystemName);
#else
		return IOnlineSubsystem::DoesInstanceExist(SubsystemName);
#endif
	}
	
	/** Reimplement all the interfaces of Online.h with support for UWorld accessors */
	IMPLEMENT_GET_INTERFACE(Session);
	IMPLEMENT_GET_INTERFACE(Party);
	IMPLEMENT_GET_INTERFACE(Chat);
	IMPLEMENT_GET_INTERFACE(Friends);
	IMPLEMENT_GET_INTERFACE(User);
	IMPLEMENT_GET_INTERFACE(SharedCloud);
	IMPLEMENT_GET_INTERFACE(UserCloud);
	IMPLEMENT_GET_INTERFACE(Voice);
	IMPLEMENT_GET_INTERFACE(ExternalUI);
	IMPLEMENT_GET_INTERFACE(Time);
	IMPLEMENT_GET_INTERFACE(Identity);
	IMPLEMENT_GET_INTERFACE(TitleFile);
	IMPLEMENT_GET_INTERFACE(Entitlements);
	IMPLEMENT_GET_INTERFACE(Leaderboards);
	IMPLEMENT_GET_INTERFACE(Achievements);
	IMPLEMENT_GET_INTERFACE(Presence);
	IMPLEMENT_GET_INTERFACE(Events);
	IMPLEMENT_GET_INTERFACE(Stats);
	IMPLEMENT_GET_INTERFACE(Purchase);
}

#undef IMPLEMENT_GET_INTERFACE
