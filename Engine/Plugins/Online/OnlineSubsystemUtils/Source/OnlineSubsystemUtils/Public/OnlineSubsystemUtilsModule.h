// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class IOnlineSubsystemUtils;
namespace UE::Online {
	class IOnlineServicesEngineUtils;
}

/**
 * Online subsystem utils module class
 * Misc functionality where dependency on the engine code is allowed (OnlineSubsystem is not allowed to require engine dependencies)
 */
class FOnlineSubsystemUtilsModule : public IModuleInterface
{
public:

	FOnlineSubsystemUtilsModule();
	virtual ~FOnlineSubsystemUtilsModule();

	/** @return the singleton utility interface */
	IOnlineSubsystemUtils* GetUtils() const { return SubsystemUtils.Get(); }

	/** @return the singleton utility interface */
	UE::Online::IOnlineServicesEngineUtils* GetServicesUtils() const { return OnlineServicesUtils.Get(); }

	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:

	/** Online subsystem utilities singleton */
	TUniquePtr<IOnlineSubsystemUtils> SubsystemUtils;

	/** Online Services (OSSv2) utilities singleton */
	TUniquePtr<UE::Online::IOnlineServicesEngineUtils> OnlineServicesUtils;
};
