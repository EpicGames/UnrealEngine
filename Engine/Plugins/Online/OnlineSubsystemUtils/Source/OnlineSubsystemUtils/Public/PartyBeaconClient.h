// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PartyBeaconState.h"
#include "TimerManager.h"
#include "OnlineBeaconClient.h"

#include "PartyBeaconClient.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FPlayerReservation;

class FOnlineSessionSearchResult;

/**
 * Types of reservation requests that can be made by this beacon
 */
UENUM()
enum class EClientRequestType : uint8
{
	/** None pending */
	NonePending,
	/** Make a reservation with an existing session */
	ExistingSessionReservation,
	/** Make an update to an existing reservation */
	ReservationUpdate,
	/** Reservation to configure an empty server  */
	EmptyServerReservation,
	/** Simple reconnect (checks for existing reservation) */
	Reconnect,
	/** Abandon the reservation beacon (game specific handling)*/
	Abandon,
	/** Remove members from an existing reservation */
	ReservationRemoveMembers,
	/** Add new reservation or Update existing one*/
	AddOrUpdateReservation,
};

inline const TCHAR* ToString(EClientRequestType RequestType)
{
	switch (RequestType)
	{
	case EClientRequestType::NonePending:
	{
		return TEXT("No Request Pending");
	}
	case EClientRequestType::ExistingSessionReservation:
	{
		return TEXT("Existing Session Reservation");
	}
	case EClientRequestType::ReservationUpdate:
	{
		return TEXT("Reservation Update");
	}
	case EClientRequestType::EmptyServerReservation:
	{
		return TEXT("Empty Server Reservation");
	}
	case EClientRequestType::Reconnect:
	{
		return TEXT("Reconnect Only");
	}
	case EClientRequestType::ReservationRemoveMembers:
	{
		return TEXT("Reservation Remove Members");
	}
	}
	return TEXT("");
}

/**
 * Delegate triggered when a response from the party beacon host has been received
 *
 * @param ReservationResponse response from the server
 */
DECLARE_DELEGATE_OneParam(FOnReservationRequestComplete, EPartyReservationResult::Type /** ReservationResponse */);

/**
 * Delegate triggered when the host indicated a reservation count has changed
 *
 * @param NumRemaining number of slots remaining in the session
 */
DECLARE_DELEGATE_OneParam(FOnReservationCountUpdate, int32 /** NumRemaining */);

/** Delegate triggered when the host indicated the reservation is full */
DECLARE_DELEGATE(FOnReservationFull);

/**
 * A beacon client used for making reservations with an existing game session
 */
UCLASS(MinimalAPI, transient, notplaceable, config=Engine)
class APartyBeaconClient : public AOnlineBeaconClient
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin AOnlineBeaconClient Interface
	UE_API virtual void OnConnected() override;
	UE_API virtual void OnFailure() override;
	//~ End AOnlineBeaconClient Interface

	/**
	 * Sends a request to the remote host to allow the specified members to reserve space
	 * in the host's session. Note this request is async.
	 *
	 * @param ConnectInfoStr the URL of the server that the connection will be made to
	 * @param InSessionId Id of the session expected to be found at this destination
	 * @param RequestingPartyLeader the leader of this party that will be joining
	 * @param PartyMembers the list of players that want to reserve space
	 *
	 * @return true if the request able to be sent, false if it failed to send
	 */
	UE_API virtual bool RequestReservation(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PartyMembers);

	/**
	 * Sends a request to the remote host to allow the specified members to reserve space
	 * in the host's session. Note this request is async.
	 *
	 * @param DesiredHost a search result describing the server that the connection will be made to
	 * @param RequestingPartyLeader the leader of this party that will be joining
	 * @param PartyMembers the list of players that want to reserve space
	 *
	 * @return true if the request able to be sent, false if it failed to send
	 */
	UE_API virtual bool RequestReservation(const FOnlineSessionSearchResult& DesiredHost, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PartyMembers);

	/**
	 * Sends an update request to the remote host to append additional members to an existing party
	 * in the host's session. Note this request is async.
	 *
	 *  ** This version is for existing / established connections only, it will not actually attempt a connection **
	 *
	 * @param RequestingPartyLeader the leader of the party that will be updated
	 * @param PlayersToAdd the list of players that will be added to the party
	 *
	 * @return true if the request able to be sent, false if it failed to send
	 */
	UE_API virtual bool RequestReservationUpdate(const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PlayersToAdd, bool bRemovePlayers = false);

	/**
	 * Sends an update request to the remote host to append additional members to an existing party
	 * in the host's session. Note this request is async.
	 *
	 * @param ConnectInfoStr the URL of the server that the connection will be made to
	 * @param InSessionId Id of the session expected to be found at this destination
	 * @param RequestingPartyLeader the leader of the party that will be updated
	 * @param PlayersToAdd the list of players that will be added to the party
	 *
	 * @return true if the request able to be sent, false if it failed to send
	 */
	UE_API virtual bool RequestReservationUpdate(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PlayersToAdd, bool bRemovePlayers = false);

	/**
	 * Sends an update request to the remote host to append additional members to an existing party
	 * in the host's session. Note this request is async.
	 *
	 * @param DesiredHost a search result describing the server that the connection will be made to
	 * @param RequestingPartyLeader the leader of the party that will be updated
	 * @param PlayersToAdd the list of players that will be added to the party
	 *
	 * @return true if the request able to be sent, false if it failed to send
	 */
	UE_API virtual bool RequestReservationUpdate(const FOnlineSessionSearchResult& DesiredHost, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PlayersToAdd, bool bRemovePlayers = false);
	
	/**
	* Sends a request to the remote host. 
	* If there is an existing reservation it will update it in the host's session.
	* Otherwise it will allow the specified members to reserve space in the host's session. 
	* Note this request is async.
	*
	* @param ConnectInfoStr the URL of the server that the connection will be made to
	* @param InSessionId Id of the session expected to be found at this destination
	* @param RequestingPartyLeader the leader of the party that will be updated
	* @param PartyMembers the list of players that will be added to the party
	*
	* @return true if the request able to be sent, false if it failed to send
	*/
	UE_API virtual bool RequestAddOrUpdateReservation(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingPartyLeader, const TArray<FPlayerReservation>& PartyMembers);

	/**
	 * Cancel an existing request to the remote host to revoke allocated space on the server.
	 * Note this request is async.
	 */
	UE_API virtual void CancelReservation();

	/**
	 * Response from the host session after making a reservation request
	 *
	 * @param ReservationResponse response from server
	 */
 	UFUNCTION(client, reliable)
	UE_API virtual void ClientReservationResponse(EPartyReservationResult::Type ReservationResponse);

	/**
	 * Response from the host session after making a cancellation request
	 *
	 * @param ReservationResponse response from server
	 */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientCancelReservationResponse(EPartyReservationResult::Type ReservationResponse);
	
	/**
	 * Response from the host session that the reservation count has changed
	 *
	 * @param NumRemainingReservations number of slots remaining until a full session
	 */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientSendReservationUpdates(int32 NumRemainingReservations);

	/** Response from the host session that the reservation is full */
	UFUNCTION(client, reliable)
	UE_API virtual void ClientSendReservationFull();

	/**
	 * Delegate triggered when a response from the party beacon host has been received
	 *
	 * @return delegate to handle response from the server
	 */
	FOnReservationRequestComplete& OnReservationRequestComplete() { return ReservationRequestComplete; }

	/**
	 * Delegate triggered when the host indicated a reservation count has changed
	 *
	 * @param NumRemaining number of slots remaining in the session
	 */
	FOnReservationCountUpdate& OnReservationCountUpdate() { return ReservationCountUpdate; }

	/** Delegate triggered when the host indicated the reservation is full */
	FOnReservationFull& OnReservationFull() { return ReservationFull; }

	/**
	* @return the pending reservation associated with this beacon client
	*/
	const FPartyReservation& GetPendingReservation() const { return PendingReservation; }
	
protected:

	/** Delegate for reservation request responses */
	FOnReservationRequestComplete ReservationRequestComplete;
	/** Delegate for reservation count updates */
	FOnReservationCountUpdate ReservationCountUpdate;

	/** Delegate for reservation full */
	FOnReservationFull ReservationFull;

	/** Session Id of the destination host */
	UPROPERTY()
	FString DestSessionId;
	/** Pending reservation that will be sent upon connection with the intended host */
	UPROPERTY()
	FPartyReservation PendingReservation;

	/** Type of request currently being handled by this client beacon */
	UPROPERTY()
	EClientRequestType RequestType;

	/** Has the reservation request been delivered */
	UPROPERTY()
	bool bPendingReservationSent;
	/** Has the reservation request been canceled */
	UPROPERTY()
	bool bCancelReservation;

	/** Timer to trigger a cancel reservation request if the server doesn't respond in time */
	FTimerHandle CancelRPCFailsafe;

	/** Timers for delaying various responses (debug) */
	FTimerHandle PendingResponseTimerHandle;
	FTimerHandle PendingCancelResponseTimerHandle;
	FTimerHandle PendingReservationUpdateTimerHandle;
	FTimerHandle PendingReservationFullTimerHandle;

	/** Clear out all the timer handles listed above */
	UE_API void ClearTimers(bool bCallFailSafeIfNeeded);
	/** Delegate triggered if the client doesn't hear from the server in time */
	UE_API void OnCancelledFailsafe();
	/** Delegate triggered when a cancel reservation request is complete */
	UE_API void OnCancelledComplete();

	/** Process a response to our RequestReservation request to the server */
	UE_API void ProcessReservationResponse(EPartyReservationResult::Type ReservationResponse);
	/** Process a response to our CancelReservation request to the server */
	UE_API void ProcessCancelReservationResponse(EPartyReservationResult::Type ReservationResponse);
	/** Process a response from the server with an update to the number of consumed reservations */
	UE_API void ProcessReservationUpdate(int32 NumRemainingReservations);
	/** Process a response from the server that the reservation beacon is full */
	UE_API void ProcessReservationFull();

	/**
	 * Tell the server about the reservation request being made
	 *
	 * @param SessionId expected session id on the other end (must match)
	 * @param Reservation pending reservation request to make with server
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerReservationRequest(const FString& SessionId, const FPartyReservation& Reservation);
	
	/**
	 * Tell the server about the reservation update request being made
	 *
	 * @param SessionId expected session id on the other end (must match)
	 * @param ReservationUpdate pending reservation request to make with server
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerUpdateReservationRequest(const FString& SessionId, const FPartyReservation& ReservationUpdate);

	/**
	 * Tell the server about the reservation add or update request being made
	 *
	 * @param SessionId expected session id on the other end (must match)
	 * @param ReservationUpdate pending reservation request to make with server
	 */

	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerAddOrUpdateReservationRequest(const FString& SessionId, const FPartyReservation& Reservation);

	/**
	 * Tell the server that we are removing members from our reservation
	 *
	 * @param SessionId expected session id on the other end (must match)
	 * @param ReservationUpdate pending reservation request to make with server
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerRemoveMemberFromReservationRequest(const FString& SessionId, const FPartyReservation& ReservationUpdate);

	/**
	 * Tell the server to cancel a pending or existing reservation
	 *
	 * @param PartyLeader id of the party leader for the reservation to cancel
	 */
	UFUNCTION(server, reliable, WithValidation)
	UE_API virtual void ServerCancelReservationRequest(const FUniqueNetIdRepl& PartyLeader);

	/**
	 * Trigger the given delegate at a later time
	 *
	 * @param Delegate function to call at a later date
	 * @param Delay time to wait before calling function
	 *
	 * @return handle in the timer system for this entry
	 */
	UE_API FTimerHandle DelayResponse(FTimerDelegate& Delegate, float Delay);
};

#undef UE_API
