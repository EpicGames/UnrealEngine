// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PartyBeaconState.h"
#include "OnlineBeaconHostObject.h"
#include "PartyBeaconHost.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FPlayerReservation;
struct FUniqueNetIdRepl;

class APartyBeaconClient;

/**
 * Delegate type for handling reservation additions/removals, or full events
 */
DECLARE_DELEGATE(FOnReservationUpdate);

/**
 * Delegate fired when a the beacon host has been told to cancel a reservation
 *
 * @param PartyLeader leader of the canceled reservation
 */
DECLARE_DELEGATE_OneParam(FOnCancelationReceived, const FUniqueNetId&);

/**
* Delegate fired when a the beacon host has added a new player
*
* @param PlayerId Player who reservation has been made for.
*/
DECLARE_DELEGATE_OneParam(FOnNewPlayerAdded, const FPlayerReservation&);

/**
 * Delegate called when the beacon gets any request, allowing the owner to validate players at a higher level (bans,etc)
 * 
 * @param PartyMembers players making up the reservation
 * @return true if these players are ok to join
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnValidatePlayers, const TArray<FPlayerReservation>&);

/**
 * Delegate fired when a the beacon host detects a duplicate reservation
 *
 * @param PartyReservation reservation that is found to be duplicated
 */
DECLARE_DELEGATE_OneParam(FOnDuplicateReservation, const FPartyReservation&);

/**
 * A beacon host used for taking reservations for an existing game session
 */
UCLASS(MinimalAPI, transient, notplaceable, config=Engine)
class APartyBeaconHost : public AOnlineBeaconHostObject
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject interface
	UE_API virtual void PostInitProperties() override;
	//~ End UObject interface

	//~ Begin AActor Interface
	UE_API virtual void Tick(float DeltaTime) override;
	//~ End AActor Interface

	//~ Begin AOnlineBeaconHostObject Interface 
	//~ End AOnlineBeaconHost Interface 

	/**
	 * Initialize the party host beacon
	 *
	 * @param InTeamCount number of teams to make room for
	 * @param InTeamSize size of each team
	 * @param InMaxReservation max number of reservations allowed
	 * @param InSessionName name of session related to the beacon
	 * @param InForceTeamNum team to force players on if applicable (usually only 1 team games)
	 * @param bInEnableRemovalRequests allow clients to remove players from beacon.
	 *
	 * @return true if successful created, false otherwise
	 */
	UE_API virtual bool InitHostBeacon(int32 InTeamCount, int32 InTeamSize, int32 InMaxReservations, FName InSessionName, int32 InForceTeamNum = 0, bool bInEnableRemovalRequests = true);

	/**
	 * Initialize the party host beacon from a previous state/configuration
	 * all existing reservations and configuration values are preserved
     *
	 * @return true if successful created, false otherwise
	 */
	UE_API virtual bool InitFromBeaconState(UPartyBeaconState* PrevState);

	/** 
	 * Reconfigures the beacon for a different team/player count configuration
	 * Allows dedicated server to change beacon parameters after a playlist configuration has been made
	 * Does no real checking against current reservations because we assume the UI wouldn't let 
	 * this party start a gametype if they were too big to fit on a team together
	 * @param InNumTeams the number of teams that are expected to join
	 * @param InNumPlayersPerTeam the number of players that are allowed to be on each team
	 * @param InNumReservations the total number of players to allow to join (if different than team * players)
	 */
	UE_API virtual bool ReconfigureTeamAndPlayerCount(int32 InNumTeams, int32 InNumPlayersPerTeam, int32 InNumReservations);

	/**
	 * Competitive integrity represents how "fair" we need to keep the game. So no friends on other teams, no cross team chat, etc.
	 *
	 * @param bNewCompetitiveIntegrity The new value for Competitive Integrity
	 */
	UE_API virtual void SetCompetitiveIntegrity(bool bNewCompetitiveIntegrity);

	/**
	 * Define the method for assignment new reservations to teams
	 * 
	 * @param NewAssignmentMethod name of the assignment method to use (@see ETeamAssignmentMethod for descriptions)
	 */
	UE_API virtual void SetTeamAssignmentMethod(FName NewAssignmentMethod);

	/**
	 * @return reference to the state of the PartyBeacon
	 */
	UPartyBeaconState* GetState() const { return State; }

	/**
	 * Notify the beacon of a player logout
	 *
	 * @param PlayerId UniqueId of player logging out
	 */
	UE_API virtual void HandlePlayerLogout(const FUniqueNetIdRepl& PlayerId);

	/**
	 * Get the current reservation count inside the beacon
	 * NOTE: This is *NOT* the same as the number of consumed reservations across all parties, just the total number of reservations!
	 *
	 * @return number of reservations inside the beacon (*NOT* number of consumed reservations)
	 */
	virtual int32 GetReservationCount() const { return State->GetReservationCount(); }

	/**
	 * Get the number of reservations actually used/consumed across all parties inside the beacon
	 * 
	 * @return the number of actually used reservations across all parties inside the beacon
	 */
	virtual int32 GetNumConsumedReservations() const { return State->GetNumConsumedReservations(); }

	/**
	 * Get the maximum number of reservations allowed inside the beacon
	 * 
	 * @return The maximum number of reservations allowed inside the beacon
	 */
	virtual int32 GetMaxReservations() const { return State->GetMaxReservations(); }

	/**
	 * Get the number of players on a team across all existing reservations
	 *
	 * @param TeamIdx team to query
	 *
	 * @return number of players on the given team
	 */
	UE_API int32 GetNumPlayersOnTeam(int32 TeamIdx) const;

	/**
	 * Finds the current team assignment of the given player net id.
	 * Looks at the reservation entries to find the player.
	 *
	 * @param PlayerId net id of player to find
	 * @return index of team assignment for the given player, INDEX_NONE if not found
	 */
	UE_API int32 GetTeamForCurrentPlayer(const FUniqueNetId& PlayerId) const;

	/**
	 * Get all the known players on a given team
	 * 
	 * @param TeamIndex valid team index to query
	 * @param TeamMembers [out] array of unique ids found to be on the given team
	 *
	 * @return number of players on team, 0 if invalid
	 */
	UE_API int32 GetPlayersOnTeam(int32 TeamIndex, TArray<FUniqueNetIdRepl>& TeamMembers) const;

	/**
	 * Get the number of teams.
	 *
	 * @return The number of teams.
	 */
	int32 GetNumTeams() const { return State->GetNumTeams(); }

	/**
	 * Get the max number of players per team
	 *
	 * @return The number of players per team
	 */
	virtual int32 GetMaxPlayersPerTeam() const { return State->GetMaxPlayersPerTeam(); }

	/**
	 * Competitive integrity represents how "fair" we need to keep the game. So no friends on other teams, no cross team chat, etc.
	 *
	 * @return Whether or not we should respect competitive integrity
	 */
	virtual bool ShouldRespectCompetitiveIntegrity() const { return State->ShouldRespectCompetitiveIntegrity(); }

	/**
	 * Determine the maximum team size that can be accommodated based
	 * on the current reservation slots occupied.
	 *
	 * @return maximum team size that is currently available
	 */
	virtual int32 GetMaxAvailableTeamSize() const { return State->GetMaxAvailableTeamSize(); }

	/**
	 * Swap the parties between teams, parties must be of same size
	 *
	 * @param PartyLeader party 1 to swap
	 * @param OtherPartyLeader party 2 to swap
	 *
	 * @return true if successful, false otherwise
	 */
	UE_API virtual bool SwapTeams(const FUniqueNetIdRepl& PartyLeader, const FUniqueNetIdRepl& OtherPartyLeader);

	/**
	 * Place a party on a new team, party must fit and team must exist
	 *
	 * @param PartyLeader party to change teams
	 * @param NewTeamNum team to change to
	 *
	 * @return true if successful, false otherwise
	 */
	UE_API virtual bool ChangeTeam(const FUniqueNetIdRepl& PartyLeader, int32 NewTeamNum);

	/**
	 * Does a given player id have an existing reservation
     *
	 * @param PlayerId uniqueid of the player to check
     *
	 * @return true if a reservation exists, false otherwise
	 */
	UE_API virtual bool PlayerHasReservation(const FUniqueNetId& PlayerId) const;

	/**
	 * Obtain player validation string from party reservation entry
	 *
	 * @param PlayerId unique id of player to find validation in an existing reservation
	 * @param OutValidation [out] validation string used when player requested a reservation
	 *
	 * @return true if reservation exists for player
	 *
	 */
	UE_API virtual bool GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const;

	/**
	 * Get the session timeout value to use for the target player
	 *
	 * @param PlayerId unique id of player to find session timeout for
	 *
	 * @return The value to sue as the session timeout
	 */
	UE_API virtual float GetSessionTimeoutSecs(const FUniqueNetIdRepl& PlayerId) const;

	/**
	 * Get the party leader for a given unique id
	 *
	 * @param InPartyMemberId valid party member of some reservation looking for its leader
	 * @param OutPartyLeaderId valid party leader id for the given party member if found, invalid if function returns false
	 *
	 * @return true if a party leader was found for a given user id, false otherwise
	 */
	UE_API bool GetPartyLeader(const FUniqueNetIdRepl& InPartyMemberId, FUniqueNetIdRepl& OutPartyLeaderId) const;

	/**
	 * Attempts to add a party reservation to the beacon
     *
     * @param ReservationRequest reservation attempt
     *
     * @return add attempt result
	 */
	UE_API virtual EPartyReservationResult::Type AddPartyReservation(const FPartyReservation& ReservationRequest);

	/**
	 * Updates an existing party reservation on the beacon
	 * An existing reservation for this party leader must already exist
	 * 
	 * @param ReservationUpdateRequest reservation update information (doesn't need existing party members, simply a delta)
	 * @param bIsRemovingMembers set to true if we wish to remove these members from the reservation instead of adding)
	 *
	 * @return update attempt result
	 */
	UE_API virtual EPartyReservationResult::Type UpdatePartyReservation(const FPartyReservation& ReservationUpdateRequest, bool bIsRemovingMembers);

	/**
	 * Attempts to remove a party reservation from the beacon
     *
     * @param PartyLeader reservation leader
	 */
	UE_API virtual EPartyReservationResult::Type RemovePartyReservation(const FUniqueNetIdRepl& PartyLeader);

	/**
	 * Register user auth ticket with the reservation system
	 * Must have an existing reservation entry
	 *
	 * @param InPartyMemberId id of player logging in 
	 * @param InAuthTicket auth ticket reported by the user
	 */
	UE_API void RegisterAuthTicket(const FUniqueNetIdRepl& InPartyMemberId, const FString& InAuthTicket);

	/**
	 * Update party leader for a given player with the reservation beacon
	 * (needed when party leader leaves, reservation beacon is in a temp/bad state until someone updates this)
	 *
	 * @param InPartyMemberId party member making the update
	 * @param NewPartyLeaderId id of new leader
	 */
	UE_API virtual void UpdatePartyLeader(const FUniqueNetIdRepl& InPartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId);

	/**
	 * Handle a reservation request received from an incoming client
	 *
	 * @param Client client beacon making the request
	 * @param SessionId id of the session that is being checked
	 * @param ReservationRequest payload of request
	 */
	UE_API virtual void ProcessReservationRequest(APartyBeaconClient* Client, const FString& SessionId, const FPartyReservation& ReservationRequest);

	/**
	 * Handle a reservation update request received from an incoming client
	 *
	 * @param Client client beacon making the request
	 * @param SessionId id of the session that is being checked
	 * @param ReservationRequest payload of the update request (existing reservation for party leader required)
	 * @param bRemovingMembers true if this update is removing members, false if adding members.
	 */
	UE_API virtual void ProcessReservationUpdateRequest(APartyBeaconClient* Client, const FString& SessionId, const FPartyReservation& ReservationUpdateRequest, bool bIsRemovingMember);


	/**
	* Handle a reservation add or update request depending on reservation existance received from an incoming client
	*
	* @param Client client beacon making the request
	* @param SessionId id of the session that is being checked
	* @param ReservationRequest payload of the update request (existing reservation for party leader required)
	*/
	UE_API virtual void ProcessReservationAddOrUpdateRequest(APartyBeaconClient* Client, const FString& SessionId, const FPartyReservation& ReservationRequest);
	/**
	 * Handle a reservation cancellation request received from an incoming client
	 *
	 * @param Client client beacon making the request
	 * @param PartyLeader reservation leader
	 */
	UE_API virtual void ProcessCancelReservationRequest(APartyBeaconClient* Client, const FUniqueNetIdRepl& PartyLeader);

	/**
	 * Crossplay
	 */

	/**
	 * @return true if there are cross play restrictions, false otherwise
	 */
	UE_API virtual bool HasCrossplayOptOutReservation() const;

	/**
	 * Get a count of all players for a given platform
	 * 
	 * @param InPlatform platform to get a count for
	 * @param bIncludeMappedPlatforms true if we should include platforms that map to InPlatform (See FPartyBeaconCrossplayPlatformMapping)
	 *
	 * @return number of players for a given platform
	 */
	UE_API virtual int32 GetReservationPlatformCount(const FString& InPlatform, bool bIncludeMappedPlatforms = false) const;

	/**
	 * Delegate fired when a the beacon host detects that all reservations are full
	 */
	FOnReservationUpdate& OnReservationsFull() { return ReservationsFull; }

	/**
	 * Delegate fired when a the beacon host detects a reservation addition/removal
	 */
	FOnReservationUpdate& OnReservationChanged() { return ReservationChanged; }

	/**
	* Delegate fired when the beacon host adds a new player
	*/
	FOnNewPlayerAdded& OnNewPlayerAdded() { return NewPlayerAddedDelegate; }

	/**
	 * Delegate fired when a the beacon host cancels a reservation
	 */
	FOnCancelationReceived& OnCancelationReceived() { return CancelationReceived; }

	/**
	 * Delegate fired when a the beacon detects a duplicate reservation
	 */
	FOnDuplicateReservation& OnDuplicateReservation() { return DuplicateReservation; }

	/**
	 * Delegate called when the beacon gets any request, allowing the owner to validate players at a higher level (bans,etc)
	 */
	FOnValidatePlayers& OnValidatePlayers() { return ValidatePlayers; }

	/**
	 * Output current state of reservations to log
	 */
	UE_API virtual void DumpReservations() const;

	/**
	 * Do party members require validation strings.
	 */
	bool IsValidationStrRequired() const { return bIsValidationStrRequired; }

	UE_API virtual bool IsPlayerActive(const FUniqueNetIdRepl& PlayerId) const;

protected:

	/** State of the beacon */
	UPROPERTY()
	TObjectPtr<UPartyBeaconState> State;

	/** Delegate fired when the beacon indicates all reservations are taken */
	FOnReservationUpdate ReservationsFull;
	/** Delegate fired when the beacon indicates a reservation add/remove */
	FOnReservationUpdate ReservationChanged;
	/** Delegate fired when the beacon indicates a reservation cancellation */
	FOnCancelationReceived CancelationReceived;
	/** Delegate fired when the beacon detects a duplicate reservation */
	FOnDuplicateReservation DuplicateReservation;
	/** Delegate fired when reservation has been added */
	FOnNewPlayerAdded NewPlayerAddedDelegate;
	/** Delegate fired when asking the beacon owner if this reservation is legit */
	FOnValidatePlayers ValidatePlayers;

	/** Do the timeouts below cause a player to be removed from the reservation list */
	UPROPERTY(Config)
	bool bLogoutOnSessionTimeout;
	/** Do party members require validation strings. */
	UPROPERTY(Config)
	bool bIsValidationStrRequired;
	/** Seconds that can elapse before a reservation is removed due to player not being registered with the session */
	UPROPERTY(Transient, Config)
	float SessionTimeoutSecs;
	/** Seconds that can elapse before a reservation is removed due to player not being registered with the session during a travel */
	UPROPERTY(Transient, Config)
	float TravelSessionTimeoutSecs;

	/**
	 * @return the class of the state object inside this beacon
	 */
	virtual TSubclassOf<UPartyBeaconState> GetPartyBeaconHostClass() const { return UPartyBeaconState::StaticClass(); }

	/**
	 * Update clients with current reservation information
	 */
	UE_API void SendReservationUpdates();

	/**
	 * Handle a newly added player
	 *
	 * @param NewPlayer reservation of newly joining player
	 */
	UE_API void NewPlayerAdded(const FPlayerReservation& NewPlayer);

	/**
	 * Handle a newly removed player
	 *
	 * @param NewPlayer reservation of newly removed player
	 */
	UE_API void PlayerRemoved(const FPlayerReservation& RemovedPlayer);

	/**
	 * Does the session match the one associated with this beacon
	 *
	 * @param SessionId session to compare
	 *
	 * @return true if the session matches, false otherwise
	 */
	UE_API bool DoesSessionMatch(const FString& SessionId) const;

	UE_API void NotifyReservationEventNextFrame(FOnReservationUpdate& ReservationEvent);
};

#undef UE_API
