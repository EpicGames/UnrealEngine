// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconReservation.h"
#include "PartyBeaconState.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

ONLINESUBSYSTEMUTILS_API DECLARE_LOG_CATEGORY_EXTERN(LogPartyBeacon, Log, All);

/** The result code that will be returned during party reservation */
UENUM()
namespace EPartyReservationResult
{
	enum Type : int
	{
		/** Empty state. */
		NoResult,
		/** Pending request due to async operation, server will contact client shortly. */
		RequestPending,
		/** An unknown error happened. */
		GeneralError,
		/** All available reservations are booked. */
		PartyLimitReached,
		/** Wrong number of players to join the session. */
		IncorrectPlayerCount,
		/** No response from the host. */
		RequestTimedOut,
		/** Already have a reservation entry for the requesting party leader. */
		ReservationDuplicate,
		/** Couldn't find the party leader specified for a reservation update request. */
		ReservationNotFound,
		/** Space was available and it's time to join. */
		ReservationAccepted,
		/** The beacon is paused and not accepting new connections. */
		ReservationDenied,
		/** Some kind of cross play restriction prevents this party from joining */
		ReservationDenied_CrossPlayRestriction,
		/** This player is banned. */
		ReservationDenied_Banned,
		/** The reservation request was canceled before being sent. */
		ReservationRequestCanceled,
		// The reservation was rejected because it was badly formed
		ReservationInvalid,
		// The reservation was rejected because this was the wrong session
		BadSessionId,
		/** The reservation contains players already in this game */
		ReservationDenied_ContainsExistingPlayers,
		/** The reservation was rejected because it failed additional, project-specific validation */
		ReservationDenied_ValidationFailed
	};
}

namespace EPartyReservationResult
{
	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(EPartyReservationResult::Type SessionType)
	{
		switch (SessionType)
		{
			case NoResult:
			{
				return TEXT("No outstanding request");
			}
			case RequestPending:
			{
				return TEXT("Pending Request");
			}
			case GeneralError:
			{
				return TEXT("General Error");
			}
			case PartyLimitReached:
			{
				return TEXT("Party Limit Reached");
			}
			case IncorrectPlayerCount:
			{
				return TEXT("Incorrect Player Count");
			}
			case RequestTimedOut:
			{
				return TEXT("Request Timed Out");
			}
			case ReservationDuplicate:
			{
				return TEXT("Reservation Duplicate");
			}
			case ReservationNotFound:
			{
				return TEXT("Reservation Not Found");
			}
			case ReservationAccepted:
			{
				return TEXT("Reservation Accepted");
			}
			case ReservationDenied:
			{
				return TEXT("Reservation Denied");
			}
			case ReservationDenied_CrossPlayRestriction:
			{
				return TEXT("Reservation Denied CrossPlayRestriction");
			}
			case ReservationDenied_Banned:
			{
				return TEXT("Reservation Banned");
			}
			case ReservationRequestCanceled:
			{
				return TEXT("Request Canceled");
			}
			case ReservationInvalid:
			{
				return TEXT("Invalid reservation");
			}
			case BadSessionId:
			{
				return TEXT("Bad Session Id");
			}
			case ReservationDenied_ContainsExistingPlayers:
			{
				return TEXT("Reservation Contains Existing Players");
			}
		}
		return TEXT("");
	}

	inline FText GetDisplayString(EPartyReservationResult::Type Response)
	{
		switch (Response)
		{
		case EPartyReservationResult::IncorrectPlayerCount:
		case EPartyReservationResult::PartyLimitReached:
			return NSLOCTEXT("EPartyReservationResult", "FullGame", "Game full");
		case EPartyReservationResult::RequestTimedOut:
			return NSLOCTEXT("EPartyReservationResult", "NoResponse", "No response");
		case EPartyReservationResult::ReservationDenied:
			return NSLOCTEXT("EPartyReservationResult", "DeniedResponse", "Not accepting connections");
		case EPartyReservationResult::ReservationDenied_Banned:
			return NSLOCTEXT("EPartyReservationResult", "BannedResponse", "Player Banned");
		case EPartyReservationResult::GeneralError:
			return NSLOCTEXT("EPartyReservationResult", "GeneralError", "Unknown Error");
		case EPartyReservationResult::ReservationNotFound:
			return NSLOCTEXT("EPartyReservationResult", "ReservationNotFound", "No Reservation");
		case EPartyReservationResult::ReservationAccepted:
			return NSLOCTEXT("EPartyReservationResult", "Accepted", "Accepted");
		case EPartyReservationResult::ReservationDuplicate:
			return NSLOCTEXT("EPartyReservationResult", "DuplicateReservation", "Duplicate reservation detected");
		case EPartyReservationResult::ReservationInvalid:
			return NSLOCTEXT("EPartyReservationResult", "InvalidReservation", "Bad reservation request");
		case EPartyReservationResult::ReservationDenied_ContainsExistingPlayers:
			return NSLOCTEXT("EPartyReservationResult", "ContainsExistingPlayers", "Party members already in session");
		case EPartyReservationResult::NoResult:
		case EPartyReservationResult::BadSessionId:
		default:
			return FText::GetEmpty();
		}
	}
}

namespace ETeamAssignmentMethod
{
	/** Fill smallest team first */
	extern ONLINESUBSYSTEMUTILS_API const FName Smallest;
	/** Optimize for best fit within the number of available reservations */
	extern ONLINESUBSYSTEMUTILS_API const FName BestFit;
	/** Assign random team */
	extern ONLINESUBSYSTEMUTILS_API const FName Random;
	/** Manually assign the team */
	extern ONLINESUBSYSTEMUTILS_API const FName Manual;
}

/** A whole party reservation */
USTRUCT()
struct FPartyReservation
{
	GENERATED_USTRUCT_BODY()

	FPartyReservation() 
		: TeamNum(INDEX_NONE)
	{}

	/** Team assigned to this party */
	UPROPERTY(Transient)
	int32 TeamNum;

	/** Player initiating the request */
	UPROPERTY(Transient)
	FUniqueNetIdRepl PartyLeader;

	/** All party members (including party leader) in the reservation */
	UPROPERTY(Transient)
	TArray<FPlayerReservation> PartyMembers;

	/** Keeping a record of all logged out players from this reservation. */
	UPROPERTY(Transient)
	TArray<FPlayerReservation> RemovedPartyMembers;
	UE_API void RemovePartyMemberAtIndex(int32 Idx);
	UE_API int32 RemoveAllPartyMembers(const FPlayerReservation& OtherRes);

	/** Is this data well formed */
	UE_API bool IsValid(bool bIsValidationStrRequired=true) const;

	/** Dump this reservation to log */
	UE_API void Dump() const;
	UE_API void DumpUniqueIdsOnly() const;

	/**
	 * Checks if a player from a different reservation can migrate to this reservation
	 * For example, TeamNum must match
	 *
	 * @param Other the other reservation the player is in
	 * @return true if the player can migrate, false if not
	 */
	UE_API bool CanPlayerMigrateFromReservation(const FPartyReservation& Other) const;
};

/** Platform mapping for crossplay checks */
USTRUCT()
struct FPartyBeaconCrossplayPlatformMapping
{
	GENERATED_USTRUCT_BODY()

	/** Platform name (eg WIN) */
	UPROPERTY()
	FString PlatformName;
	/** Platform type the name is mapped to (eg PlatformName=Win, PlatformType=OTHER) */
	UPROPERTY()
	FString PlatformType;

	bool operator==(const FString& InPlatformName) { return PlatformName == InPlatformName; }
};

/**
 * A beacon host used for taking reservations for an existing game session
 */
UCLASS(MinimalAPI, transient, config=Engine)
class UPartyBeaconState : public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	 * Initialize this state object 
	 *
	 * @param InTeamCount number of teams to make room for
	 * @param InTeamSize size of each team
	 * @param InMaxReservation max number of reservations allowed
	 * @param InSessionName name of session related to the beacon
	 * @param InForceTeamNum team to force players on if applicable (usually only 1 team games)
	 *
	 * @return true if successful created, false otherwise
	 */
	UE_API virtual bool InitState(int32 InTeamCount, int32 InTeamSize, int32 InMaxReservations, FName InSessionName, int32 InForceTeamNum, bool bEnableRemovalRequests);

	/**
	 * Reconfigures the beacon for a different team/player count configuration
	 * Allows dedicated server to change beacon parameters after a playlist configuration has been made
	 * Does no real checking against current reservations because we assume the UI wouldn't let
	 * this party start a gametype if they were too big to fit on a team together
	 *
	 * @param InNumTeams the number of teams that are expected to join
	 * @param InNumPlayersPerTeam the number of players that are allowed to be on each team
	 * @param InNumReservations the total number of players to allow to join (if different than team * players)
	 */
	UE_API virtual bool ReconfigureTeamAndPlayerCount(int32 InNumTeams, int32 InNumPlayersPerTeam, int32 InNumReservations);

	/**
	 * Competitive integrity represents how "fair" we need to keep the game. So no friends on other teams, no cross team chat, etc.
	 *
	 * @param bNewCompetitiveIntegrity New value for Competitive integrity
	 */
	UE_API virtual void SetCompetitiveIntegrity(bool bNewCompetitiveIntegrity);

	/**
	 * Define the method for assignment new reservations to teams
	 * 
	 * @param NewAssignmentMethod name of the assignment method to use (@see ETeamAssignmentMethod for descriptions)
	 */
	UE_API virtual void SetTeamAssignmentMethod(FName NewAssignmentMethod);

	/**
	 * Add a reservation to the beacon state, tries to assign a team
	 * 
	 * @param ReservationRequest reservation to possibly add to this state
	 *
	 * @return true if successful, false otherwise
	 */
	UE_API virtual bool AddReservation(const FPartyReservation& ReservationRequest);

	/**
	 * Remove an entire reservation from this state object
	 *
	 * @param PartyLeader leader holding the reservation for the party 
	 *
	 * @return true if successful, false if reservation not found
	 */
	UE_API virtual bool RemoveReservation(const FUniqueNetIdRepl& PartyLeader);

	/**
	 * Register user auth ticket with the reservation system
	 * Must have an existing reservation entry
	 *
	 * @param InPartyMemberId id of player logging in 
	 * @param InAuthTicket auth ticket reported by the user
	 */
	UE_API void RegisterAuthTicket(const FUniqueNetIdRepl& InPartyMemberId, const FString& InAuthTicket);

	/**
	 * Update party leader for a given player with the reservation beacon
	 * (needed when party leader leaves, reservation beacon is in a temp/bad state until someone updates this)
	 *
	 * @param InPartyMemberId party member making the update
	 * @param PartyLeaderId id of new leader
	 */
	UE_API virtual void UpdatePartyLeader(const FUniqueNetIdRepl& InPartyMemberId, const FUniqueNetIdRepl& NewPartyLeaderId);

	/**
	 * Swap the parties between teams, parties must be able to fit on other team after swap
	 *
	 * @param PartyLeader party 1 to swap
	 * @param OtherPartyLeader party 2 to swap
	 * 
	 * @return true if successful, false otherwise
	 */
	UE_API virtual bool SwapTeams(const FUniqueNetIdRepl& PartyLeader, const FUniqueNetIdRepl& OtherPartyLeader);

	/**
	 * Place a party on a new team, party must fit and team must exist
	 *
	 * @param PartyLeader party to change teams
	 * @param NewTeamNum team to change to
	 *
	 * @return true if successful, false otherwise
	 */
	UE_API virtual bool ChangeTeam(const FUniqueNetIdRepl& PartyLeader, int32 NewTeamNum);

	/**
	 * Remove a single player from their party's reservation
	 *
	 * PlayerId player to remove
	 *
	 * @return true if player found and removed, false otherwise
	 */
	UE_API virtual bool RemovePlayer(const FUniqueNetIdRepl& PlayerId);

	/**
	 * @return the name of the session associated with this beacon state
	 */
	virtual FName GetSessionName() const { return SessionName; }

	/**
	 * @return all reservations in this beacon state
	 */
	virtual TArray<FPartyReservation>& GetReservations() { return Reservations; }

	/**
	 * Get an existing reservation for a given party
	 *
	 * @param PartyLeader UniqueId of party leader for a reservation
	 *
	 * @return index of reservation, INDEX_NONE otherwise
	 */
	UE_API virtual int32 GetExistingReservation(const FUniqueNetIdRepl& PartyLeader) const;

	/**
	 * Get an existing reservation containing a given party member
	 *
	 * @param PartyMember UniqueId of party member for a reservation
	 *
	 * @return index of reservation, INDEX_NONE otherwise
	 */
	UE_API virtual int32 GetExistingReservationContainingMember(const FUniqueNetIdRepl& PartyMember) const;

	/**
	* Get the current reservation count inside the beacon
	* this is NOT the number of players in the game
	*
	* @return number of consumed reservations
	*/
	virtual int32 GetReservationCount() const { return Reservations.Num(); }

	/**
	 * @return the total number of reservations allowed
	 */
	virtual int32 GetMaxReservations() const { return MaxReservations; }

	/**
	 * @return the amount of space left in the beacon
	 */
	virtual int32 GetRemainingReservations() const { return MaxReservations - NumConsumedReservations; }

	/**
	 * @return the number of actually used reservations across all parties
	 */
	virtual int32 GetNumConsumedReservations() const { return NumConsumedReservations; }

	/**
	 * Determine if this reservation fits all rules for fitting in the game
	 * 
	 * @param ReservationRequest reservation to test for 
	 *
	 * @return true if reservation can be added to this state, false otherwise
	 */
	UE_API virtual bool DoesReservationFit(const FPartyReservation& ReservationRequest) const;

	/**
	 * Determine if this reservation fits all rules for fitting in the game
	 * 
	 * @param ExistingReservation reservation to test for
	 * @param NewPlayers players requested to be added
	 *
	 * @return true if reservation can be modified to fit this state, false otherwise
	 */
	UE_API virtual bool DoesModifiedReservationFit(const FPartyReservation& ExistingReservation, const TArray<FPlayerReservation>& NewPlayers) const;

	/**
	 * @return true if the beacon is currently at max capacity
	 */
	virtual bool IsBeaconFull() const { return NumConsumedReservations == MaxReservations; }

	/**
	 * Get the number of teams.
	 *
	 * @return The number of teams.
	 */
	virtual int32 GetNumTeams() const { return NumTeams; }

	/**
	 * Get the max number of players per team
	 *
	 * @return The number of player per team
	 */
	virtual int32 GetMaxPlayersPerTeam() const { return NumPlayersPerTeam; }

	/**
	 * Competitive integrity represents how "fair" we need to keep the game. So no friends on other teams, no cross team chat, etc.
	 *
	 * @return Whether we should respect competitive integrity
	 */
	virtual bool ShouldRespectCompetitiveIntegrity() const { return bRespectCompetitiveIntegrity; }

	/**
	 * Determine the maximum team size that can be accommodated based
	 * on the current reservation slots occupied.
	 *
	 * @return maximum team size that is currently available
	 */
	UE_API virtual int32 GetMaxAvailableTeamSize() const;

	/**
	 * Get the number of current players on a given team.
	 *
	 * @param TeamIdx team of interest
	 *
	 * @return The number of teams.
	 */
	UE_API virtual int32 GetNumPlayersOnTeam(int32 TeamIdx) const;

	/**
	 * Get the team index for a given player
	 *
	 * @param PlayerId uniqueid for the player of interest
	 *
	 * @return team index for the given player, INDEX_NONE otherwise
	 */
	UE_API virtual int32 GetTeamForCurrentPlayer(const FUniqueNetId& PlayerId) const;

	/**
	 * Get the amount of teams used by reservations
	 *
	 * @return The number of teams that are used by reservations
	 */
	UE_API virtual int32 GetReservationTeamCount() const;

	/**
	 * Get all the known players on a given team
	 * 
	 * @param TeamIndex valid team index to query
	 * @param TeamMembers [out] array of unique ids found to be on the given team
	 *
	 * @return number of players on team, 0 if invalid
	 */
	UE_API int32 GetPlayersOnTeam(int32 TeamIndex, TArray<FUniqueNetIdRepl>& TeamMembers) const;

	/**
	 * Does a given player id have an existing reservation
	 *
	 * @param PlayerId uniqueid of the player to check
	 *
	 * @return true if a reservation exists, false otherwise
	 */
	UE_API virtual bool PlayerHasReservation(const FUniqueNetId& PlayerId) const;

	/**
	* Updates the platform on an existing reservation
	* (Used when MMS does not set a platform when handing out a match assignment)
	*
	* @param PlayerId uniqueid of the player to update
	* @param PlatformName platform to set new player as using
	*
	* @return true if a reservation exists, false otherwise
	*/
	UE_API virtual bool UpdateMemberPlatform(const FUniqueNetIdRepl& PartyMember, const FString& PlatformName);
	
	/**
	 * Obtain player validation string from party reservation entry
	 *
	 * @param PlayerId unique id of player to find validation in an existing reservation
	 * @param OutValidation [out] validation string used when player requested a reservation
	 *
	 * @return true if reservation exists for player
	 *
	 */
	UE_API virtual bool GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const;

	/**
	 * Get the party leader for a given unique id
	 *
	 * @param InPartyMemberId valid party member of some reservation looking for its leader
	 * @param OutPartyLeaderId valid party leader id for the given party member if found, invalid if function returns false
	 *
	 * @return true if a party leader was found for a given user id, false otherwise
	 */
	UE_API virtual bool GetPartyLeader(const FUniqueNetIdRepl& InPartyMemberId, FUniqueNetIdRepl& OutPartyLeaderId) const;

	/**
	 * Randomly assign a team for the reservation configuring the beacon
	 */
	UE_API virtual void InitTeamArray();

	/**
	* Determine if there are any teams that can fit the current party request.
	*
	* @param ReservationRequest reservation request to fit
	* @return true if there are teams available, false otherwise
	*/
	UE_API virtual bool AreTeamsAvailable(const FPartyReservation& ReservationRequest) const;

	/**
	 * Perform a cross play compatible check for the new reservation against existing reservations
	 * 
	 * @param ReservationRequest new reservation to compare
	 * 
	 * @return true if there are no cross play restrictions, false otherwise
	 */
	UE_API virtual bool CrossPlayAllowed(const FPartyReservation& ReservationRequest) const;

	/**
	 * @return true if there are cross play restrictions, false otherwise
	 */
	UE_API virtual bool HasCrossplayOptOutReservation() const;

	/**
	 * Get a count of all players for a given platform
	 * 
	 * @param InPlatform platform to get a count for
	 * @param bIncludeMappedPlatforms true if we should include platforms that map to InPlatform (See FPartyBeaconCrossplayPlatformMapping)
	 * 
	 * @return number of players for a given platform
	 */
	UE_API virtual int32 GetReservationPlatformCount(const FString& InPlatform, bool bIncludeMappedPlatforms) const;

	/**
	* Determine the team number for the given party reservation request.
	* Uses the list of current reservations to determine what teams have open slots.
	*
	* @param PartyRequest the party reservation request received from the client beacon
	* @return index of the team to assign to all members of this party
	*/
	UE_API virtual int32 GetTeamAssignment(const FPartyReservation& Party);

	/**
	 * Output current state of reservations to log
	 */
	UE_API virtual void DumpReservations() const;

	/**
	 * Allows child classes to add extra validation when a new reservation request comes in, 
	 * for example to validate the new players are allowed in based on privacy settings
	 * 
	 * @param ReservationRequest The new reservation that will be added to the beacon's list of reservations
	 *
	 * @return True if the players in the reservation are allowed in the session
	 */
	virtual bool ValidateNewPartyReservation(const FPartyReservation& ReservationRequest) { return true; }

protected:

	/** Session tied to the beacon */
	UPROPERTY(Transient)
	FName SessionName;
	/** Number of currently consumed reservations */
	UPROPERTY(Transient)
	int32 NumConsumedReservations;
	/** Maximum allowed reservations */
	UPROPERTY(Transient)
	int32 MaxReservations;
	/** Number of teams in the game */
	UPROPERTY(Transient)
	int32 NumTeams;
	/** Number of players on each team for balancing */
	UPROPERTY(Transient)
	int32 NumPlayersPerTeam;
	/** Team assignment method */
	UPROPERTY(Transient)
	FName TeamAssignmentMethod;
	/** Team that the host has been assigned to */
	UPROPERTY(Transient)
	int32 ReservedHostTeamNum;
	/** Team that everyone is forced to in single team games */
	UPROPERTY(Transient)
	int32 ForceTeamNum;
	/** Are multiple consoles types allowed to play together */
	UPROPERTY(Config)
	bool bRestrictCrossConsole;
	/** Platform crossplay restrictions.  Expected in the format "Plat1=Plat2,Plat3" to indicate Plat1 is considered crossplay with Plat2 and Plat3 */
	UPROPERTY(Config)
	TArray<FString> PlatformCrossplayRestrictions;
	UPROPERTY(Config)
	TArray<FPartyBeaconCrossplayPlatformMapping> PlatformTypeMapping;
	/** Process requests from clients to remove players from beacon */
	UPROPERTY(Transient)
	bool bEnableRemovalRequests;
	/** Competitive integrity represents how "fair" we need to keep the game. So no friends on other teams, no cross team chat, etc. */
	UPROPERTY(Transient)
	bool bRespectCompetitiveIntegrity;

	/** Current reservations in the system */
	UPROPERTY(Transient)
	TArray<FPartyReservation> Reservations;
	/** Players that are expected to join shortly */
	TArray< FUniqueNetIdPtr > PlayersPendingJoin;

	/**
	 * Arrange reservations to make the most room available on a single team
	 * allowing larger parties to fit into this session
	 * Since teams change, this shouldn't be used after the teams have been set in the game logic
	 */
	UE_API virtual void BestFitTeamAssignmentJiggle();

	/** 
	 * Check that our reservations are in a good state
	 * @param bIgnoreEmptyReservations Whether we want to ignore empty reservations or not (because we intend to clean them up later)
	 */
	UE_API void SanityCheckReservations(const bool bIgnoreEmptyReservations) const;

	friend class APartyBeaconHost;
};

#undef UE_API
