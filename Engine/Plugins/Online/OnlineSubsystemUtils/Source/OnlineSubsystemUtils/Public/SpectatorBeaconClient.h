// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpectatorBeaconState.h"
#include "TimerManager.h"
#include "OnlineBeaconClient.h"

#include "SpectatorBeaconClient.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FPlayerReservation;

class FOnlineSessionSearchResult;

/**
* Types of reservation requests that can be made by this beacon
*/
UENUM()
enum class ESpectatorClientRequestType : uint8
{
	/** None pending */
	NonePending,
	/** Make a reservation with an existing session */
	ExistingSessionReservation,
	/** Make an update to an existing reservation */
	ReservationUpdate,
	/** Reservation to configure an empty server  */
	EmptyServerReservation,
	/** Simple reconnect (checks for existing reservation) */
	Reconnect,
	/** Abandon the reservation beacon (game specific handling)*/
	Abandon
};

inline const TCHAR* ToString(ESpectatorClientRequestType RequestType)
{
	switch (RequestType)
	{
	case ESpectatorClientRequestType::NonePending:
	{
		return TEXT("No Request Pending");
	}
	case ESpectatorClientRequestType::ExistingSessionReservation:
	{
		return TEXT("Existing Session Reservation");
	}
	case ESpectatorClientRequestType::ReservationUpdate:
	{
		return TEXT("Reservation Update");
	}
	case ESpectatorClientRequestType::EmptyServerReservation:
	{
		return TEXT("Empty Server Reservation");
	}
	case ESpectatorClientRequestType::Reconnect:
	{
		return TEXT("Reconnect Only");
	}
	}
	return TEXT("");
}

/**
* Delegate triggered when a response from the spectator beacon host has been received
*
* @param ReservationResponse response from the server
*/
DECLARE_DELEGATE_OneParam(FOnSpectatorReservationRequestComplete, ESpectatorReservationResult::Type /** ReservationResponse */);

/**
* Delegate triggered when the host indicated a reservation count has changed
*
* @param NumRemaining number of slots remaining in the session
*/
DECLARE_DELEGATE_OneParam(FOnReservationCountUpdate, int32 /** NumRemaining */);

/** Delegate triggered when the host indicated the reservation is full */
DECLARE_DELEGATE(FOnReservationFull);

/**
* A beacon client used for making reservations with an existing game session
*/
UCLASS(MinimalAPI, transient, notplaceable, config = Engine)
class ASpectatorBeaconClient : public AOnlineBeaconClient
{
	GENERATED_UCLASS_BODY()

		//~ Begin UObject Interface
		UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin AOnlineBeaconClient Interface
	UE_API virtual void OnConnected() override;
	UE_API virtual void OnFailure() override;
	//~ End AOnlineBeaconClient Interface

	/**
	* Sends a request to the remote host to allow the specified members to reserve space
	* in the host's session. Note this request is async.
	*
	* @param ConnectInfoStr the URL of the server that the connection will be made to
	* @param InSessionId Id of the session expected to be found at this destination
	* @param RequestingSpectator the spectator that will be joining
	* @param Reservation the reservation for the spectator who wants to reserve space
	*
	* @return true if the request able to be sent, false if it failed to send
	*/
	UE_API virtual bool RequestReservation(const FString& ConnectInfoStr, const FString& InSessionId, const FUniqueNetIdRepl& RequestingSpectator, const FPlayerReservation& Reservation);

	/**
	* Sends a request to the remote host to allow the specified members to reserve space
	* in the host's session. Note this request is async.
	*
	* @param DesiredHost a search result describing the server that the connection will be made to
	* @param RequestingSpectator the spectator that will be joining
	* @param Reservation the reservation for the spectator who wants to reserve space
	*
	* @return true if the request able to be sent, false if it failed to send
	*/
	UE_API virtual bool RequestReservation(const FOnlineSessionSearchResult& DesiredHost, const FUniqueNetIdRepl& RequestingSpectator, const FPlayerReservation& Reservation);

	/**
	* Cancel an existing request to the remote host to revoke allocated space on the server.
	* Note this request is async.
	*/
	UE_API virtual void CancelReservation();

	/**
	* Response from the host session after making a reservation request
	*
	* @param ReservationResponse response from server
	*/
	UFUNCTION(client, reliable)
		UE_API virtual void ClientReservationResponse(ESpectatorReservationResult::Type ReservationResponse);

	/**
	* Response from the host session after making a cancellation request
	*
	* @param ReservationResponse response from server
	*/
	UFUNCTION(client, reliable)
		UE_API virtual void ClientCancelReservationResponse(ESpectatorReservationResult::Type ReservationResponse);

	/**
	* Response from the host session that the reservation count has changed
	*
	* @param NumRemainingReservations number of slots remaining until a full session
	*/
	UFUNCTION(client, reliable)
		UE_API virtual void ClientSendReservationUpdates(int32 NumRemainingReservations);

	/** Response from the host session that the reservation is full */
	UFUNCTION(client, reliable)
		UE_API virtual void ClientSendReservationFull();

	/**
	* Delegate triggered when a response from the spectator beacon host has been received
	*
	* @return delegate to handle response from the server
	*/
	FOnSpectatorReservationRequestComplete& OnReservationRequestComplete() { return ReservationRequestComplete; }

	/**
	* Delegate triggered when the host indicated a reservation count has changed
	*
	* @param NumRemaining number of slots remaining in the session
	*/
	FOnReservationCountUpdate& OnReservationCountUpdate() { return ReservationCountUpdate; }

	/** Delegate triggered when the host indicated the reservation is full */
	FOnReservationFull& OnReservationFull() { return ReservationFull; }

	/**
	* @return the pending reservation associated with this beacon client
	*/
	const FSpectatorReservation& GetPendingReservation() const { return PendingReservation; }

protected:

	/** Delegate for reservation request responses */
	FOnSpectatorReservationRequestComplete ReservationRequestComplete;
	/** Delegate for reservation count updates */
	FOnReservationCountUpdate ReservationCountUpdate;

	/** Delegate for reservation full */
	FOnReservationFull ReservationFull;

	/** Session Id of the destination host */
	UPROPERTY()
		FString DestSessionId;
	/** Pending reservation that will be sent upon connection with the intended host */
	UPROPERTY()
		FSpectatorReservation PendingReservation;

	/** Type of request currently being handled by this client beacon */
	UPROPERTY()
		ESpectatorClientRequestType RequestType;

	/** Has the reservation request been delivered */
	UPROPERTY()
		bool bPendingReservationSent;
	/** Has the reservation request been canceled */
	UPROPERTY()
		bool bCancelReservation;

	/** Timer to trigger a cancel reservation request if the server doesn't respond in time */
	FTimerHandle CancelRPCFailsafe;

	/** Timers for delaying various responses (debug) */
	FTimerHandle PendingResponseTimerHandle;
	FTimerHandle PendingCancelResponseTimerHandle;
	FTimerHandle PendingReservationUpdateTimerHandle;
	FTimerHandle PendingReservationFullTimerHandle;

	/** Clear out all the timer handles listed above */
	UE_API void ClearTimers(bool bCallFailSafeIfNeeded);
	/** Delegate triggered if the client doesn't hear from the server in time */
	UE_API void OnCancelledFailsafe();
	/** Delegate triggered when a cancel reservation request is complete */
	UE_API void OnCancelledComplete();

	/** Process a response to our RequestReservation request to the server */
	UE_API void ProcessReservationResponse(ESpectatorReservationResult::Type ReservationResponse);
	/** Process a response to our CancelReservation request to the server */
	UE_API void ProcessCancelReservationResponse(ESpectatorReservationResult::Type ReservationResponse);
	/** Process a response from the server with an update to the number of consumed reservations */
	UE_API void ProcessReservationUpdate(int32 NumRemainingReservations);
	/** Process a response from the server that the reservation beacon is full */
	UE_API void ProcessReservationFull();

	/**
	* Tell the server about the reservation request being made
	*
	* @param SessionId expected session id on the other end (must match)
	* @param Reservation pending reservation request to make with server
	*/
	UFUNCTION(server, reliable, WithValidation)
		UE_API virtual void ServerReservationRequest(const FString& SessionId, const FSpectatorReservation& Reservation);

	/**
	* Tell the server to cancel a pending or existing reservation
	*
	* @param Spectator id of the spectator for the reservation to cancel
	*/
	UFUNCTION(server, reliable, WithValidation)
		UE_API virtual void ServerCancelReservationRequest(const FUniqueNetIdRepl& Spectator);

	/**
	* Trigger the given delegate at a later time
	*
	* @param Delegate function to call at a later date
	* @param Delay time to wait before calling function
	*
	* @return handle in the timer system for this entry
	*/
	UE_API FTimerHandle DelayResponse(FTimerDelegate& Delegate, float Delay);
};

#undef UE_API
