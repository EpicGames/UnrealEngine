// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpectatorBeaconState.h"
#include "OnlineBeaconHostObject.h"
#include "SpectatorBeaconHost.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

struct FPlayerReservation;
struct FUniqueNetIdRepl;

class ASpectatorBeaconClient;

/**
* Delegate type for handling reservation additions/removals, or full events
*/
DECLARE_DELEGATE(FOnReservationUpdate);

/**
* Delegate fired when a the beacon host has been told to cancel a reservation
*
* @param Spectator spectator cancelling reservation
*/
DECLARE_DELEGATE_OneParam(FOnCancelationReceived, const FUniqueNetId&);

/**
* Delegate fired when a the beacon host has added a new player
*
* @param PlayerId Player who reservation has been made for.
*/
DECLARE_DELEGATE_OneParam(FOnNewPlayerAdded, const FPlayerReservation&);

/**
* Delegate called when the beacon gets any request, allowing the owner to validate players at a higher level (bans,etc)
*
* @param Spectator player making up the reservation
* @return true if these players are ok to join
*/
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnSpectatorValidatePlayers, const FPlayerReservation&);

/**
* Delegate fired when a the beacon host detects a duplicate reservation
*
* @param SpectatorReservation reservation that is found to be duplicated
*/
DECLARE_DELEGATE_OneParam(FOnSpectatorDuplicateReservation, const FSpectatorReservation&);

/**
* A beacon host used for taking reservations for an existing game session
*/
UCLASS(MinimalAPI, transient, notplaceable, config = Engine)
class ASpectatorBeaconHost : public AOnlineBeaconHostObject
{
	GENERATED_UCLASS_BODY()

		//~ Begin UObject interface
		UE_API virtual void PostInitProperties() override;
	//~ End UObject interface

	//~ Begin AActor Interface
	UE_API virtual void Tick(float DeltaTime) override;
	//~ End AActor Interface

	//~ Begin AOnlineBeaconHostObject Interface 
	//~ End AOnlineBeaconHost Interface 

	/**
	* Initialize the spectator host beacon
	*
	* @param InMaxReservation max number of reservations allowed
	* @param InSessionName name of session related to the beacon
	*
	* @return true if successful created, false otherwise
	*/
	UE_API virtual bool InitHostBeacon(int32 InMaxReservations, FName InSessionName);

	/**
	* Initialize the spectator host beacon from a previous state/configuration
	* all existing reservations and configuration values are preserved
	*
	* @return true if successful created, false otherwise
	*/
	UE_API virtual bool InitFromBeaconState(USpectatorBeaconState* PrevState);

	/**
	* @return reference to the state of the SpectatorBeacon
	*/
	USpectatorBeaconState* GetState() const { return State; }

	/**
	* Notify the beacon of a player logout
	*
	* @param PlayerId UniqueId of player logging out
	*/
	UE_API virtual void HandlePlayerLogout(const FUniqueNetIdRepl& PlayerId);

	/**
	* Get the current reservation count inside the beacon
	* NOTE: This is *NOT* the same as the number of consumed reservations across all parties, just the total number of reservations!
	*
	* @return number of reservations inside the beacon (*NOT* number of consumed reservations)
	*/
	virtual int32 GetReservationCount() const { return State->GetReservationCount(); }

	/**
	* Get the number of reservations actually used/consumed across all parties inside the beacon
	*
	* @return the number of actually used reservations across all parties inside the beacon
	*/
	virtual int32 GetNumConsumedReservations() const { return State->GetNumConsumedReservations(); }

	/**
	* Get the maximum number of reservations allowed inside the beacon
	*
	* @return The maximum number of reservations allowed inside the beacon
	*/
	virtual int32 GetMaxReservations() const { return State->GetMaxReservations(); }

	/**
	* Does a given player id have an existing reservation
	*
	* @param PlayerId uniqueid of the player to check
	*
	* @return true if a reservation exists, false otherwise
	*/
	UE_API virtual bool PlayerHasReservation(const FUniqueNetId& PlayerId) const;

	/**
	* Obtain player validation string from spectator reservation entry
	*
	* @param PlayerId unique id of player to find validation in an existing reservation
	* @param OutValidation [out] validation string used when player requested a reservation
	*
	* @return true if reservation exists for player
	*
	*/
	UE_API virtual bool GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const;

	/**
	* Attempts to add a spectator reservation to the beacon
	*
	* @param ReservationRequest reservation attempt
	*
	* @return add attempt result
	*/
	UE_API virtual ESpectatorReservationResult::Type AddSpectatorReservation(const FSpectatorReservation& ReservationRequest);

	/**
	* Attempts to remove a spectator reservation from the beacon
	*
	* @param Spectator reservation owner
	*/
	UE_API virtual ESpectatorReservationResult::Type RemoveSpectatorReservation(const FUniqueNetIdRepl& Spectator);

	/**
	* Register user auth ticket with the reservation system
	* Must have an existing reservation entry
	*
	* @param InSpectatorId id of player logging in
	* @param InAuthTicket auth ticket reported by the user
	*/
	UE_API void RegisterAuthTicket(const FUniqueNetIdRepl& InSpectatorId, const FString& InAuthTicket);

	/**
	* Handle a reservation request received from an incoming client
	*
	* @param Client client beacon making the request
	* @param SessionId id of the session that is being checked
	* @param ReservationRequest payload of request
	*/
	UE_API virtual void ProcessReservationRequest(ASpectatorBeaconClient* Client, const FString& SessionId, const FSpectatorReservation& ReservationRequest);

	/**
	* Handle a reservation cancellation request received from an incoming client
	*
	* @param Client client beacon making the request
	* @param Spectator reservation owner
	*/
	UE_API virtual void ProcessCancelReservationRequest(ASpectatorBeaconClient* Client, const FUniqueNetIdRepl& Spectator);

	/**
	* Crossplay
	*/


	/**
	* Delegate fired when a the beacon host detects a reservation addition/removal
	*/
	FOnReservationUpdate& OnReservationsFull() { return ReservationsFull; }

	/**
	* Delegate fired when a the beacon host detects that all reservations are full
	*/
	FOnReservationUpdate& OnReservationChanged() { return ReservationChanged; }

	/**
	* Delegate fired when the beacon host adds a new player
	*/
	FOnNewPlayerAdded& OnNewPlayerAdded() { return NewPlayerAddedDelegate; }

	/**
	* Delegate fired when a the beacon host cancels a reservation
	*/
	FOnCancelationReceived& OnCancelationReceived() { return CancelationReceived; }

	/**
	* Delegate fired when a the beacon detects a duplicate reservation
	*/
	FOnSpectatorDuplicateReservation& OnDuplicateReservation() { return DuplicateReservation; }

	/**
	* Delegate called when the beacon gets any request, allowing the owner to validate players at a higher level (bans,etc)
	*/
	FOnSpectatorValidatePlayers& OnValidatePlayers() { return ValidatePlayers; }

	/**
	* Output current state of reservations to log
	*/
	UE_API virtual void DumpReservations() const;

protected:

	/** State of the beacon */
	UPROPERTY()
		TObjectPtr<USpectatorBeaconState> State;

	/** Delegate fired when the beacon indicates all reservations are taken */
	FOnReservationUpdate ReservationsFull;
	/** Delegate fired when the beacon indicates a reservation add/remove */
	FOnReservationUpdate ReservationChanged;
	/** Delegate fired when the beacon indicates a reservation cancellation */
	FOnCancelationReceived CancelationReceived;
	/** Delegate fired when the beacon detects a duplicate reservation */
	FOnSpectatorDuplicateReservation DuplicateReservation;
	/** Delegate fired when reservation has been added */
	FOnNewPlayerAdded NewPlayerAddedDelegate;
	/** Delegate fired when asking the beacon owner if this reservation is legit */
	FOnSpectatorValidatePlayers ValidatePlayers;

	/** Do the timeouts below cause a player to be removed from the reservation list */
	UPROPERTY(Config)
		bool bLogoutOnSessionTimeout;
	/** Do spectators require validation strings. */
	UPROPERTY(Config)
		bool bIsValidationStrRequired;
	/** Seconds that can elapse before a reservation is removed due to player not being registered with the session */
	UPROPERTY(Transient, Config)
		float SessionTimeoutSecs;
	/** Seconds that can elapse before a reservation is removed due to player not being registered with the session during a travel */
	UPROPERTY(Transient, Config)
		float TravelSessionTimeoutSecs;

	/**
	* @return the class of the state object inside this beacon
	*/
	virtual TSubclassOf<USpectatorBeaconState> GetSpectatorBeaconHostClass() const { return USpectatorBeaconState::StaticClass(); }

	/**
	* Update clients with current reservation information
	*/
	UE_API void SendReservationUpdates();

	/**
	* Handle a newly added player
	*
	* @param NewPlayer reservation of newly joining player
	*/
	UE_API void NewPlayerAdded(const FPlayerReservation& NewPlayer);

	/**
	* Does the session match the one associated with this beacon
	*
	* @param SessionId session to compare
	*
	* @return true if the session matches, false otherwise
	*/
	UE_API bool DoesSessionMatch(const FString& SessionId) const;

	UE_API void NotifyReservationEventNextFrame(FOnReservationUpdate& ReservationEvent);
};

#undef UE_API
