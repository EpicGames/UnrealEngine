// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineBeaconReservation.h"
#include "SpectatorBeaconState.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

ONLINESUBSYSTEMUTILS_API DECLARE_LOG_CATEGORY_EXTERN(LogSpectatorBeacon, Log, All);

/** The result code that will be returned during spectator reservation */
UENUM()
namespace ESpectatorReservationResult
{
	enum Type : int
	{
		/** Empty state. */
		NoResult,
		/** Pending request due to async operation, server will contact client shortly. */
		RequestPending,
		/** An unknown error happened. */
		GeneralError,
		/** All available reservations are booked. */
		SpectatorLimitReached,
		/** Wrong number of players to join the session. */
		IncorrectPlayerCount,
		/** No response from the host. */
		RequestTimedOut,
		/** Already have a reservation entry for the requesting spectator. */
		ReservationDuplicate,
		/** Couldn't find the spectator specified for a reservation update request. */
		ReservationNotFound,
		/** Space was available and it's time to join. */
		ReservationAccepted,
		/** The beacon is paused and not accepting new connections. */
		ReservationDenied,
		/** Some kind of cross play restriction prevents this spectator from joining */
		ReservationDenied_CrossPlayRestriction,
		/** This player is banned. */
		ReservationDenied_Banned,
		/** The reservation request was canceled before being sent. */
		ReservationRequestCanceled,
		// The reservation was rejected because it was badly formed
		ReservationInvalid,
		// The reservation was rejected because this was the wrong session
		BadSessionId,
		/** The reservation contains players already in this game */
		ReservationDenied_ContainsExistingPlayers,
	};
}

namespace ESpectatorReservationResult
{
	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(ESpectatorReservationResult::Type SessionType)
	{
		switch (SessionType)
		{
		case NoResult:
		{
			return TEXT("No outstanding request");
		}
		case RequestPending:
		{
			return TEXT("Pending Request");
		}
		case GeneralError:
		{
			return TEXT("General Error");
		}
		case SpectatorLimitReached:
		{
			return TEXT("Spectator Limit Reached");
		}
		case IncorrectPlayerCount:
		{
			return TEXT("Incorrect Player Count");
		}
		case RequestTimedOut:
		{
			return TEXT("Request Timed Out");
		}
		case ReservationDuplicate:
		{
			return TEXT("Reservation Duplicate");
		}
		case ReservationNotFound:
		{
			return TEXT("Reservation Not Found");
		}
		case ReservationAccepted:
		{
			return TEXT("Reservation Accepted");
		}
		case ReservationDenied:
		{
			return TEXT("Reservation Denied");
		}
		case ReservationDenied_CrossPlayRestriction:
		{
			return TEXT("Reservation Denied CrossPlayRestriction");
		}
		case ReservationDenied_Banned:
		{
			return TEXT("Reservation Banned");
		}
		case ReservationRequestCanceled:
		{
			return TEXT("Request Canceled");
		}
		case ReservationInvalid:
		{
			return TEXT("Invalid reservation");
		}
		case BadSessionId:
		{
			return TEXT("Bad Session Id");
		}
		case ReservationDenied_ContainsExistingPlayers:
		{
			return TEXT("Reservation Contains Existing Players");
		}
		}
		return TEXT("");
	}

	inline FText GetDisplayString(ESpectatorReservationResult::Type Response)
	{
		switch (Response)
		{
		case ESpectatorReservationResult::IncorrectPlayerCount:
		case ESpectatorReservationResult::SpectatorLimitReached:
			return NSLOCTEXT("ESpectatorReservationResult", "FullGame", "Game full");
		case ESpectatorReservationResult::RequestTimedOut:
			return NSLOCTEXT("ESpectatorReservationResult", "NoResponse", "No response");
		case ESpectatorReservationResult::ReservationDenied:
			return NSLOCTEXT("ESpectatorReservationResult", "DeniedResponse", "Not accepting connections");
		case ESpectatorReservationResult::ReservationDenied_Banned:
			return NSLOCTEXT("ESpectatorReservationResult", "BannedResponse", "Player Banned");
		case ESpectatorReservationResult::GeneralError:
			return NSLOCTEXT("ESpectatorReservationResult", "GeneralError", "Unknown Error");
		case ESpectatorReservationResult::ReservationNotFound:
			return NSLOCTEXT("ESpectatorReservationResult", "ReservationNotFound", "No Reservation");
		case ESpectatorReservationResult::ReservationAccepted:
			return NSLOCTEXT("ESpectatorReservationResult", "Accepted", "Accepted");
		case ESpectatorReservationResult::ReservationDuplicate:
			return NSLOCTEXT("ESpectatorReservationResult", "DuplicateReservation", "Duplicate reservation detected");
		case ESpectatorReservationResult::ReservationInvalid:
			return NSLOCTEXT("ESpectatorReservationResult", "InvalidReservation", "Bad reservation request");
		case ESpectatorReservationResult::ReservationDenied_ContainsExistingPlayers:
			return NSLOCTEXT("ESpectatorReservationResult", "ContainsExistingPlayers", "Spectator already in session");
		case ESpectatorReservationResult::NoResult:
		case ESpectatorReservationResult::BadSessionId:
		default:
			return FText::GetEmpty();
		}
	}
}

/** A whole Spectator reservation */
USTRUCT()
struct FSpectatorReservation
{
	GENERATED_USTRUCT_BODY()

		FSpectatorReservation()
		{}


	/** Player initiating the request */
	UPROPERTY(Transient)
		FUniqueNetIdRepl SpectatorId;

	/** Spectator reservation */
	UPROPERTY(Transient)
		FPlayerReservation Spectator;

	/** Is this data well formed */
	UE_API bool IsValid(bool bIsValidationStrRequired=true) const;

	/** Dump this reservation to log */
	UE_API void Dump() const;
};

/**
* A beacon host used for taking reservations for an existing game session
*/
UCLASS(MinimalAPI, transient, config = Engine)
class USpectatorBeaconState : public UObject
{
	GENERATED_UCLASS_BODY()

		/**
		* Initialize this state object
		*
		* @param InTeamCount number of teams to make room for
		* @param InTeamSize size of each team
		* @param InMaxReservation max number of reservations allowed
		* @param InSessionName name of session related to the beacon
		* @param InForceTeamNum team to force players on if applicable (usually only 1 team games)
		*
		* @return true if successful created, false otherwise
		*/
		UE_API virtual bool InitState(int32 InMaxReservations, FName InSessionName);

	/**
	* Add a reservation to the beacon state, tries to assign a team
	*
	* @param ReservationRequest reservation to possibly add to this state
	*
	* @return true if successful, false otherwise
	*/
	UE_API virtual bool AddReservation(const FSpectatorReservation& ReservationRequest);

	/**
	* Remove an entire reservation from this state object
	*
	* @param Spectator player holding reservation
	*
	* @return true if successful, false if reservation not found
	*/
	UE_API virtual bool RemoveReservation(const FUniqueNetIdRepl& Spectator);

	/**
	* Register user auth ticket with the reservation system
	* Must have an existing reservation entry
	*
	* @param InSpectatorId id of player logging in
	* @param InAuthTicket auth ticket reported by the user
	*/
	UE_API void RegisterAuthTicket(const FUniqueNetIdRepl& InSpectatorId, const FString& InAuthTicket);

	/**
	* Remove a single player from their reservation
	*
	* PlayerId player to remove
	*
	* @return true if player found and removed, false otherwise
	*/
	UE_API virtual bool RemovePlayer(const FUniqueNetIdRepl& PlayerId);

	/**
	* @return the name of the session associated with this beacon state
	*/
	virtual FName GetSessionName() const { return SessionName; }

	/**
	* @return all reservations in this beacon state
	*/
	virtual TArray<FSpectatorReservation>& GetReservations() { return Reservations; }

	/**
	* Get an existing reservation for a given spectator
	*
	* @param Spectator UniqueId of spectator for a reservation
	*
	* @return index of reservation, INDEX_NONE otherwise
	*/
	UE_API virtual int32 GetExistingReservation(const FUniqueNetIdRepl& Spectator) const;

	/**
	* Get the current reservation count inside the beacon
	* this is NOT the number of players in the game
	*
	* @return number of consumed reservations
	*/
	virtual int32 GetReservationCount() const { return Reservations.Num(); }

	/**
	* @return the total number of reservations allowed
	*/
	virtual int32 GetMaxReservations() const { return MaxReservations; }

	/**
	* @return the amount of space left in the beacon
	*/
	virtual int32 GetRemainingReservations() const { return MaxReservations - NumConsumedReservations; }

	/**
	* @return the number of actually used reservations across all parties
	*/
	virtual int32 GetNumConsumedReservations() const { return NumConsumedReservations; }

	/**
	* Determine if this reservation fits all rules for fitting in the game
	*
	* @param ReservationRequest reservation to test for
	*
	* @return true if reservation can be added to this state, false otherwise
	*/
	UE_API virtual bool DoesReservationFit(const FSpectatorReservation& ReservationRequest) const;

	/**
	* @return true if the beacon is currently at max capacity
	*/
	virtual bool IsBeaconFull() const { return NumConsumedReservations == MaxReservations; }

	/**
	* Does a given player id have an existing reservation
	*
	* @param PlayerId uniqueid of the player to check
	*
	* @return true if a reservation exists, false otherwise
	*/
	UE_API virtual bool PlayerHasReservation(const FUniqueNetId& PlayerId) const;

	/**
	* Updates the platform on an existing reservation
	* (Used when MMS does not set a platform when handing out a match assignment)
	*
	* @param PlayerId uniqueid of the player to update
	* @param PlatformName platform to set new player as using
	*
	* @return true if a reservation exists, false otherwise
	*/
	UE_API virtual bool UpdateMemberPlatform(const FUniqueNetIdRepl& Spectator, const FString& PlatformName);

	/**
	* Obtain player validation string from spectator reservation entry
	*
	* @param PlayerId unique id of player to find validation in an existing reservation
	* @param OutValidation [out] validation string used when player requested a reservation
	*
	* @return true if reservation exists for player
	*
	*/
	UE_API virtual bool GetPlayerValidation(const FUniqueNetId& PlayerId, FString& OutValidation) const;

	/**
	* Perform a cross play compatible check for the new reservation against existing reservations
	*
	* @param ReservationRequest new reservation to compare
	*
	* @return true if there are no cross play restrictions, false otherwise
	*/
	UE_API virtual bool CrossPlayAllowed(const FSpectatorReservation& ReservationRequest) const;

	/**
	* @return true if there are cross play restrictions, false otherwise
	*/
	UE_API virtual bool HasCrossplayOptOutReservation() const;

	/**
	* Get a count of all players for a given platform
	*
	* @param InPlatform platform to get a count for
	*
	* @return number of players for a given platform
	*/
	UE_API virtual int32 GetReservationPlatformCount(const FString& InPlatform) const;

	/**
	* Output current state of reservations to log
	*/
	UE_API virtual void DumpReservations() const;

protected:

	/** Session tied to the beacon */
	UPROPERTY(Transient)
		FName SessionName;
	/** Number of currently consumed reservations */
	UPROPERTY(Transient)
		int32 NumConsumedReservations;
	/** Maximum allowed reservations */
	UPROPERTY(Transient)
		int32 MaxReservations;
	/** Are multiple consoles types allowed to play together */
	UPROPERTY(Config)
		bool bRestrictCrossConsole;

	/** Current reservations in the system */
	UPROPERTY(Transient)
		TArray<FSpectatorReservation> Reservations;
	/** Players that are expected to join shortly */
	TArray< FUniqueNetIdPtr > PlayersPendingJoin;

	/**
	* Check that our reservations are in a good state
	* @param bIgnoreEmptyReservations Whether we want to ignore empty reservations or not (because we intend to clean them up later)
	*/
	UE_API void SanityCheckReservations(const bool bIgnoreEmptyReservations) const;

	friend class ASpectatorBeaconHost;
};

#undef UE_API
