// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Engine/NetworkDelegates.h"
#include "Engine/World.h"
#include "Online/CoreOnlineFwd.h"
#include "OnlineError.h"
#include "Templates/Function.h"
#include "TimerManager.h"
#include "UObject/ObjectPtr.h"

class AOnlineBeacon;
class AOnlineBeaconClient;
class AOnlineBeaconHostObject;
class FNetworkNotify;
class ISocketSubsystem;
struct FURL;
class UWorld;

namespace BeaconUnitTest
{
	extern const FName NetDriverDefinitionName;
	extern const FName SocketSubsystemName;

	enum class ESocketFlags : uint8
	{
		Disabled = 0,
		SendEnabled = 1 << 0,
		RecvEnabled = 1 << 1,

		Default = SendEnabled | RecvEnabled,
	};
	ENUM_CLASS_FLAGS(ESocketFlags);

	enum class ETickFlags : uint8
	{
		None = 0,

		/**
		 * When set, sleep during the tick duration to progress wall time.
		 * Network timeouts depend on FPlatformTime::Seconds() instead of world tick time.
		 */
		SleepTickTime = 1 << 0,
	};
	ENUM_CLASS_FLAGS(ETickFlags);

	class FNetworkStats
	{
	public:
		TArray<uint8> ReceivedControlMessages;
	};

	struct FTestStats
	{
		struct FEncryption
		{
			struct FNetworkEncryptionToken
			{
				uint32 InvokeCount = 0;
				uint32 CallbackCount = 0;
			};

			struct FNetworkEncryptionAck
			{
				uint32 InvokeCount = 0;
				uint32 CallbackCount = 0;
			};

			struct FNetworkEncryptionFailure
			{
				uint32 InvokeCount = 0;
			};

			FNetworkEncryptionToken NetworkEncryptionToken;
			FNetworkEncryptionAck NetworkEncryptionAck;
			FNetworkEncryptionFailure NetworkEncryptionFailure;
		};

		struct FClient
		{
			struct FOnConnected
			{
				uint32 InvokeCount = 0;
			};

			struct FOnFailure
			{
				uint32 InvokeCount = 0;
			};

			FOnConnected OnConnected;
			FOnFailure OnFailure;
		};

		struct FHost
		{
			struct FOnFailure
			{
				uint32 InvokeCount = 0;
			};

			FOnFailure OnFailure;
		};

		struct FHostObject
		{
			struct FOnClientConnected
			{
				uint32 InvokeCount = 0;
			};

			struct FNotifyClientDisconnected
			{
				uint32 InvokeCount = 0;
			};

			FOnClientConnected OnClientConnected;
			FNotifyClientDisconnected NotifyClientDisconnected;
		};

		FEncryption Encryption;
		FClient Client;
		FHost Host;
		FHostObject HostObject;
	};

	struct FTestConfig
	{
		struct FNetDriver
		{
			bool bFailInit = false;

			float ConnectionTimeout = 1/15.f;

			float InitialConnectTimeout = 1/30.f;

			/* 
			 * Keepalive packets will also flush any waiting outbound messages.
			 * 
			 * This should normally be set to a time more frequent than the connection timeout, but
			 * less frequent than the world tick rate. Beacons should manually flush after each
			 * message to prevent needing to wait on a keep-alive flush to progress the handshake.
			 */
			float KeepAliveTime =  1/60.f;

			int32 ServerListenPort = 0;
		};

		struct FEncryption
		{
			struct FEncryptionHost
			{
				bool bDelayDelegate = false;
				EEncryptionResponse Response;
				FString ErrorMsg;
				FEncryptionData EncryptionData;
			};

			struct FEncryptionClient
			{
				bool bDelayDelegate = false;
				EEncryptionResponse Response;
				FString ErrorMsg;
				FEncryptionData EncryptionData;
			};

			bool bEnabled = false;

			FEncryptionHost Host;
			FEncryptionClient Client;
			EEncryptionFailureAction FailureAction = EEncryptionFailureAction::Default;
			FString NetDriverEncryptionComponentName;
		};

		struct FAuth
		{
			struct FVerify
			{
				bool bEnabled = false;
				bool bResult = false;
			};

			bool bEnabled = false;
			bool bDelayDelegate = false;
			FOnlineError Result;
			FVerify Verify;
		};

		struct FClient
		{
			struct FOnConnected
			{
				TFunction<void()> Callback;
			};

			FOnConnected OnConnected;
		};

		float WorldTickRate = 1/120.f;

		FNetDriver NetDriver;
		FEncryption Encryption;
		FAuth Auth;
		FClient Client;
	};

	class FTestPrerequisites : public FNoncopyable, public TSharedFromThis<FTestPrerequisites>
	{
	public:
		virtual ~FTestPrerequisites() = default;

		static ONLINESUBSYSTEMUTILS_API TSharedPtr<FTestPrerequisites> TryCreate();
		static ONLINESUBSYSTEMUTILS_API TSharedPtr<FTestPrerequisites> Get();

		static ONLINESUBSYSTEMUTILS_API FTestStats* GetActiveTestStats();
		static ONLINESUBSYSTEMUTILS_API const FTestConfig* GetActiveTestConfig();

		virtual void BindNetEncryptionDelegates() = 0;
		virtual void UnbindNetEncryptionDelegates() = 0;

		UWorld* GetWorld() const
		{
			return World.Get();
		}

		FTestStats& GetStats()
		{
			return Stats;
		}

		const FTestStats& GetStats() const
		{
			return Stats;
		}

		FTestConfig& GetConfig()
		{
			return Config;
		}

		const FTestConfig& GetConfig() const
		{
			return Config;
		}

	protected:
		FTestPrerequisites() = default;

		TObjectPtr<UWorld> World;
		FTestStats Stats;
		FTestConfig Config;
	};

	/**
	 * Adjust the socket flags for a beacon unit test socket associated with an online beacon.
	 * When send or receive are disabled pending messages will be queued. The messages will be dispatched on the following tick after re-enabling the channel.
	 */
	ONLINESUBSYSTEMUTILS_API bool SetSocketFlags(AOnlineBeacon* OnlineBeacon, ESocketFlags Flags);

	ONLINESUBSYSTEMUTILS_API bool SetTimeoutsEnabled(AOnlineBeacon* OnlineBeacon, bool bEnabled);

	ONLINESUBSYSTEMUTILS_API bool ConfigureBeacon(const FTestPrerequisites& Prerequisites, AOnlineBeacon* OnlineBeacon);

	ONLINESUBSYSTEMUTILS_API bool ConfigureBeaconNetDriver(const FTestPrerequisites& Prerequisites, AOnlineBeacon* OnlineBeacon, TSharedPtr<FNetworkStats>* OutStats);

	/**
	 * Helper to initialize the connection for a beacon client and overriding the default UserId.
	 */
	ONLINESUBSYSTEMUTILS_API bool InitClientForUser(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, const FUniqueNetIdRef& User);

	/**
	 * Helper to get the hosts version of the client beacon actor for the specified user.
	 */
	ONLINESUBSYSTEMUTILS_API AOnlineBeaconClient* GetBeaconClientForUser(AOnlineBeaconHostObject* HostObject, const FUniqueNetIdRef& User);


	/**
	 * Tick the unit test world one time to progress network state.
	 */
	ONLINESUBSYSTEMUTILS_API bool TickOnce(const FTestPrerequisites& Prerequisites, ETickFlags Flags = ETickFlags::None);

	/**
	 * Tick the unit test world until the beacon client connection has completed the packet handler handshake.
	 */
	ONLINESUBSYSTEMUTILS_API bool TickUntilConnectionInitialized(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, ETickFlags Flags = ETickFlags::None);

	/**
	 * Tick the unit test world until the beacon client has changed its status to 'Closed' or 'Invalid'.
	 */
	ONLINESUBSYSTEMUTILS_API bool TickUntilControlMessageReceived(const FTestPrerequisites& Prerequisites, const FNetworkStats& Stats, ETickFlags Flags = ETickFlags::None);

	/**
	 * Tick the unit test world until the beacon client has changed its status to 'Open'.
	 */
	ONLINESUBSYSTEMUTILS_API bool TickUntilConnected(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, ETickFlags Flags = ETickFlags::None);

	/**
	 * Tick the unit test world until the beacon client has changed its status to 'Closed' or 'Invalid'.
	 */
	ONLINESUBSYSTEMUTILS_API bool TickUntilDisconnected(const FTestPrerequisites& Prerequisites, AOnlineBeaconClient* OnlineBeaconClient, ETickFlags Flags = ETickFlags::None);

	/**
	 * Tick the unit test world until the network timeout has elapsed.
	 */
	ONLINESUBSYSTEMUTILS_API bool TickUntilTimeoutElapsed(const FTestPrerequisites& Prerequisite, ETickFlags Flags = ETickFlags::None);

	/**
	 * Tick the unit test world until predicated condition is met or max iterations is reached.
	 */
	template <typename FPredicate>
	inline bool TickUntil(const FTestPrerequisites& Prerequisites, FPredicate Predicate, ETickFlags Flags = ETickFlags::None)
	{
		constexpr uint32 MaxIterations = 64;
		for (uint32 i = 0; i < MaxIterations; ++i)
		{
			if (Predicate())
			{
				return true;
			}

			if (!TickOnce(Prerequisites, Flags))
			{
				return false;
			}
		}

		return false;
	}

	/**
	 * Schedule the callback to be triggered on the next frame. SetTimerForNextTick may run the
	 * timer on the same frame as scheduled if the timer manager has not yet been ticked.
	 */
	template <typename TCallback>
	bool SetTimerForNextFrame(UWorld* World, uint64 RequestingFrame, TCallback Callback)
	{
		if (ensure(World))
		{
			World->GetTimerManager().SetTimerForNextTick([World, RequestingFrame, Callback]()
			{
				if (RequestingFrame < GFrameCounter)
				{
					Callback();
				}
				else
				{
					// callback was signaled too early.
					SetTimerForNextFrame(World, RequestingFrame, Callback);
				}
			});

			return true;
		}

		return false;
	}

/* BeaconUnitTest */ }

#endif /* WITH_DEV_AUTOMATION_TESTS */
