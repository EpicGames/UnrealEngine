// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/VoiceInterface.h"
#include "Net/VoiceDataCommon.h"
#include "Interfaces/VoiceCapture.h"
#include "Online/OnlineBase.h"
#include "OnlineSubsystemUtilsPackage.h"
#include "OnlineSubsystemUtilsPackage.h"
#include "AudioDevice.h"

#define UE_API ONLINESUBSYSTEMUTILS_API


class USoundWaveProcedural;
class UVOIPTalker;
class UVoipListenerSynthComponent;
struct FVoiceSettings;

class IOnlineSubsystem;
class IVoiceDecoder;
class IVoiceEncoder;
class IVoiceCapture;

/**
 * Container for unprocessed voice data
 */
struct FLocalVoiceData
{
	FLocalVoiceData() :
		VoiceRemainderSize(0)
	{
	}

	/** Amount of voice data not encoded last time */
	uint32 VoiceRemainderSize;
	/** Voice sample data not encoded last time */
	TArray<uint8> VoiceRemainder;
	/** Output for a local talker. */
	Audio::FPatchSplitter LocalVoiceOutput;
};

/**
 * Container for voice amplitude data
 */
struct FVoiceAmplitudeData
{
	float Amplitude = 0.0;
	double LastSeen = 0.0;
};

/** 
 * Remote voice data playing on a single client
 */
class FRemoteTalkerDataImpl
{
public:

	UE_API FRemoteTalkerDataImpl();
	/** Required for TMap FindOrAdd() */
	UE_API FRemoteTalkerDataImpl(const FRemoteTalkerDataImpl& Other);
	UE_API FRemoteTalkerDataImpl(FRemoteTalkerDataImpl&& Other);
	UE_API ~FRemoteTalkerDataImpl();

	/** Reset the talker after long periods of silence */
	UE_API void Reset();
	/** Cleanup the talker before unregistration */
	UE_API void Cleanup();

	/** Maximum size of a single decoded packet */
	int32 MaxUncompressedDataSize;
	/** Maximum size of the outgoing playback queue */
	int32 MaxUncompressedDataQueueSize;
	/** Amount of data currently in the outgoing playback queue */
	int32 CurrentUncompressedDataQueueSize;

	/** Receive side timestamp since last voice packet fragment */
	double LastSeen;
	/** Number of frames starved of audio */
	int32 NumFramesStarved;
	/** Synth component playing this buffer (only valid on remote instances) */
	TWeakObjectPtr<UVoipListenerSynthComponent> VoipSynthComponent;
	/** Cached Talker Ptr. Is checked against map before use to ensure it has not been destroyed. */
	UVOIPTalker* CachedTalkerPtr;
	/** Boolean used to ensure that we only bind the VOIP talker to the SynthComponent's corresponding envelope delegate once. */
	bool bIsEnvelopeBound;
	/** Boolean flag used to tell whether this synth component is currently consuming incoming voice packets. */
	bool bIsActive;
	/** Buffer for outgoing audio intended for procedural streaming */
	mutable FCriticalSection QueueLock;
	TArray<uint8> UncompressedDataQueue;
	/** Per remote talker voice decoding state */
	TSharedPtr<IVoiceDecoder> VoiceDecoder;
	/** Patch splitter to expose incoming audio to multiple outputs. */
	Audio::FPatchSplitter RemoteVoiceOutput;
	/** Loudness of the incoming audio, computed on the remote machine using the microphonei input audio and serialized into the packet. */
	float MicrophoneAmplitude;
};

/**
 * Small class that manages an audio endpoint. Used in FVoiceEngineImpl.
 */
class FVoiceEndpoint : Audio::IAudioMixer
{
public:
	FVoiceEndpoint(const FString& InEndpointName, float InSampleRate, int32 InNumChannels);
	virtual ~FVoiceEndpoint();

	void PatchInOutput(Audio::FPatchOutputStrongPtr& InOutput);


	// Begin of IAudioMixer overrides
	bool OnProcessAudioStream(Audio::AlignedFloatBuffer& OutputBuffer) override;
	void OnAudioStreamShutdown() override;
	// End of IAudioMixer overrides

private:
	int32 NumChannelsComingIn;
	Audio::FAlignedFloatBuffer DownmixBuffer;

	TUniquePtr<Audio::IAudioMixerPlatformInterface> PlatformEndpoint;

	Audio::FAudioMixerOpenStreamParams OpenParams;
	Audio::FAudioPlatformDeviceInfo PlatformDeviceInfo;
	
	Audio::FPatchOutputStrongPtr OutputPatch;
	FCriticalSection OutputPatchCriticalSection;
};


/**
 * Generic implementation of voice engine, using Voice module for capture/codec
 */
class FVoiceEngineImpl : public IVoiceEngine, public FSelfRegisteringExec, public IDeviceChangedListener
{
	class FVoiceSerializeHelper : public FGCObject
	{
		/** Reference to audio components */
		FVoiceEngineImpl* VoiceEngine;
		FVoiceSerializeHelper() :
			VoiceEngine(nullptr)
		{}

	public:

		FVoiceSerializeHelper(FVoiceEngineImpl* InVoiceEngine) :
			VoiceEngine(InVoiceEngine)
		{}
		~FVoiceSerializeHelper() = default;
		
		/** FGCObject interface */
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			// Prevent garbage collection of audio components
			for (FRemoteTalkerData::TIterator It(VoiceEngine->RemoteTalkerBuffers); It; ++It)
			{
				FRemoteTalkerDataImpl& RemoteData = It.Value();
				if (auto& VoipSynthComponent = RemoteData.VoipSynthComponent; VoipSynthComponent.GetEvenIfUnreachable())
				{
					Collector.AddReferencedObject(VoipSynthComponent);
				}
			}
		}
		virtual FString GetReferencerName() const override
		{
			return TEXT("FVoiceEngineImpl::FVoiceSerializeHelper");
		}
	};

	friend class FVoiceSerializeHelper;

	/** Mapping of UniqueIds to the incoming voice data and their audio component */
	typedef TMap<FUniqueNetIdWrapper, FRemoteTalkerDataImpl> FRemoteTalkerData;

	/** Instance name of associated online subsystem */
	FName OnlineInstanceName;

	FLocalVoiceData PlayerVoiceData[MAX_SPLITSCREEN_TALKERS];
	/** Reference to voice capture device */
	TSharedPtr<IVoiceCapture> VoiceCapture;
	/** Reference to voice encoding object */
	TSharedPtr<IVoiceEncoder> VoiceEncoder;

	/** User index currently holding onto the voice interface */
	int32 OwningUserIndex;
	/** Amount of uncompressed data available this frame */
	uint32 UncompressedBytesAvailable;
	/** Amount of compressed data available this frame */
	uint32 CompressedBytesAvailable;
	/** Current frame state of voice capture */
	EVoiceCaptureState::Type AvailableVoiceResult;
	/** Have we stopped capturing voice but are waiting for its completion */
	mutable bool bPendingFinalCapture;
	/** State of voice recording */
	bool bIsCapturing;

	/** Data from voice codec, waiting to send to network. */
	TArray<uint8> CompressedVoiceBuffer;
	/** Data from network playing on an audio component. */
	FRemoteTalkerData RemoteTalkerBuffers;
	/** Voice decompression buffer, shared by all talkers, valid during SubmitRemoteVoiceData */
	TArray<uint8> DecompressedVoiceBuffer;
	/** Serialization helper */
	FVoiceSerializeHelper* SerializeHelper;

	/** Voice Amplitude data to prevent using FRemoteTalkerData if we don't actually require voice*/
	TMap<FUniqueNetIdWrapper, FVoiceAmplitudeData> VoiceAmplitudes;

	/** Audio taps for the full mixdown of all remote players. */
	Audio::FPatchMixerSplitter AllRemoteTalkerAudio;

	/**
	 * Collection of external endpoints that we are sending local or remote audio to. 
	 * Note that we need to wrap each FVoiceEndpoint in a unique pointer to ensure the FVoiceEndpoint itself isn't moved elsewhere.
	 * Otherwise, this will cause a crash in FOutputBuffer::MixNextBuffer(), due to AudioMixer->OnProcessAudioStream(); being called on a stale pointer. 
	 */
	TArray<TUniquePtr<FVoiceEndpoint>> ExternalEndpoints;

protected:
	/**
	 * Determines if the specified index is the owner or not
	 *
	 * @param InIndex the index being tested
	 *
	 * @return true if this is the owner, false otherwise
	 */
	inline virtual bool IsOwningUser(uint32 UserIndex)
	{
		return UserIndex >= 0 && UserIndex < MAX_SPLITSCREEN_TALKERS && OwningUserIndex == UserIndex;
	}

	/** Start capturing voice data */
	UE_API virtual void StartRecording() const;

	/** Stop capturing voice data */
	UE_API virtual void StopRecording() const;

	/** Called when "last half second" is over */
	UE_API virtual void StoppedRecording() const;

	/** @return is active recording occurring at the moment */
	virtual bool IsRecording() const { return bIsCapturing || bPendingFinalCapture; }

private:
	/**
	 * Update the internal state of the voice capturing state
	 * Handles possible continuation waiting for capture stop event
	 */
	UE_API void VoiceCaptureUpdate() const;

	/**
	 * Callback from streaming audio when data is requested for playback
	 *
	 * @param InProceduralWave SoundWave requesting more data
	 * @param SamplesRequired number of samples needed for immediate playback
	 * @param TalkerId id of the remote talker to allocate voice data for
	 */
	UE_API void GenerateVoiceData(USoundWaveProcedural* InProceduralWave, int32 SamplesRequired, const FUniqueNetId& TalkerId);

PACKAGE_SCOPE:

	/** Constructor */
	FVoiceEngineImpl();

	// IVoiceEngine
	UE_API virtual bool Init(int32 MaxLocalTalkers, int32 MaxRemoteTalkers) override;

public:

	UE_API FVoiceEngineImpl(IOnlineSubsystem* InSubsystem);
	UE_API virtual ~FVoiceEngineImpl();

	// IVoiceEngine
	UE_API virtual uint32 StartLocalVoiceProcessing(uint32 LocalUserNum) override;
	UE_API virtual uint32 StopLocalVoiceProcessing(uint32 LocalUserNum) override;

	virtual uint32 StartRemoteVoiceProcessing(const FUniqueNetId& UniqueId) override
	{
		// Not needed
		return ONLINE_SUCCESS;
	}

	virtual uint32 StopRemoteVoiceProcessing(const FUniqueNetId& UniqueId) override
	{
		// Not needed
		return ONLINE_SUCCESS;
	}

	UE_API virtual uint32 RegisterLocalTalker(uint32 LocalUserNum) override;
	UE_API virtual uint32 UnregisterLocalTalker(uint32 LocalUserNum) override;

	virtual uint32 RegisterRemoteTalker(const FUniqueNetId& UniqueId) override
	{
		// Not needed
		return ONLINE_SUCCESS;
	}

	UE_API virtual uint32 UnregisterRemoteTalker(const FUniqueNetId& UniqueId) override;

	virtual bool IsHeadsetPresent(uint32 LocalUserNum) override
	{
		return IsOwningUser(LocalUserNum) ? true : false;
	}

	virtual bool IsLocalPlayerTalking(uint32 LocalUserNum) override
	{
		return (GetVoiceDataReadyFlags() & (LocalUserNum << 1)) != 0;
	}

	virtual bool IsRemotePlayerTalking(const FUniqueNetId& UniqueId) override
	{
		return RemoteTalkerBuffers.Find(FUniqueNetIdWrapper(UniqueId.AsShared())) != nullptr;
	}

	UE_API virtual uint32 GetVoiceDataReadyFlags() const override;
	virtual uint32 SetPlaybackPriority(uint32 LocalUserNum, const FUniqueNetId& RemoteTalkerId, uint32 Priority) override
	{
		// Not supported
		return ONLINE_SUCCESS;
	}

	virtual uint32 ReadLocalVoiceData(uint32 LocalUserNum, uint8* Data, uint32* Size) override { return ReadLocalVoiceData(LocalUserNum, Data, Size, nullptr); }
	UE_API virtual uint32 ReadLocalVoiceData(uint32 LocalUserNum, uint8* Data, uint32* Size, uint64* OutSampleCount) override;

	virtual uint32 SubmitRemoteVoiceData(const FUniqueNetId& RemoteTalkerId, uint8* Data, uint32* Size) 
	{ 
		checkf(false, TEXT("Please use the following function signature instead: SubmitRemoteVoiceData(const FUniqueNetIdWrapper& RemoteTalkerId, uint8* Data, uint32* Size, uint64& InSampleCount)"));
		return 0; 
	}
	UE_API virtual uint32 SubmitRemoteVoiceData(const FUniqueNetIdWrapper& RemoteTalkerId, uint8* Data, uint32* Size, uint64& InSampleCount) override;
	
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API FString GetVoiceDebugState() const override;

	virtual void GetVoiceSettingsOverride(const FUniqueNetIdWrapper& RemoteTalkerId, FVoiceSettings& VoiceSettings) {}


	UE_API virtual Audio::FPatchOutputStrongPtr GetMicrophoneOutput() override;
	UE_API virtual Audio::FPatchOutputStrongPtr GetRemoteTalkerOutput() override;
	UE_API virtual float GetMicrophoneAmplitude(int32 LocalUserNum) override;
	UE_API virtual float GetIncomingAudioAmplitude(const FUniqueNetIdWrapper& RemoteUserId) override;
	UE_API virtual uint32 SetRemoteVoiceAmplitude(const FUniqueNetIdWrapper& RemoteTalkerId, float InAmplitude) override;


	UE_API virtual bool PatchRemoteTalkerOutputToEndpoint(const FString& InDeviceName, bool bMuteInGameOutput = true) override;


	UE_API virtual void DisconnectAllEndpoints() override;


	UE_API virtual bool PatchLocalTalkerOutputToEndpoint(const FString& InDeviceName) override;

protected:
	// FSelfRegisteringExec
	UE_API virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:

	/**
	 * Update the state of all remote talkers, possibly dropping data or the talker entirely
	 */
	UE_API void TickTalkers(float DeltaTime);

	/**
	 * Delegate triggered when an audio component Stop() function is called
	 */
	UE_API void OnAudioFinished();

	/**
	 * Delegate that fixes up remote audio components when the level changes
	 */
	UE_API void OnPostLoadMap(UWorld*);

protected:
	UE_API virtual IOnlineSubsystem*				 GetOnlineSubSystem();
	virtual const TSharedPtr<IVoiceCapture>& GetVoiceCapture() const		{ return VoiceCapture; }
	virtual TSharedPtr<IVoiceCapture>&		 GetVoiceCapture()				{ return VoiceCapture; }
	virtual const TSharedPtr<IVoiceEncoder>& GetVoiceEncoder() const		{ return VoiceEncoder; }
	virtual TSharedPtr<IVoiceEncoder>&		 GetVoiceEncoder()				{ return VoiceEncoder; }
	virtual FRemoteTalkerData&				 GetRemoteTalkerBuffers()		{ return RemoteTalkerBuffers; }
	virtual TArray<uint8>&					 GetCompressedVoiceBuffer()		{ return CompressedVoiceBuffer; }
	virtual TArray<uint8>&					 GetDecompressedVoiceBuffer()	{ return DecompressedVoiceBuffer; }
	virtual FLocalVoiceData*				 GetLocalPlayerVoiceData()		{ return PlayerVoiceData; }
	UE_API virtual int32							 GetMaxVoiceRemainderSize();
	UE_API virtual void							 CreateSerializeHelper();

	virtual void OnDefaultDeviceChanged() override {}
	virtual void OnDeviceRemoved(FString DeviceID) override {}
	//~ End IDeviceChangedListener
};

typedef TSharedPtr<FVoiceEngineImpl, ESPMode::ThreadSafe> FVoiceEngineImplPtr;

#undef UE_API
