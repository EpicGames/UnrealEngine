// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/VoiceInterface.h"
#include "VoicePacketImpl.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

/**
 * The generic implementation of the voice interface 
 */

class FOnlineVoiceImpl : public IOnlineVoice {
protected:
	/** Reference to the main online subsystem */
	class IOnlineSubsystem* OnlineSubsystem;
	/** Reference to the sessions interface */
	class IOnlineSession* SessionInt;
	/** Reference to the profile interface */
	class IOnlineIdentity* IdentityInt;
	/** Reference to the voice engine for acquiring voice data */
	IVoiceEnginePtr VoiceEngine;

	/** Maximum permitted local talkers */
	int32 MaxLocalTalkers;
	/** Maximum permitted remote talkers */
	int32 MaxRemoteTalkers;

	/** State of all possible local talkers */
	TArray<FLocalTalker> LocalTalkers;
	/** State of all possible remote talkers */
	TArray<FRemoteTalker> RemoteTalkers;
	/** Remote players locally muted explicitly */

	TArray<FUniqueNetIdWrapper> SystemMuteList;
	/** All remote players locally muted (super set of SystemMuteList) */
	TArray<FUniqueNetIdWrapper> MuteList;

	/** Time to wait for new data before triggering "not talking" */
	float VoiceNotificationDelta;

	/** Buffered voice data I/O */
	FVoiceDataImpl VoiceData;

	/**
	 * Finds a remote talker in the cached list
	 *
	 * @param UniqueId the net id of the player to search for
	 *
	 * @return pointer to the remote talker or NULL if not found
	 */
	UE_API virtual struct FRemoteTalker* FindRemoteTalker(const FUniqueNetId& UniqueId);

	/**
	 * Is a given id presently muted (either by system mute or game server)
	 *
	 * @param UniqueId the net id to query
	 *
	 * @return true if the net id is muted at all, false otherwise
	 */
	UE_API virtual bool IsLocallyMuted(const FUniqueNetId& UniqueId) const;

	/**
	 * Does a given id exist in the system wide mute list
	 *
	 * @param UniqueId the net id to query
	 *
	 * @return true if the net id is on the system wide mute list, false otherwise
	 */
	UE_API virtual bool IsSystemWideMuted(const FUniqueNetId& UniqueId) const;

PACKAGE_SCOPE:

	FOnlineVoiceImpl() :
		OnlineSubsystem(NULL),
		SessionInt(NULL),
		IdentityInt(NULL),
		VoiceEngine(NULL),
		MaxLocalTalkers(MAX_SPLITSCREEN_TALKERS),
		MaxRemoteTalkers(MAX_REMOTE_TALKERS),
		VoiceNotificationDelta(0.0f)
	{};

	// IOnlineVoice
	UE_API virtual bool Init() override;
	UE_API virtual void ProcessMuteChangeNotification() override;

	UE_API virtual IVoiceEnginePtr CreateVoiceEngine() override;

	/**
	 * Cleanup voice interface
	 */
	UE_API virtual void Shutdown();

	/**
	 * Processes any talking delegates that need to be fired off
	 *
	 * @param DeltaTime the amount of time that has elapsed since the last tick
	 */
	UE_API virtual void ProcessTalkingDelegates(float DeltaTime);

	/**
	 * Reads any data that is currently queued
	 */
	UE_API virtual void ProcessLocalVoicePackets();

	/**
	 * Submits network packets to audio system for playback
	 */
	UE_API virtual void ProcessRemoteVoicePackets();

	/**
	 * Figures out which remote talkers need to be muted for a given local talker
	 *
	 * @param TalkerIndex the talker that needs the mute list checked for
	 * @param PlayerController the player controller associated with this talker
	 */
	UE_API virtual void UpdateMuteListForLocalTalker(int32 TalkerIndex);

public:

	/** Constructor */
	UE_API FOnlineVoiceImpl(class IOnlineSubsystem* InOnlineSubsystem);

	/** Virtual destructor to force proper child cleanup */
	UE_API virtual ~FOnlineVoiceImpl();

	// IOnlineVoice
	UE_API virtual void StartNetworkedVoice(uint8 LocalUserNum) override;
	UE_API virtual void StopNetworkedVoice(uint8 LocalUserNum) override;
    UE_API virtual bool RegisterLocalTalker(uint32 LocalUserNum) override;
	UE_API virtual void RegisterLocalTalkers() override;
    UE_API virtual bool UnregisterLocalTalker(uint32 LocalUserNum) override;
	UE_API virtual void UnregisterLocalTalkers() override;
    UE_API virtual bool RegisterRemoteTalker(const FUniqueNetId& UniqueId) override;
    UE_API virtual bool UnregisterRemoteTalker(const FUniqueNetId& UniqueId) override;
	UE_API virtual void RemoveAllRemoteTalkers() override;
    UE_API virtual bool IsHeadsetPresent(uint32 LocalUserNum) override;
    UE_API virtual bool IsLocalPlayerTalking(uint32 LocalUserNum) override;
	UE_API virtual bool IsRemotePlayerTalking(const FUniqueNetId& UniqueId) override;
	UE_API bool IsMuted(uint32 LocalUserNum, const FUniqueNetId& UniqueId) const override;
	UE_API bool MuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide) override;
	UE_API bool UnmuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide) override;
	UE_API virtual TSharedPtr<class FVoicePacket> SerializeRemotePacket(FArchive& Ar) override;
	UE_API virtual TSharedPtr<class FVoicePacket> GetLocalPacket(uint32 LocalUserNum) override;
	virtual int32 GetNumLocalTalkers() override { return LocalTalkers.Num(); };
	UE_API virtual void ClearVoicePackets() override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual FString GetVoiceDebugState() const override;
	UE_API virtual Audio::FPatchOutputStrongPtr GetMicrophoneOutput() override;
	UE_API virtual Audio::FPatchOutputStrongPtr GetRemoteTalkerOutput() override;
	UE_API virtual float GetAmplitudeOfRemoteTalker(const FUniqueNetId& PlayerId) override;
	UE_API virtual bool PatchRemoteTalkerOutputToEndpoint(const FString& InDeviceName, bool bMuteInGameOutput = true) override;
	UE_API virtual bool PatchLocalTalkerOutputToEndpoint(const FString& InDeviceName) override;
	UE_API virtual void DisconnectAllEndpoints() override;
	//~IOnlineVoice
};

typedef TSharedPtr<FOnlineVoiceImpl, ESPMode::ThreadSafe> FOnlineVoiceImplPtr;

#undef UE_API
