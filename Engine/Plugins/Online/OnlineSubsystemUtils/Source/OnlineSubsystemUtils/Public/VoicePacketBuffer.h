// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "HAL/ThreadSafeBool.h"
#include "Net/VoiceConfig.h"
#include "DSP/FloatArrayMath.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

// Set this to 1 to enforce a critical section between FVoicePacketBuffer::PopAudio and FVoicePacketBuffer::SubmitPacket.
#define SCOPELOCK_VOICE_PACKET_BUFFER 0

enum class EVoipStreamDataFormat : uint8
{
	Float,
	Int16,
	Unknown
};

/************************************************************************/
/* FSortedVoicePacketNode                                               */
/* This structure represents an individual packet in the                */
/* FVoicePacketBuffer.                                                  */
/************************************************************************/
struct FSortedVoicePacketNode
{
	// The actual resulting audio data. allocated ahead of time to minimize memory churn.
	TArray<float> AudioBufferMem;

	// Number of samples.
	int32 BufferNumSamples;

	// updated on every call of PopAudio to keep track of where we are mid-packet.
	int32 SamplesLeft;

	// used for sorting packet nodes. While the packets exist in a contiguous memory
	// pool in the FVoicePacketBuffer. However, the packet buffer is traversed like
	// a singly linked list to allow for fast middle insertion.
	FSortedVoicePacketNode* NextPacket;

	// This is used when we are done with this packet. The index is enqueued so that when the FVoicePacketBuffer
	// gets a new packet, they can Initialize this one.
	int32 IndexInPacketBuffer;
	
	// Actual sample start time for this sample.
	uint64 StartSample;

	// Default constructor.
	FSortedVoicePacketNode()
		: BufferNumSamples(0)
		, SamplesLeft(0)
		, NextPacket(nullptr)
		, IndexInPacketBuffer(INDEX_NONE)
		, StartSample(0)
	{
	}

	// This function is called when a new packet is pushed to avoid reallocating memory.
	void Initialize(const void* InBuffer, uint32 NumBytes, int64 InStartSample,  EVoipStreamDataFormat InFormat)
	{
		check(NumBytes <= UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel());
		NextPacket = nullptr;
		IndexInPacketBuffer = INDEX_NONE;
		StartSample = InStartSample;

		if (InFormat == EVoipStreamDataFormat::Int16)
		{
			BufferNumSamples = NumBytes / sizeof(int16);
		}
		else if (InFormat == EVoipStreamDataFormat::Float)
		{
			BufferNumSamples = NumBytes / sizeof(float);
		}
		

		AudioBufferMem.Reset();
		AudioBufferMem.AddUninitialized(BufferNumSamples);

		float* AudioBuffer = AudioBufferMem.GetData();

		// Handle whatever data format the packet is in.
		switch (InFormat)
		{
			case EVoipStreamDataFormat::Float:
			{
				SamplesLeft = BufferNumSamples;
				FMemory::Memcpy(AudioBuffer, InBuffer, NumBytes);
				break;
			}

			case EVoipStreamDataFormat::Int16:
			{
				SamplesLeft = BufferNumSamples;
				//Convert to float.
				int16* BufferPtr = (int16*)InBuffer;
				Audio::ArrayPcm16ToFloat(MakeArrayView(BufferPtr, BufferNumSamples), MakeArrayView(AudioBuffer, BufferNumSamples));
				break;
			}

			default:
			{
				checkf(false, TEXT("Invalid Format for submitted buffer!"));
				break;
			}
		}
	}
};

/************************************************************************/
/* FVoicePacketBuffer                                                   */
/* This class is used to handle sorting of packets as they arrive.      */
/* This class is intended to be thread safe, as long as PushPacket() nor*/
/* PopAudio() are called by multiple threads.                           */
/************************************************************************/
class FVoicePacketBuffer
{
public:
	// This is the only constructor that should be used with FVoicePacketBuffer.
	UE_API FVoicePacketBuffer(int32 BufferSize, int32 InNumSamplesUntilIdling, uint64 InStartSample);

	// Push a new packet of decompressed audio onto the buffer to be consumed. This should be called on
	// the voice engine thread.
	UE_API void PushPacket(const void* InBuffer, int32 NumBytes, uint64 InStartSample, EVoipStreamDataFormat Format);

	// Pop RequestedSamples of float samples into DestBuffer. Returns the number of "non-silent" samples
	// copied into the buffer- the remainder of the buffer will be filled with silence.
	// This should be called on the audio render thread.
	UE_API int32 PopAudio(float* DestBuffer, uint32 RequestedSamples);

	// Whether we have idled out. This is used to automatically close the audio stream and free resources when unused.
	bool IsIdle() { return bIsIdle; }

	// Get whatever the most recent sample processed is for buffer-accurate timing.
	uint64 GetCurrentSample() { return SampleCounter; }

	// This will return the current amount of samples buffered here. 
	UE_API int32 GetNumBufferedSamples();

	// When called, will seek forward in the buffer by the requested number of samples, skipping and removing the audio we seek past.
	// This will return the number of samples we were able to seek forward by.
	UE_API uint32 DropOldestAudio(uint32 InNumSamples);

	// Clears the buffer.
	UE_API void Reset(int32 InStartSample);

private:

	static int32 SizeOfSample(EVoipStreamDataFormat InFormat)
	{
		switch (InFormat)
		{
			case EVoipStreamDataFormat::Float:
			{
				return sizeof(float);
			}
			case EVoipStreamDataFormat::Int16:
			{
				return sizeof(int16);
			}
			default:
			{
				checkNoEntry();
				return 1;
			}
		}
	}

	// Default constructor. Hidden on purpose.
	UE_API FVoicePacketBuffer();

	// Current packet that the buffer is popping audio from.
	FSortedVoicePacketNode* ListHead;

	// Sample counter used to track silences when buffers are dropped, etc.
	uint64 SampleCounter;

	// This tracks the number of samples of silence we've popped since we last had a viable packet of audio.
	uint32 IdleSampleCounter;

	// bool set when IdleSampleCounter is above NumSamplesUntilIdling.
	FThreadSafeBool bIsIdle;

	// number of samples until we raise the bIsIdle flag.
	uint32 NumSamplesUntilIdling;

	// This is treated as an unsorted memory pool for packets. Packets are overwritten after they are used.
	TArray<FSortedVoicePacketNode> PacketBuffer;

	// After a packet is used, we push its index on the PacketBuffer so that we can Initialize it.
	TQueue<int32> FreedIndicesQueue;

	// The amount of samples of audio we currently have buffered.
	FThreadSafeCounter NumBufferedSamples;

#if SCOPELOCK_VOICE_PACKET_BUFFER
	FCriticalSection ListHeadCriticalSection;
#endif
};

#undef UE_API
