// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/VoiceDataCommon.h"
#include "OnlineSubsystemUtilsPackage.h"
#include "Net/VoiceConfig.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

#define DEBUG_VOICE_PACKET_ENCODING 0

/** Defines the data involved in a voice packet */
class FVoicePacketImpl : public FVoicePacket
{

PACKAGE_SCOPE:

	/** The unique net id of the talker sending the data */
	FUniqueNetIdPtr Sender;
	/** The data that is to be sent/processed */
	TArray<uint8> Buffer;
	/** The current amount of space used in the buffer for this packet */
	uint16 Length;
	uint64 SampleCount; // this is a "sample accurate" representation of the audio data, used for interleaving silent buffers, etc.
	/** Current loudness of the given microphone, in Q15. */
	int16 MicrophoneAmplitude;

public:
	/** Zeros members and validates the assumptions */
	FVoicePacketImpl() :
		Sender(NULL),
		Length(0),
		MicrophoneAmplitude(0)
	{
		Buffer.Empty(UVOIPStatics::GetMaxVoiceDataSize());
		Buffer.AddUninitialized(UVOIPStatics::GetMaxVoiceDataSize());
	}

	/** Should only be used by TSharedPtr and FVoiceData */
	virtual ~FVoicePacketImpl() = default;

	/**
	 * Copies another packet
	 *
	 * @param Other packet to copy
	 */
	UE_API FVoicePacketImpl(const FVoicePacketImpl& Other);

	UE_API virtual void ResetData();

	//~ Begin FVoicePacket interface
	UE_API virtual uint16 GetTotalPacketSize() override;
	UE_API virtual uint16 GetBufferSize() override;
	UE_API virtual FUniqueNetIdPtr GetSender() override;
	virtual bool IsReliable() override { return false; }
	UE_API virtual void Serialize(class FArchive& Ar) override;
	virtual uint64 GetSampleCounter() const override { return SampleCount; }
	//~ End FVoicePacket interface
};

/** Holds the current voice packet data state */
struct FVoiceDataImpl
{
	/** Data used by the local talkers before sent */
	FVoicePacketImpl LocalPackets[MAX_SPLITSCREEN_TALKERS];
	/** Holds the set of received packets that need to be processed */
	FVoicePacketList RemotePackets;

	FVoiceDataImpl() {}
	~FVoiceDataImpl() = default;
};

#undef UE_API
