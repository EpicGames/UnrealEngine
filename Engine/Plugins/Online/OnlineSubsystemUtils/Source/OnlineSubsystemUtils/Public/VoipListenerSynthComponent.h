// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "DSP/MultithreadedPatching.h"
#include "VoipListenerSynthComponent.generated.h"

#define UE_API ONLINESUBSYSTEMUTILS_API

class FVoicePacketBuffer;
enum class EVoipStreamDataFormat : uint8;

#define DEBUG_BUFFERING 0

#if DEBUG_BUFFERING
class FDebugFMTone
{
public:
	FDebugFMTone() = default;
	FDebugFMTone(float InSampleRate, float InCarrierFreq, float InModFreq, float InCarrierAmp, float InModAmp);

	void Generate(int16* BufferPtr, int32 NumSamples);

private:
	float SampleRate;
	float CarrierFreq;
	float CarrierAmp;

	float ModFreq;
	float ModAmp;

	int32 n;
};
#endif

UCLASS(MinimalAPI)
class UVoipListenerSynthComponent : public USynthComponent
{

	GENERATED_BODY()

	// Called when synth is created
	UE_API virtual bool Init(int32& SampleRate) override;

	// Called to generate more audio
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

public:
	UE_API ~UVoipListenerSynthComponent();

	/**
	 * Start accepting packets with a given delay for buffering.
	 * @param BeginningSampleCount The expected sample count of the first packet buffered.
	        after this function is called.
	 * @param BufferSize that amount of packets we should reserve memory for.
	 * @param JitterDelay the amount of time we wait before we start playing audio from submitted packets.
	*/
	UE_API void OpenPacketStream(uint64 BeginningSampleCount, int32 BufferSize, float JitterDelay);

	/**
	 * Close an open packet stream and start outputting silence.
	*/
	UE_API void ClosePacketStream();

	/*
	 * Reset the packet stream buffer if it's already open. Using this call prevents memory churn.
	 * @param InStartSample The expected sample count of the first packet buffered after this function is called.
	 * @param JitterDelay The amount of time we wait before we start playing audio from submitted packets.
	*/
	UE_API void ResetBuffer(int32 InStartSample, float JitterDelay);

	/*
	 * Submit a packet to be enqueued for playback. This should be called AFTER a the stream is opened with OpenPacketStream().
	 * @param InBuffer pointer to a single channel buffer of audio samples.
	 * @param NumBytes the number of bytes referenced by InBuffer.
	 * @param InStartSample the Starting Sample associated with this packet.
	 * @param DataFormat the format of this sample (float, fixed16, etc.)
	*/
	UE_API void SubmitPacket(void* InBuffer, int32 NumBytes, int64 InStartSample, EVoipStreamDataFormat DataFormat);

	/**
	 * Optionally connect this VOIP Synth Component to a splitter.
	 */
	UE_API void ConnectToSplitter(Audio::FPatchMixerSplitter& InSplitter);

	/*
	 * Thread safe way to get whether this synth component still has buffered packets of audio to play back.
	 * This function is used by the Voice Engine to clean up voice sounds when they are not playing back audio
	 * to make space for other sounds to be rendered.

	 * @returns true if this synth component is out of audio to play. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Voice|Sender")
	UE_API bool IsIdling();

	/*
	 * This function returns the more recent sample counter of the packet currently getting played.
	 * This may prove useful for strongly timing events off of VOIP playback, since VOIP playback has
	 * a highly variable amount of delay.
	*/
	UE_API uint64 GetSampleCounter();


	UE_API virtual void BeginDestroy() override;

private:
	/**
	 * This call will check how much audio we have buffered and drop the oldest audio if necessary.
	 */
	void ForceResync();


	/** This patch input can optionally be set  */
	Audio::FPatchInput ExternalSend;

	/** This is allocated on OpenPacketStream() */
	TUniquePtr<FVoicePacketBuffer> PacketBuffer;
	FCriticalSection PacketBufferCriticalSection;

	float MySampleRate;

	int32 PreDelaySampleCounter;

	/** The amount of audio we keep buffered to reduce underruns. */
	float JitterDelayInSeconds;

#if DEBUG_BUFFERING
	FDebugFMTone FMToneGenerator;
#endif

};

#undef UE_API
